// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryCacheUSD : ModuleRules
{
	public GeometryCacheUSD(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"GeometryCache",
				"GeometryCacheStreamer",
				"RenderCore",
				"RHI",
				"UnrealUSDWrapper",
				"USDUtilities"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"USDClasses",
            }
        );

		PrivateIncludePathModuleNames.Add("DerivedDataCache");
	}
}
