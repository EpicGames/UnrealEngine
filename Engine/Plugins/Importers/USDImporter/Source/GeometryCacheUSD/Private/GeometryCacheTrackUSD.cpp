// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrackUSD.h"

#include "USDErrorUtils.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"

#include "GeometryCacheHelpers.h"
#include "GeometryCacheUSDStream.h"
#include "HAL/IConsoleManager.h"
#include "IGeometryCacheStreamer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheTrackUSD)

static bool GDisableGeoCacheTracks = false;
static FAutoConsoleVariableRef CVarDisableGeoCacheTracks(
	TEXT("USD.DisableGeoCacheTracks"),
	GDisableGeoCacheTracks,
	TEXT("Set to true to disable geometry cache tracks in Sequencer and drive them by the Time property instead. The stage must be reloaded after "
		 "changing this value.")
);

UGeometryCacheTrackUsd::UGeometryCacheTrackUsd()
	: FramesPerSecond(24.0)
	, StartFrameIndex(0)
	, EndFrameIndex(0)
{
}

void UGeometryCacheTrackUsd::BeginDestroy()
{
	UnloadUsdStage();

	UnregisterStream();
	UsdStream.Reset();

	Super::BeginDestroy();
}

void UGeometryCacheTrackUsd::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	// Include only memory usage for data that is part of the track itself
	// Stream data should only be relevant for the streamer, not the asset cache

	// This is an additional copy that lives on the track
	MeshData.GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SampleInfos.GetAllocatedSize());
}

const bool UGeometryCacheTrackUsd::UpdateMeshData(
	const float Time,
	const bool bLooping,
	int32& InOutMeshSampleIndex,
	FGeometryCacheMeshData*& OutMeshData
)
{
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);

	// If InOutMeshSampleIndex equals -1 (first creation) update the OutVertices and InOutMeshSampleIndex
	// Update the Vertices and Index if SampleIndex is different from the stored InOutMeshSampleIndex
	if (InOutMeshSampleIndex == -1 || SampleIndex != InOutMeshSampleIndex)
	{
		if (GetMeshData(SampleIndex, MeshData))
		{
			OutMeshData = &MeshData;
			InOutMeshSampleIndex = SampleIndex;
			return true;
		}
	}
	return false;
}

const bool UGeometryCacheTrackUsd::UpdateBoundsData(
	const float Time,
	const bool bLooping,
	const bool bIsPlayingBackward,
	int32& InOutBoundsSampleIndex,
	FBox& OutBounds
)
{
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);

	const FGeometryCacheTrackSampleInfo& SampledInfo = GetSampleInfo(SampleIndex);
	if (InOutBoundsSampleIndex != SampleIndex)
	{
		OutBounds = SampledInfo.BoundingBox;
		InOutBoundsSampleIndex = SampleIndex;
		return true;
	}
	return false;
}

const int32 UGeometryCacheTrackUsd::FindSampleIndexFromTime(const float Time, const bool bLooping) const
{
	// Time is relative to the start of the section on the track
	float SampleTime = Time;
	if (bLooping)
	{
		SampleTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
	}
	// Which is converted to an index
	int32 FrameIndex = FMath::FloorToInt(Time * FramesPerSecond);

	// The final computed frame index must be offset by the start frame index
	return FMath::Clamp(FrameIndex + StartFrameIndex, StartFrameIndex, EndFrameIndex - 1);
}

float UGeometryCacheTrackUsd::GetTimeFromSampleIndex(int32 SampleIndex) const
{
	// Time is relative to the start of the section on the track
	return float((SampleIndex - StartFrameIndex) / FramesPerSecond);
}

void UGeometryCacheTrackUsd::GetFractionalFrameIndexFromTime(const float Time, const bool bLooping, int& OutFrameIndex, float& OutFraction) const
{
	OutFrameIndex = FindSampleIndexFromTime(Time, bLooping);

	float AdjustedTime = Time;
	if (bLooping)
	{
		AdjustedTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
	}
	// Time at ThisFrameIndex with index normalized to 0
	const float FrameIndexTime = (OutFrameIndex - StartFrameIndex) / FramesPerSecond;
	OutFraction = float((AdjustedTime - FrameIndexTime) * FramesPerSecond);
	// The fractional part is clamped to (0, 1) for subframe interpolation since the FrameIndex is floored
	OutFraction = FMath::Clamp(OutFraction, 0.0f, 1.0f);
}

const FGeometryCacheTrackSampleInfo& UGeometryCacheTrackUsd::GetSampleInfo(float Time, bool bLooping)
{
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);
	return GetSampleInfo(SampleIndex);
}

const FGeometryCacheTrackSampleInfo& UGeometryCacheTrackUsd::GetSampleInfo(int32 SampleIndex)
{
	if (SampleIndex < 0)
	{
		return FGeometryCacheTrackSampleInfo::EmptySampleInfo;
	}

	if (SampleInfos.Num() == 0)
	{
		if (Duration > 0.f)
		{
			const int32 NumFrames = EndFrameIndex - StartFrameIndex + 1;
			SampleInfos.SetNum(NumFrames);
		}
		else
		{
			return FGeometryCacheTrackSampleInfo::EmptySampleInfo;
		}
	}

	// The sample info index must start from 0, while the sample index is between the range of the animation
	const int32 SampleInfoIndex = SampleIndex - StartFrameIndex;

	FGeometryCacheTrackSampleInfo& CurrentSampleInfo = SampleInfos[SampleInfoIndex];

	if (CurrentSampleInfo.SampleTime == 0.0f && CurrentSampleInfo.NumVertices == 0 && CurrentSampleInfo.NumIndices == 0)
	{
		FGeometryCacheMeshData TempMeshData;
		if (GetMeshData(SampleIndex, TempMeshData))
		{
			float Time = GetTimeFromSampleIndex(SampleIndex);
			CurrentSampleInfo = FGeometryCacheTrackSampleInfo(
				Time,
				(FBox)TempMeshData.BoundingBox,
				TempMeshData.Positions.Num(),
				TempMeshData.Indices.Num()
			);
		}
		else
		{
			// This shouldn't really happen but if it does make sure this is initialized,
			// as it can crash/throw ensures depending on the uninitialized memory values,
			// while it will only be slightly visually glitchy with a zero bounding box
			CurrentSampleInfo.BoundingBox.Init();
		}
	}

	return CurrentSampleInfo;
}

bool UGeometryCacheTrackUsd::GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData)
{
	const bool bLooping = true;
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);
	return GetMeshData(SampleIndex, OutMeshData);
}

bool UGeometryCacheTrackUsd::GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	return GetMeshData(SampleIndex, OutMeshData);
}

bool UGeometryCacheTrackUsd::GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (IGeometryCacheStreamer::Get().IsTrackRegistered(this))
	{
		return IGeometryCacheStreamer::Get().TryGetFrameData(this, SampleIndex, OutMeshData);
	}
	return false;
}

bool UGeometryCacheTrackUsd::LoadUsdStage()
{
	if (CurrentStagePinned)
	{
		// Already loaded
		return true;
	}

	if (CurrentStageWeak)
	{
		// Upgrade our weak pointer if its not invalid already
		CurrentStagePinned = CurrentStageWeak;
		return true;
	}
	else if (!StageRootLayerPath.IsEmpty())
	{
		USD_LOG_WARNING(
			TEXT("UGeometryCacheTrackUsd is reopening the stage '%s' to stream in frames for the geometry cache generated for prim '%s'"),
			*StageRootLayerPath,
			*PrimPath
		);

		// Reopen the stage. If our weak pointer is no longer valid then nothing cared about keeping that
		// stage alive anyway, so it's likely not a problem if we start reading frames from the reopened stage
		// and abandon any previous in-memory changes we had, if any.
		// Keep in mind currently it's effectively impossible to get in here at all anyway, as we'll only
		// have UGeometryCacheTrackUsd streaming stuff if a stage actor caused it to stream more frames,
		// and in that case that stage actor would have kept our stage opened.
		// Not using the stage cache here because there's currently no way of knowing when to erase it from the cache
		// after we're done with it, and the UGeometryCacheTrackUsd shouldn't have authority to blindly just remove it
		// from the cache as the user may have place it there intentionally
		const bool bUseStageCache = false;
		CurrentStagePinned = UnrealUSDWrapper::OpenStage(*StageRootLayerPath, EUsdInitialLoadSet::LoadAll, bUseStageCache);
		CurrentStageWeak = CurrentStagePinned;
		return true;
	}

	USD_LOG_WARNING(TEXT("UGeometryCacheTrackUsd track failed to access USD stage to stream requested frames"));
	return false;
}

void UGeometryCacheTrackUsd::UnloadUsdStage()
{
	CurrentStagePinned = UE::FUsdStage();
}

void UGeometryCacheTrackUsd::Initialize(
	const UE::FUsdStage& InStage,
	const FString& InPrimPath,
	int32 InStartFrameIndex,
	int32 InEndFrameIndex,
	FReadUsdMeshFunction InReadFunc
)
{
	CurrentStagePinned = InStage;
	CurrentStageWeak = CurrentStagePinned;
	StageRootLayerPath = CurrentStagePinned ? CurrentStagePinned.GetRootLayer().GetRealPath() : FString();

	PrimPath = InPrimPath;
	StartFrameIndex = InStartFrameIndex;
	EndFrameIndex = InEndFrameIndex;
	FramesPerSecond = CurrentStagePinned.GetTimeCodesPerSecond();

	if (FramesPerSecond == 0)
	{
		USD_LOG_WARNING(TEXT("Invalid USD GeometryCache FPS detected. Falling back to 1 FPS"));
		FramesPerSecond = 1;
	}

	const int32 NumFrames = EndFrameIndex - StartFrameIndex + 1;
	Duration = NumFrames / FramesPerSecond;

	UsdStream.Reset(new FGeometryCacheUsdStream(this, InReadFunc));
}

void UGeometryCacheTrackUsd::UpdateTime(float Time, bool bLooping)
{
	if (UsdStream)
	{
		int32 FrameIndex = FindSampleIndexFromTime(Time, bLooping);
		UsdStream->UpdateCurrentFrameIndex(FrameIndex);
	}
}

void UGeometryCacheTrackUsd::RegisterStream()
{
	if (UsdStream)
	{
		const bool bNeedPrefetch = !IGeometryCacheStreamer::Get().IsTrackRegistered(this);
		IGeometryCacheStreamer::Get().RegisterTrack(this, UsdStream.Get());
		if (bNeedPrefetch)
		{
			UsdStream->Prefetch(StartFrameIndex);
		}
	}
}

void UGeometryCacheTrackUsd::UnregisterStream()
{
	IGeometryCacheStreamer::Get().UnregisterTrack(this);
}
