// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheUSDComponent.h"

#include "GeometryCache.h"
#include "GeometryCacheTrackUSD.h"
#include "GeometryCacheUSDSceneProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheUSDComponent)

FPrimitiveSceneProxy* UGeometryCacheUsdComponent::CreateSceneProxy()
{
	return new FGeometryCacheUsdSceneProxy(this);
}

void UGeometryCacheUsdComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	// Setup the members that are not included in the duplication
	PlayDirection = 1.f;

	SetupTrackData();
}

void UGeometryCacheUsdComponent::OnRegister()
{
	if (GeometryCache != nullptr)
	{
		for (UGeometryCacheTrack* Track : GeometryCache->Tracks)
		{
			if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
			{
				UsdTrack->RegisterStream();
			}
		}
	}

	UGeometryCacheComponent::OnRegister();
}

void UGeometryCacheUsdComponent::OnUnregister()
{
	if (GeometryCache != nullptr)
	{
		for (UGeometryCacheTrack* Track : GeometryCache->Tracks)
		{
			if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
			{
				UsdTrack->UnregisterStream();
			}
		}
	}

	UGeometryCacheComponent::OnUnregister();
}
