// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FGeometryCacheUsdModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FGeometryCacheUsdModule, GeometryCacheUSD);
