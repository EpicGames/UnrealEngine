// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheUSDSceneProxy.h"
#include "GeometryCacheTrackUSD.h"
#include "GeometryCacheUSDComponent.h"
#include "SceneInterface.h"

FGeometryCacheUsdSceneProxy::FGeometryCacheUsdSceneProxy(UGeometryCacheUsdComponent* Component)
	: FGeometryCacheSceneProxy(
		Component,
		[this]()
		{
			return new FGeomCacheTrackUsdProxy(GetScene().GetFeatureLevel());
		}
	)
{
}

bool FGeomCacheTrackUsdProxy::UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomCacheTrackUsdProxy::UpdateMeshData);

	if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
	{
		FGeometryCacheMeshData* PtrMeshData = nullptr;
		bool bResult = UsdTrack->UpdateMeshData(Time, bLooping, InOutMeshSampleIndex, PtrMeshData);
		if (bResult)
		{
			OutMeshData = *PtrMeshData;
		}
		return bResult;
	}
	return FGeomCacheTrackProxy::UpdateMeshData(Time, bLooping, InOutMeshSampleIndex, OutMeshData);
}

bool FGeomCacheTrackUsdProxy::GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomCacheTrackUsdProxy::GetMeshData);

	if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
	{
		return UsdTrack->GetMeshData(SampleIndex, OutMeshData);
	}
	return FGeomCacheTrackProxy::GetMeshData(SampleIndex, OutMeshData);
}

bool FGeomCacheTrackUsdProxy::IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomCacheTrackUsdProxy::IsTopologyCompatible);

	if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
	{
		const int32 NumVerticesA = UsdTrack->GetSampleInfo(SampleIndexA).NumVertices;
		const int32 NumVerticesB = UsdTrack->GetSampleInfo(SampleIndexB).NumVertices;

		return NumVerticesA == NumVerticesB;
	}
	return FGeomCacheTrackProxy::IsTopologyCompatible(SampleIndexA, SampleIndexB);
}

const FVisibilitySample& FGeomCacheTrackUsdProxy::GetVisibilitySample(float Time, const bool bLooping) const
{
	if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
	{
		// Assume the track is visible for its whole duration
		return FVisibilitySample::VisibleSample;
	}
	return FGeomCacheTrackProxy::GetVisibilitySample(Time, bLooping);
}

void FGeomCacheTrackUsdProxy::FindSampleIndexesFromTime(
	float Time,
	bool bLooping,
	bool bIsPlayingBackwards,
	int32& OutFrameIndex,
	int32& OutNextFrameIndex,
	float& InInterpolationFactor
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeomCacheTrackUsdProxy::FindSampleIndexesFromTime);

	if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
	{
		UsdTrack->GetFractionalFrameIndexFromTime(Time, bLooping, OutFrameIndex, InInterpolationFactor);
		OutNextFrameIndex = OutFrameIndex + 1;

		// If playing backwards the logical order of previous and next is reversed
		if (bIsPlayingBackwards)
		{
			Swap(OutFrameIndex, OutNextFrameIndex);
			InInterpolationFactor = 1.0f - InInterpolationFactor;
		}
	}
	else
	{
		FGeomCacheTrackProxy::FindSampleIndexesFromTime(Time, bLooping, bIsPlayingBackwards, OutFrameIndex, OutNextFrameIndex, InInterpolationFactor);
	}
}
