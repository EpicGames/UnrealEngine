// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheUSDStream.h"

#include "USDGeomMeshConversion.h"

#include "DerivedDataCacheInterface.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackUSD.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreMisc.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

static bool GUsdStreamCacheInDDC = true;
static FAutoConsoleVariableRef CVarUsdStreamCacheInDDC(
	TEXT("GeometryCache.Streamer.UsdStream.CacheInDDC"),
	GUsdStreamCacheInDDC,
	TEXT("Cache the streamed USD mesh data in the DDC")
);

static int32 kUsdReadConcurrency = 10;

// If UsdStream derived data needs to be rebuilt (new format, serialization
// differences, etc.) replace the version GUID below with a new one.
// In case of merge conflicts with DDC versions, you MUST generate a new GUID
// and set this new GUID as the version.
#define USDSTREAM_DERIVED_DATA_VERSION TEXT("7F54F423DF3247B7B77B9E2B356FBE83")

class FUsdStreamDDCUtils
{
private:
	static const FString& GetUsdStreamDerivedDataVersion()
	{
		static FString CachedVersionString(USDSTREAM_DERIVED_DATA_VERSION);
		return CachedVersionString;
	}

	static FString BuildDerivedDataKey(const FString& KeySuffix)
	{
		return FDerivedDataCacheInterface::BuildCacheKey(TEXT("USDSTREAM_"), *GetUsdStreamDerivedDataVersion(), *KeySuffix);
	}

public:
#if USE_USD_SDK
	static FString GetUsdStreamDDCKey(
		const UE::FUsdStage& Stage,
		const FString& PrimPath,
		int32 FrameIndex,
		const UsdToUnreal::FUsdMeshConversionOptions& MeshConversionOptions
	)
	{
		FString Hash;
		{
			FMD5 MD5;

			UsdUtils::HashGeomMeshPrim(Stage, PrimPath, FrameIndex, MD5);

			// Also hash these options because they may affect how the mesh conversion functions end up parsing
			// the mesh data. This may seem a bit awkward, but it would be weirder still to provide MeshConversionOptions
			// to HashGeomMeshPrim directly, as it already has a TimeCode parameter.
			// To remove that and just use the TimeCode within the FUsdMeshConversionOptions instead would mean we would
			// need to take a local copy of MeshConversionOptions to set FrameIndex as its TimeCode in here (as we can't
			// modify MeshConversionOptions directly as it's owned by the UsdTrack and we may be called from a thread)
			MD5.Update((uint8*)&MeshConversionOptions.SubdivisionLevel, sizeof(MeshConversionOptions.SubdivisionLevel));
			MD5.Update((uint8*)&MeshConversionOptions.PurposesToLoad, sizeof(MeshConversionOptions.PurposesToLoad));
			MD5.Update((uint8*)&MeshConversionOptions.bMergeIdenticalMaterialSlots, sizeof(MeshConversionOptions.bMergeIdenticalMaterialSlots));

			uint8 Digest[16];
			MD5.Final(Digest);

			for (int32 i = 0; i < 16; ++i)
			{
				Hash += FString::Printf(TEXT("%02x"), Digest[i]);
			}
		}

		return BuildDerivedDataKey(Hash);
	}
#endif
};

FGeometryCacheUsdStream::FGeometryCacheUsdStream(UGeometryCacheTrackUsd* InUsdTrack, FReadUsdMeshFunction InReadFunc)
	: FGeometryCacheStreamBase(
		kUsdReadConcurrency,
		FGeometryCacheStreamDetails{
			InUsdTrack->EndFrameIndex - InUsdTrack->StartFrameIndex + 1,
			float((InUsdTrack->EndFrameIndex - InUsdTrack->StartFrameIndex + 1) / InUsdTrack->FramesPerSecond),
			float(1.0f / InUsdTrack->FramesPerSecond),
			InUsdTrack->StartFrameIndex,
			InUsdTrack->EndFrameIndex
		}
	)
	, UsdTrack(InUsdTrack)
	, ReadFunc(InReadFunc)
	, bReadyForRead(false)
{
}

bool FGeometryCacheUsdStream::GetFrameData(int32 FrameIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (!FGeometryCacheStreamBase::GetFrameData(FrameIndex, OutMeshData))
	{
		// If the user requested a frame that isn't loaded yet, synchronously fetch it right away
		// or else UGeometryCacheTrackUsd::GetSampleInfo may return invalid bounding boxes that may lead
		// to issues, and wouldn't otherwise be updated until that frame is requested again.
		// It may lead to a bit of stuttering when animating through an unloaded section with the sequencer
		// or by dragging the Time property of the stage actor, but the alternative would be a spam of
		// warnings on the output log and glitchy bounding boxes on the level
		if (IsInGameThread())
		{
			LoadFrameData(FrameIndex);
			return FGeometryCacheStreamBase::GetFrameData(FrameIndex, OutMeshData);
		}
		else
		{
			return false;
		}
	}
	return true;
}

void FGeometryCacheUsdStream::UpdateRequestStatus(TArray<int32>& OutFramesCompleted)
{
	FGeometryCacheStreamBase::UpdateRequestStatus(OutFramesCompleted);

	// We're fully done fetching what we need from USD for now, we can drop the track's strong stage reference so that the stage
	// can close if needed
	if (FramesNeeded.Num() == 0 && FramesRequested.Num() == 0 && UsdTrack && UsdTrack->CurrentStagePinned)
	{
		UsdTrack->UnloadUsdStage();
		bReadyForRead = false;
	}
}

void FGeometryCacheUsdStream::PrepareRead()
{
	// Do this before calling the ReadFunc as FUsdStreamDDCUtils::GetUsdStreamDDCKey also needs a valid stage to work with
	if (!bReadyForRead)
	{
		bReadyForRead = UsdTrack->LoadUsdStage();
	}
}

void FGeometryCacheUsdStream::GetMeshData(int32 FrameIndex, int32 ConcurrencyIndex, FGeometryCacheMeshData& OutMeshData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCacheUsdStream::GetMeshData);

	if (!bReadyForRead)
	{
		return;
	}

#if USE_USD_SDK
	// Get the mesh data straight from the Alembic file or from the DDC if it's already cached
	if (GUsdStreamCacheInDDC)
	{
		const FString& UsdPrimPath = UsdTrack->PrimPath;
		const FString DerivedDataKey = FUsdStreamDDCUtils::GetUsdStreamDDCKey(
			UsdTrack->CurrentStagePinned,
			UsdPrimPath,
			FrameIndex,
			UsdTrack->MeshConversionOptions
		);

		if (!DerivedDataKey.IsEmpty())
		{
			TArray<uint8> DerivedData;
			if (GetDerivedDataCacheRef().GetSynchronous(*DerivedDataKey, DerivedData, UsdPrimPath))
			{
				FMemoryReader Ar(DerivedData, /*bIsPersistent=*/true);
				Ar << OutMeshData;
			}
			else
			{
				// Don't cache the data if it failed to read it
				if (ReadFunc(UsdTrack, FrameIndex, OutMeshData))
				{
					FMemoryWriter Ar(DerivedData, true);
					Ar << OutMeshData;

					GetDerivedDataCacheRef().Put(*DerivedDataKey, DerivedData, UsdPrimPath);
				}
			}
		}
		// If the key is empty, it means the prim was invalid or not a mesh, so don't do anything
	}
	else
#endif
	{
		// Synchronously load the requested frame data
		ReadFunc(UsdTrack, FrameIndex, OutMeshData);
	}
}
