// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDGeomMeshConversion.h"

#include "UsdWrappers/UsdStage.h"

#include "CoreMinimal.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "GeometryCacheTrackUSDTypes.h"
#include "GeometryCacheUSDStream.h"

#include "GeometryCacheTrackUSD.generated.h"

#define UE_API GEOMETRYCACHEUSD_API

/** GeometryCacheTrack for querying USD */
UCLASS(collapsecategories, hidecategories = Object, BlueprintType, config = Engine)

class UGeometryCacheTrackUsd : public UGeometryCacheTrack
{
	GENERATED_BODY()

	UE_API UGeometryCacheTrackUsd();

public:
	UE_API void Initialize(
		const UE::FUsdStage& InStage,
		const FString& InPrimPath,
		int32 InStartFrameIndex,
		int32 InEndFrameIndex,
		FReadUsdMeshFunction InReadFunc
	);

	//~ Begin UObject Interface.
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject Interface.

	//~ Begin UGeometryCacheTrack Interface.
	UE_API virtual const bool UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData)
		override;
	UE_API virtual const bool UpdateBoundsData(
		const float Time,
		const bool bLooping,
		const bool bIsPlayingBackward,
		int32& InOutBoundsSampleIndex,
		FBox& OutBounds
	) override;
	UE_API virtual const FGeometryCacheTrackSampleInfo& GetSampleInfo(float Time, const bool bLooping) override;
	UE_API virtual bool GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData) override;
	UE_API virtual bool GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	UE_API virtual void UpdateTime(float Time, bool bLooping) override;
	//~ End UGeometryCacheTrack Interface.

	UE_API const int32 FindSampleIndexFromTime(const float Time, const bool bLooping) const;
	UE_API float GetTimeFromSampleIndex(int32 SampleIndex) const;
	UE_API void GetFractionalFrameIndexFromTime(const float Time, const bool bLooping, int& OutFrameIndex, float& OutFraction) const;
	// GetSampleInfo version that avoids converting time to index. Prefer this version when the index is already available
	UE_API const FGeometryCacheTrackSampleInfo& GetSampleInfo(int32 SampleIndex);

	UE_API bool GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData);

	// Upgrades our CurrentStageWeak into CurrentStagePinned, or re-opens the stage if its stale. Returns whether the stage was successfully opened or
	// not
	UE_API bool LoadUsdStage();

	// Discards our CurrentStagePinned to release the stage
	UE_API void UnloadUsdStage();

	UE_API void RegisterStream();
	UE_API void UnregisterStream();

public:
	double FramesPerSecond;
	int32 StartFrameIndex;
	int32 EndFrameIndex;

	FString PrimPath;

#if USE_USD_SDK
	UsdToUnreal::FUsdMeshConversionOptions MeshConversionOptions;
#endif

	UE::FUsdStage CurrentStagePinned;
	UE::FUsdStageWeak CurrentStageWeak;
	FString StageRootLayerPath;

private:
	FGeometryCacheMeshData MeshData;
	TArray<FGeometryCacheTrackSampleInfo> SampleInfos;

	TUniquePtr<FGeometryCacheUsdStream> UsdStream;
};

#undef UE_API
