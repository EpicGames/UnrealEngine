// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "UObject/WeakObjectPtr.h"

class UGeometryCacheTrackUsd;
struct FGeometryCacheMeshData;

typedef TFunction<bool(const TWeakObjectPtr<UGeometryCacheTrackUsd>, float Time, FGeometryCacheMeshData&)> FReadUsdMeshFunction;
