// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCacheComponent.h"

#include "GeometryCacheUSDComponent.generated.h"

#define UE_API GEOMETRYCACHEUSD_API

/** GeometryCacheUSDComponent, encapsulates a transient GeometryCache asset instance that fetches its data from a USD file and implements
 * functionality for rendering and playback */
UCLASS(HideDropDown, ClassGroup = (Rendering), meta = (DisplayName = "USD Geometry Cache"), Experimental, ClassGroup = Experimental)

class UGeometryCacheUsdComponent : public UGeometryCacheComponent
{
	GENERATED_BODY()

public:
	//~ Begin UObject Interface
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UActorComponent Interface.
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	//~ End UActorComponent Interface.
};

#undef UE_API
