// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCacheSceneProxy.h"

/** Scene proxy that uses FGeomCacheTrackUsdProxy */
class FGeometryCacheUsdSceneProxy final : public FGeometryCacheSceneProxy
{
public:
	FGeometryCacheUsdSceneProxy(class UGeometryCacheUsdComponent* Component);
};

/** Track proxy that gets its data from a GeometryCacheUsdTrack */
class FGeomCacheTrackUsdProxy : public FGeomCacheTrackProxy
{
public:
	FGeomCacheTrackUsdProxy(ERHIFeatureLevel::Type InFeatureLevel)
		: FGeomCacheTrackProxy(InFeatureLevel)
	{
	}

	//~ Begin FGeomCacheTrackProxy Interface
	virtual bool UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	virtual bool GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	virtual bool IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB) override;
	virtual const FVisibilitySample& GetVisibilitySample(float Time, const bool bLooping) const override;
	virtual void FindSampleIndexesFromTime(
		float Time,
		bool bLooping,
		bool bIsPlayingBackwards,
		int32& OutFrameIndex,
		int32& OutNextFrameIndex,
		float& InterpolationFactor
	) override;
	//~ End FGeomCacheTrackProxy Interface
};
