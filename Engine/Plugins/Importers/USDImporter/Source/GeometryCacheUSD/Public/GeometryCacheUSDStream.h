// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCacheStreamBase.h"
#include "GeometryCacheTrackUSDTypes.h"

#include <atomic>

class UGeometryCacheTrackUsd;

class FGeometryCacheUsdStream : public FGeometryCacheStreamBase
{
public:
	FGeometryCacheUsdStream(UGeometryCacheTrackUsd* InUsdTrack, FReadUsdMeshFunction InReadFunc);

	//~ Begin IGeometryCacheStream Interface
	virtual bool GetFrameData(int32 FrameIndex, FGeometryCacheMeshData& OutMeshData) override;
	virtual void UpdateRequestStatus(TArray<int32>& OutFramesCompleted) override;
	//~ End IGeometryCacheStream Interface

protected:
	virtual void PrepareRead() override;
	virtual void GetMeshData(int32 FrameIndex, int32 ConcurrencyIndex, FGeometryCacheMeshData& OutMeshData) override;

	UGeometryCacheTrackUsd* UsdTrack;
	FReadUsdMeshFunction ReadFunc;
	std::atomic<bool> bReadyForRead;
};
