// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_USDAssetCache.h"

#include "USDAssetCache3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_USDAssetCache)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FText UAssetDefinition_UsdAssetCache::GetAssetDisplayName() const
{
	return LOCTEXT("AssetTypeActions_AssetCache", "USD Asset Cache");
}

FLinearColor UAssetDefinition_UsdAssetCache::GetAssetColor() const
{
	return FColor(32, 145, 208);
}

TSoftClassPtr<UObject> UAssetDefinition_UsdAssetCache::GetAssetClass() const
{
	return UUsdAssetCache3::StaticClass();
}

bool UAssetDefinition_UsdAssetCache::CanImport() const
{
	return false;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_UsdAssetCache::GetAssetCategories() const
{
	static const auto Categories = {EAssetCategoryPaths::Misc};
	return Categories;
}

#undef LOCTEXT_NAMESPACE
