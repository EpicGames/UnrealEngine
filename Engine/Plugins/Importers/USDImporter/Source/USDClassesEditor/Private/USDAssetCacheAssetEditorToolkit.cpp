// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAssetCacheAssetEditorToolkit.h"

#include "USDAssetCache3.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FUsdAssetCacheAssetEditorToolkit"

const FName FUsdAssetCacheAssetEditorToolkit::TabId("AssetCacheEditor");

void FUsdAssetCacheAssetEditorToolkit::Initialize(
	const EToolkitMode::Type Mode,
	const TSharedPtr<IToolkitHost>& InitToolkitHost,
	UUsdAssetCache3* InAssetCache
)
{
	const TSharedRef<FTabManager::FLayout>
		StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_AssetCacheEditor")
									  ->AddArea(
										  FTabManager::NewPrimaryArea()
											  ->SetOrientation(Orient_Vertical)
											  ->Split(FTabManager::NewSplitter()
														  ->SetOrientation(Orient_Horizontal)
														  ->Split(FTabManager::NewStack()->SetHideTabWell(true)->AddTab(TabId, ETabState::OpenedTab)))
									  );

	AssetCache = InAssetCache;

	// Just use a property details view for now
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	AssetCacheEditorWidget = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	AssetCacheEditorWidget->SetObject(InAssetCache);

	const bool bUseSmallIcons = true;
	const bool bToolbarFocusable = false;
	const bool bCreateDefaultToolbar = true;
	const bool bCreateDefaultStandaloneMenu = true;
	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		TabId,
		StandaloneDefaultLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		AssetCache,
		bToolbarFocusable,
		bUseSmallIcons
	);
}

void FUsdAssetCacheAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(GetBaseToolkitName());

	const FSlateIcon LayersIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.USDStage");

	InTabManager->RegisterTabSpawner(TabId, FOnSpawnTab::CreateSP(this, &FUsdAssetCacheAssetEditorToolkit::SpawnTab))
		.SetDisplayName(GetBaseToolkitName())
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(LayersIcon);
}

void FUsdAssetCacheAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner(TabId);
}

TSharedRef<SDockTab> FUsdAssetCacheAssetEditorToolkit::SpawnTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == TabId);

	TSharedRef<SDockTab>
		NewDockTab = SNew(SDockTab).Label(GetBaseToolkitName()).TabColorScale(GetWorldCentricTabColorScale())[AssetCacheEditorWidget.ToSharedRef()];

	return NewDockTab;
}

void FUsdAssetCacheAssetEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(AssetCache);
}

FString FUsdAssetCacheAssetEditorToolkit::GetReferencerName() const
{
	return TEXT("FUsdAssetCacheAssetEditorToolkit");
}

FText FUsdAssetCacheAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "USD Asset Cache Editor");
}

FName FUsdAssetCacheAssetEditorToolkit::GetToolkitFName() const
{
	static FName Name("USD Asset Cache Editor");
	return Name;
}

FLinearColor FUsdAssetCacheAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor{FColor(32, 145, 208)};
}

FString FUsdAssetCacheAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "USD Asset Cache ").ToString();
}

#undef LOCTEXT_NAMESPACE
