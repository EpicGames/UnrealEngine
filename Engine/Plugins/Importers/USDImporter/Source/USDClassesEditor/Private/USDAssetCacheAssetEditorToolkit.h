// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"

class UUsdAssetCache3;

class FUsdAssetCacheAssetEditorToolkit
	: public FAssetEditorToolkit
	, public FGCObject
{
public:
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UUsdAssetCache3* InAssetCache);

private:
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;

	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:
	TObjectPtr<UUsdAssetCache3> AssetCache;

	TSharedPtr<class IDetailsView> AssetCacheEditorWidget;

	static const FName TabId;
};
