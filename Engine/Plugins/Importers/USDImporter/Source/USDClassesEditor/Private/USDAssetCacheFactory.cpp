// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAssetCacheFactory.h"

#include "USDAssetCache3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDAssetCacheFactory)

UUsdAssetCacheFactory::UUsdAssetCacheFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUsdAssetCache3::StaticClass();
}

UObject* UUsdAssetCacheFactory::FactoryCreateNew(
	UClass* Class,
	UObject* InParent,
	FName Name,
	EObjectFlags Flags,
	UObject* Context,
	FFeedbackContext* Warn
)
{
	return NewObject<UUsdAssetCache3>(InParent, Name, Flags | RF_Transactional | RF_Public | RF_Standalone);
}

bool UUsdAssetCacheFactory::ShouldShowInNewMenu() const
{
	return true;
}
