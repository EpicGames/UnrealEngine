// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "UObject/ObjectMacros.h"

#include "USDAssetCacheFactory.generated.h"

UCLASS(hidecategories = Object)
class UUsdAssetCacheFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	// UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
		override;
	virtual bool ShouldShowInNewMenu() const override;
	//~ End UFactory Interface
};
