// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDClassesEditorModule.h"

#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "USDClassesEditorModule"

class FUsdClassesEditorModule : public IUsdClassesEditorModule
{
};

IMPLEMENT_MODULE(FUsdClassesEditorModule, USDClassesEditor);

#undef LOCTEXT_NAMESPACE
