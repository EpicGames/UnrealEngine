// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class USDClassesEditor : ModuleRules
	{
		public USDClassesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ContentBrowser",
					"InputCore",
					"UnrealEd",
					"USDClasses",
				}
			);
		}
	}
}
