// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceExporterUSD.h"

#include "SkeletalMeshExporterUSDOptions.h"
#include "UnrealUSDWrapper.h"
#include "USDErrorUtils.h"
#include "USDExporterModule.h"
#include "USDExportUtils.h"
#include "USDLayerUtils.h"
#include "USDObjectUtils.h"
#include "USDOptionsWindow.h"
#include "USDPrimConversion.h"
#include "USDSkeletalDataConversion.h"
#include "USDUnrealAssetInfo.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Animation/AnimSequence.h"
#include "AnimSequenceExporterUSDOptions.h"
#include "AssetExportTask.h"
#include "Engine/SkeletalMesh.h"
#include "EngineAnalytics.h"
#include "Misc/EngineVersion.h"
#include "UObject/GCObjectScopeGuard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequenceExporterUSD)

#define LOCTEXT_NAMESPACE "AnimSequenceExporterUSD"

namespace UE::AnimSequenceExporterUSD::Private
{
	void SendAnalytics(
		UObject* Asset,
		UAnimSequenceExporterUSDOptions* Options,
		bool bAutomated,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension
	)
	{
		if (!Asset || !FEngineAnalytics::IsAvailable())
		{
			return;
		}

		const FString ClassName = IUsdClassesModule::GetClassNameForAnalytics(Asset);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Emplace(TEXT("AssetType"), ClassName);

		if (Options)
		{
			UsdUtils::AddAnalyticsAttributes(*Options, EventAttributes);
		}

		IUsdClassesModule::SendAnalytics(
			MoveTemp(EventAttributes),
			FString::Printf(TEXT("Export.%s"), *ClassName),
			bAutomated,
			ElapsedSeconds,
			NumberOfFrames,
			Extension
		);
	}
}

UAnimSequenceExporterUSD::UAnimSequenceExporterUSD()
{
#if USE_USD_SDK
	UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(FormatExtension, FormatDescription);
	SupportedClass = UAnimSequence::StaticClass();
	bText = false;
#endif	  // #if USE_USD_SDK
}

bool UAnimSequenceExporterUSD::ExportBinary(
	UObject* Object,
	const TCHAR* Type,
	FArchive& Ar,
	FFeedbackContext* Warn,
	int32 FileIndex,
	uint32 PortFlags
)
{
#if USE_USD_SDK
	UAnimSequence* AnimSequence = CastChecked<UAnimSequence>(Object);
	if (!AnimSequence)
	{
		return false;
	}

	FScopedUsdMessageLog UsdMessageLog;

	// We may dispatch another export task in between, so lets cache this for ourselves as it may
	// be overwritten
	const FString AnimSequenceFile = UExporter::CurrentFilename;

	UAnimSequenceExporterUSDOptions* Options = nullptr;
	if (ExportTask)
	{
		Options = Cast<UAnimSequenceExporterUSDOptions>(ExportTask->Options);
	}
	if (!Options)
	{
		Options = GetMutableDefault<UAnimSequenceExporterUSDOptions>();

		// Prompt with an options dialog if we can
		if (Options && (!ExportTask || !ExportTask->bAutomated))
		{
			Options->PreviewMeshOptions.MaterialBakingOptions.TexturesDir.Path = FPaths::Combine(FPaths::GetPath(AnimSequenceFile), TEXT("Textures"));

			const bool bContinue = SUsdOptionsWindow::ShowExportOptions(*Options);
			if (!bContinue)
			{
				return false;
			}
		}
	}
	if (!Options)
	{
		return false;
	}

	if (!IUsdExporterModule::CanExportToLayer(UExporter::CurrentFilename))
	{
		return false;
	}

	UsdUnreal::ExportUtils::FUniquePathScope UniquePathScope;

	// See comment on the analogous line within StaticMeshExporterUSD.cpp
	ExportTask->bPrompt = false;

	// Export preview mesh if needed
	USkeletalMesh* SkeletalMesh = nullptr;
	FString MeshAssetFile;
	bool bExportAsSkeletal = true;
	if (Options && Options->bExportPreviewMesh)
	{
		// To convert to non-skeletal, need to export the preview mesh in the first place
		bExportAsSkeletal = !Options->PreviewMeshOptions.bConvertSkeletalToNonSkeletal;

		SkeletalMesh = AnimSequence->GetPreviewMesh();
		USkeleton* AnimSkeleton = SkeletalMesh ? SkeletalMesh->GetSkeleton() : nullptr;

		if (!AnimSkeleton && !SkeletalMesh)
		{
			AnimSkeleton = AnimSequence->GetSkeleton();
			SkeletalMesh = AnimSkeleton ? AnimSkeleton->GetAssetPreviewMesh(AnimSequence) : nullptr;
		}

		if (AnimSkeleton && !SkeletalMesh)
		{
			SkeletalMesh = AnimSkeleton->FindCompatibleMesh();
		}

		if (SkeletalMesh)
		{
			FString PathPart;
			FString FilenamePart;
			FString ExtensionPart;
			FPaths::Split(AnimSequenceFile, PathPart, FilenamePart, ExtensionPart);
			MeshAssetFile = FPaths::Combine(PathPart, FilenamePart + TEXT("_SkeletalMesh.") + ExtensionPart);

			USkeletalMeshExporterUSDOptions* SkeletalMeshOptions = GetMutableDefault<USkeletalMeshExporterUSDOptions>();
			SkeletalMeshOptions->StageOptions = Options->StageOptions;
			SkeletalMeshOptions->MeshAssetOptions = Options->PreviewMeshOptions;
			SkeletalMeshOptions->MetadataOptions = Options->MetadataOptions;
			SkeletalMeshOptions->bReExportIdenticalAssets = Options->bReExportIdenticalAssets;

			UAssetExportTask* LevelExportTask = NewObject<UAssetExportTask>();
			FGCObjectScopeGuard ExportTaskGuard(LevelExportTask);
			LevelExportTask->Object = SkeletalMesh;
			LevelExportTask->Options = SkeletalMeshOptions;
			LevelExportTask->Exporter = nullptr;
			LevelExportTask->Filename = MeshAssetFile;
			LevelExportTask->bSelected = false;
			LevelExportTask->bReplaceIdentical = ExportTask->bReplaceIdentical;
			LevelExportTask->bPrompt = false;
			LevelExportTask->bUseFileArchive = false;
			LevelExportTask->bWriteEmptyFiles = false;
			LevelExportTask->bAutomated = true;	   // Pretend this is an automated task so it doesn't pop the options dialog

			const bool bSucceeded = UExporter::RunAssetExportTask(LevelExportTask);
			if (!bSucceeded)
			{
				MeshAssetFile = {};
			}
		}
		else
		{
			USD_LOG_USERWARNING(FText::Format(
				NSLOCTEXT("AnimSequenceExporterUSD", "InvalidSkelMesh", "Couldn't find the skeletal mesh to export for anim sequence {0}."),
				FText::FromName(AnimSequence->GetFName())
			));
		}
	}

	// Collect the target paths for our SkelAnimation prim and its SkelRoot, if any
	UE::FSdfPath SkelRootPath;
	UE::FSdfPath SkelAnimPath;
	if (MeshAssetFile.IsEmpty() || !bExportAsSkeletal)
	{
		SkelAnimPath = UE::FSdfPath::AbsoluteRootPath().AppendChild(*UsdUtils::SanitizeUsdIdentifier(*AnimSequence->GetName()));
	}
	else
	{
		SkelRootPath = UE::FSdfPath::AbsoluteRootPath().AppendChild(*UsdUtils::SanitizeUsdIdentifier(*SkeletalMesh->GetName()));

		SkelAnimPath = SkelRootPath.AppendChild(*UsdUtils::SanitizeUsdIdentifier(*AnimSequence->GetName()));
	}

	FString AnimSequenceVersion;
	if (IAnimationDataModel* DataModel = AnimSequence->GetDataModel())
	{
		FSHA1 SHA1;

		FGuid DataModelGuid = DataModel->GenerateGuid();
		SHA1.Update(reinterpret_cast<uint8*>(&DataModelGuid), sizeof(DataModelGuid));

		UsdUtils::HashForAnimSequenceExport(*Options, SHA1);

		SHA1.Final();
		FSHAHash Hash;
		SHA1.GetHash(&Hash.Hash[0]);
		AnimSequenceVersion = Hash.ToString();
	}

	// Check if we already have exported what we plan on exporting anyway
	if (FPaths::FileExists(AnimSequenceFile) && !AnimSequenceVersion.IsEmpty())
	{
		if (!ExportTask->bReplaceIdentical)
		{
			USD_LOG_USERINFO(FText::Format(
				LOCTEXT("FileAlreadyExists", "Skipping export of asset '{0}' as the target file '{1}' already exists."),
				FText::FromString(Object->GetPathName()),
				FText::FromString(UExporter::CurrentFilename)
			));
			return false;
		}
		// If we don't want to re-export this asset we need to check if its the same version
		else if (!Options->bReExportIdenticalAssets)
		{
			// Don't use the stage cache here as we want this stage to close within this scope in case
			// we have to overwrite its files due to e.g. missing payload or anything like that
			const bool bUseStageCache = false;
			const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadNone;
			if (UE::FUsdStage TempStage = UnrealUSDWrapper::OpenStage(*AnimSequenceFile, InitialLoadSet, bUseStageCache))
			{
				if (UE::FUsdPrim SkelAnimPrim = TempStage.GetPrimAtPath(SkelAnimPath))
				{
					FUsdUnrealAssetInfo Info = UsdUtils::GetPrimAssetInfo(SkelAnimPrim);

					const bool bVersionMatches = !Info.Version.IsEmpty() && Info.Version == AnimSequenceVersion;

					const bool bAssetTypeMatches = !Info.UnrealAssetType.IsEmpty() && Info.UnrealAssetType == Object->GetClass()->GetName();

					if (bVersionMatches && bAssetTypeMatches)
					{
						USD_LOG_USERINFO(FText::Format(
							LOCTEXT(
								"FileUpToDate",
								"Skipping export of asset '{0}' as the target file '{1}' already contains up-to-date exported data."
							),
							FText::FromString(Object->GetPathName()),
							FText::FromString(UExporter::CurrentFilename)
						));
						return true;
					}
				}
			}
		}
	}

	double StartTime = FPlatformTime::Cycles64();

	UE::FUsdStage AnimationStage = UnrealUSDWrapper::NewStage(*AnimSequenceFile);
	if (!AnimationStage)
	{
		return false;
	}

	UE::FUsdPrim SkelRootPrim;
	UE::FUsdPrim SkelAnimPrim;

	// Haven't exported the SkeletalMesh, just make a stage with a SkelAnimation prim
	if (MeshAssetFile.IsEmpty())
	{
		SkelAnimPrim = AnimationStage.DefinePrim(SkelAnimPath, TEXT("SkelAnimation"));
		if (!SkelAnimPrim)
		{
			return false;
		}

		AnimationStage.SetDefaultPrim(SkelAnimPrim);
	}
	// Exported a SkeletalMesh prim elsewhere, create a SkelRoot containing this SkelAnimation prim
	else
	{
		if (bExportAsSkeletal)
		{
			SkelRootPrim = AnimationStage.DefinePrim(SkelRootPath, TEXT("SkelRoot"));
			if (!SkelRootPrim)
			{
				return false;
			}

			SkelAnimPrim = AnimationStage.DefinePrim(SkelAnimPath, TEXT("SkelAnimation"));
			if (!SkelAnimPrim)
			{
				return false;
			}

			UE::FSdfPath SkeletonPath = SkelRootPath.AppendChild(UnrealIdentifiers::ExportedSkeletonPrimName);
			UE::FUsdPrim SkeletonPrim = AnimationStage.DefinePrim(SkeletonPath, TEXT("Skeleton"));
			if (!SkeletonPrim)
			{
				return false;
			}

			AnimationStage.SetDefaultPrim(SkelRootPrim);

			// Add a reference to the SkelRoot of the static mesh, which will compose in the Mesh and Skeleton prims
			UsdUtils::AddReference(SkelRootPrim, *MeshAssetFile);

			// We bind the animation directly to the Skeleton (and not the skel root) because binding it to the SkelRoot
			// may lead to trouble when we're exporting nested SkeletalMeshComponents, as it will be inherited by
			// all child Skeletons (even the ones that wouldn't otherwise receive any animation)
			UsdUtils::BindAnimationSource(SkeletonPrim, SkelAnimPrim);
		}
		else
		{
			SkelAnimPrim = AnimationStage.DefinePrim(SkelAnimPath);
			if (!SkelAnimPrim)
			{
				return false;
			}

			AnimationStage.SetDefaultPrim(SkelAnimPrim);

			// Add a reference to the static mesh prim on which the time samples will be authored
			UsdUtils::AddReference(SkelAnimPrim, *MeshAssetFile);
		}
	}

	// Configure stage metadata
	{
		if (Options)
		{
			UsdUtils::SetUsdStageMetersPerUnit(AnimationStage, Options->StageOptions.MetersPerUnit);
			UsdUtils::SetUsdStageUpAxis(AnimationStage, Options->StageOptions.UpAxis);
		}

		const double StartTimeCode = 0.0;
		const double EndTimeCode = AnimSequence->GetNumberOfSampledKeys() - 1;
		UsdUtils::AddTimeCodeRangeToLayer(AnimationStage.GetRootLayer(), StartTimeCode, EndTimeCode);

		AnimationStage.SetTimeCodesPerSecond(AnimSequence->GetSamplingFrameRate().AsDecimal());
	}

	if (bExportAsSkeletal)
	{
		UnrealToUsd::ConvertAnimSequence(AnimSequence, SkelAnimPrim);
	}
	else
	{
		UnrealToUsd::ConvertAnimSequenceToAnimatedMesh(AnimSequence, SkeletalMesh, SkelAnimPrim);
	}

	if (Options->MetadataOptions.bExportAssetInfo)
	{
		FUsdUnrealAssetInfo Info;
		Info.Name = AnimSequence->GetName();
		Info.Identifier = AnimSequenceFile;
		Info.Version = AnimSequenceVersion;
		Info.UnrealContentPath = AnimSequence->GetPathName();
		Info.UnrealAssetType = AnimSequence->GetClass()->GetName();
		Info.UnrealExportTime = FDateTime::Now().ToString();
		Info.UnrealEngineVersion = FEngineVersion::Current().ToString();

		UsdUtils::SetPrimAssetInfo(SkelAnimPrim, Info);
	}

	if (Options->MetadataOptions.bExportAssetMetadata)
	{
		if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(AnimSequence))
		{
			UnrealToUsd::ConvertMetadata(
				UserData,
				SkelAnimPrim,
				Options->MetadataOptions.BlockedPrefixFilters,
				Options->MetadataOptions.bInvertFilters
			);
		}
	}

	AnimationStage.GetRootLayer().Save();

	// Analytics
	{
		bool bAutomated = ExportTask ? ExportTask->bAutomated : false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(AnimSequenceFile);

		UE::AnimSequenceExporterUSD::Private::SendAnalytics(
			Object,
			Options,
			bAutomated,
			ElapsedSeconds,
			UsdUtils::GetUsdStageNumFrames(AnimationStage),
			Extension
		);
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
