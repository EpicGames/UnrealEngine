// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceExporterUSDOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequenceExporterUSDOptions)

void UsdUtils::AddAnalyticsAttributes(const UAnimSequenceExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	UsdUtils::AddAnalyticsAttributes(Options.StageOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ExportPreviewMesh"), LexToString(Options.bExportPreviewMesh));
	if (Options.bExportPreviewMesh)
	{
		UsdUtils::AddAnalyticsAttributes(Options.PreviewMeshOptions, InOutAttributes);
	}
	UsdUtils::AddAnalyticsAttributes(Options.MetadataOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
}

void UsdUtils::HashForAnimSequenceExport(const UAnimSequenceExporterUSDOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForExport(Options.StageOptions, HashToUpdate);
	UsdUtils::HashForExport(Options.MetadataOptions, HashToUpdate);
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportPreviewMesh), sizeof(Options.bExportPreviewMesh));
	if (Options.bExportPreviewMesh)
	{
		HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.PreviewMeshOptions.bConvertSkeletalToNonSkeletal), sizeof(Options.PreviewMeshOptions.bConvertSkeletalToNonSkeletal));
	}
}
