// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheExporterUSD.h"

#include "GeometryCacheExporterUSDOptions.h"
#include "MaterialExporterUSD.h"
#include "UnrealUSDWrapper.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDExporterModule.h"
#include "USDExportUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDObjectUtils.h"
#include "USDOptionsWindow.h"
#include "USDPrimConversion.h"
#include "USDUnrealAssetInfo.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "AssetExportTask.h"
#include "EngineAnalytics.h"
#include "GeometryCache.h"
#include "Misc/EngineVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheExporterUSD)

#define LOCTEXT_NAMESPACE "GeometryCacheExporterUSD"

namespace UE::GeometryCacheExporterUSD::Private
{
	void SendAnalytics(
		UObject* Asset,
		UGeometryCacheExporterUSDOptions* Options,
		bool bAutomated,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension
	)
	{
		if (!Asset || !FEngineAnalytics::IsAvailable())
		{
			return;
		}

		const FString ClassName = IUsdClassesModule::GetClassNameForAnalytics(Asset);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Emplace(TEXT("AssetType"), ClassName);

		if (Options)
		{
			UsdUtils::AddAnalyticsAttributes(*Options, EventAttributes);
		}

		IUsdClassesModule::SendAnalytics(
			MoveTemp(EventAttributes),
			FString::Printf(TEXT("Export.%s"), *ClassName),
			bAutomated,
			ElapsedSeconds,
			NumberOfFrames,
			Extension
		);
	}
}

UGeometryCacheExporterUSD::UGeometryCacheExporterUSD()
{
#if USE_USD_SDK
	UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(FormatExtension, FormatDescription);
	SupportedClass = UGeometryCache::StaticClass();
	bText = false;
#endif	  // #if USE_USD_SDK
}

bool UGeometryCacheExporterUSD::ExportBinary(
	UObject* Object,
	const TCHAR* Type,
	FArchive& Ar,
	FFeedbackContext* Warn,
	int32 FileIndex,
	uint32 PortFlags
)
{
#if USE_USD_SDK
	UGeometryCache* GeometryCache = CastChecked<UGeometryCache>(Object);
	if (!GeometryCache)
	{
		return false;
	}

	UGeometryCacheExporterUSDOptions* Options = nullptr;
	if (ExportTask)
	{
		Options = Cast<UGeometryCacheExporterUSDOptions>(ExportTask->Options);
	}
	if (!Options)
	{
		Options = GetMutableDefault<UGeometryCacheExporterUSDOptions>();

		// Prompt with an options dialog if we can
		if (Options && (!ExportTask || !ExportTask->bAutomated))
		{
			Options->MeshAssetOptions.MaterialBakingOptions.TexturesDir.Path = FPaths::Combine(
				FPaths::GetPath(UExporter::CurrentFilename),
				TEXT("Textures")
			);

			const bool bContinue = SUsdOptionsWindow::ShowExportOptions(*Options);
			if (!bContinue)
			{
				return false;
			}
		}
	}
	if (!Options)
	{
		return false;
	}

	// The intended export workflow has us exporting into the Ar argument, and would only later actually write to the
	// file, potentially prompting the user beforehand if they want to overwrite an existing file or not (check
	// UExporter::RunAssetExportTask). On the USD export workflows we always write to the file directly ourselves as
	// its easier to just let the USD SDK do it, which means that the "Do you want to overwrite?" file prompt happens
	// way after we already saved everything, and has no effect. Lets force bPrompt to false then, so this dialog
	// doesn't appear under any circumstances
	ExportTask->bPrompt = false;

	// If bUsePayload is true, we'll intercept the filename so that we write the mesh data to
	// "C:/MyFolder/file_payload.usda" and create an "asset" file "C:/MyFolder/file.usda" that uses it
	// as a payload, pointing at the default prim
	FString PayloadFilename = UExporter::CurrentFilename;
	if (Options->MeshAssetOptions.bUsePayload)
	{
		FString PathPart;
		FString FilenamePart;
		FString ExtensionPart;
		FPaths::Split(PayloadFilename, PathPart, FilenamePart, ExtensionPart);

		if (FormatExtension.Contains(Options->MeshAssetOptions.PayloadFormat))
		{
			ExtensionPart = Options->MeshAssetOptions.PayloadFormat;
		}

		PayloadFilename = FPaths::Combine(PathPart, FilenamePart + TEXT("_payload.") + ExtensionPart);
	}

	if (!IUsdExporterModule::CanExportToLayer(UExporter::CurrentFilename)
		|| (Options->MeshAssetOptions.bUsePayload && !IUsdExporterModule::CanExportToLayer(PayloadFilename)))
	{
		return false;
	}

	UsdUnreal::ExportUtils::FUniquePathScope UniquePathScope;

	// Get a simple GUID hash/identifier of our mesh
	FString DDCKeyHash;
	const FString& DDCKey = GeometryCache->GetHash();
	FSHA1 SHA1;
	SHA1.UpdateWithString(*DDCKey, DDCKey.Len());
	UsdUtils::HashForGeometryCacheExport(*Options, SHA1);

	SHA1.Final();
	FSHAHash Hash;
	SHA1.GetHash(&Hash.Hash[0]);
	DDCKeyHash = Hash.ToString();

	// Check if we already have exported what we plan on exporting anyway
	if (FPaths::FileExists(UExporter::CurrentFilename) && FPaths::FileExists(PayloadFilename))
	{
		if (!ExportTask->bReplaceIdentical)
		{
			USD_LOG_USERINFO(FText::Format(
				LOCTEXT("FileAlreadyExists", "Skipping export of asset '{0}' as the target file '{1}' already exists."),
				FText::FromString(Object->GetPathName()),
				FText::FromString(UExporter::CurrentFilename)
			));
			return false;
		}
		// If we don't want to re-export this asset we need to check if its the same version
		else if (!Options->bReExportIdenticalAssets)
		{
			bool bSkipMeshExport = false;

			// Don't use the stage cache here as we want this stage to close within this scope in case
			// we have to overwrite its files due to e.g. missing payload or anything like that
			const bool bUseStageCache = false;
			const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadNone;
			if (UE::FUsdStage TempStage = UnrealUSDWrapper::OpenStage(*UExporter::CurrentFilename, InitialLoadSet, bUseStageCache))
			{
				if (UE::FUsdPrim DefaultPrim = TempStage.GetDefaultPrim())
				{
					FUsdUnrealAssetInfo Info = UsdUtils::GetPrimAssetInfo(DefaultPrim);

					const bool bVersionMatches = !Info.Version.IsEmpty() && Info.Version == DDCKeyHash;

					const bool bAssetTypeMatches = !Info.UnrealAssetType.IsEmpty() && Info.UnrealAssetType == GeometryCache->GetClass()->GetName();

					if (bVersionMatches && bAssetTypeMatches)
					{
						USD_LOG_USERINFO(FText::Format(
							LOCTEXT(
								"FileUpToDate",
								"Skipping export of asset '{0}' as the target file '{1}' already contains up-to-date exported data."
							),
							FText::FromString(GeometryCache->GetPathName()),
							FText::FromString(UExporter::CurrentFilename)
						));

						bSkipMeshExport = true;
					}
				}
			}

			if (bSkipMeshExport)
			{
				// Even if we're not going to export the mesh, we may still need to re-bake materials
				if (Options->MeshAssetOptions.bBakeMaterials)
				{
					TSet<UMaterialInterface*> MaterialsToBake;
					for (UMaterialInterface* MaterialInterface : GeometryCache->Materials)
					{
						MaterialsToBake.Add(MaterialInterface);
					}

					const bool bIsAssetLayer = true;
					UMaterialExporterUsd::ExportMaterialsForStage(
						MaterialsToBake.Array(),
						Options->MeshAssetOptions.MaterialBakingOptions,
						Options->MetadataOptions,
						UExporter::CurrentFilename,
						bIsAssetLayer,
						Options->MeshAssetOptions.bUsePayload,
						ExportTask->bReplaceIdentical,
						Options->bReExportIdenticalAssets,
						ExportTask->bAutomated
					);
				}

				return true;
			}
		}
	}

	double StartTime = FPlatformTime::Cycles64();

	// UsdStage is the payload stage when exporting with payloads, or just the single stage otherwise
	UE::FUsdStage UsdStage = UnrealUSDWrapper::NewStage(*PayloadFilename);
	if (!UsdStage)
	{
		return false;
	}

	UsdUtils::SetUsdStageMetersPerUnit(UsdStage, Options->StageOptions.MetersPerUnit);
	UsdUtils::SetUsdStageUpAxis(UsdStage, Options->StageOptions.UpAxis);

	FString RootPrimPath = (TEXT("/") + UsdUtils::SanitizeUsdIdentifier(*GeometryCache->GetName()));

	UE::FUsdPrim RootPrim = UsdStage.DefinePrim(UE::FSdfPath(*RootPrimPath), TEXT("Mesh"));
	if (!RootPrim)
	{
		return false;
	}

	UsdStage.SetDefaultPrim(RootPrim);

	// Asset stage always the stage where we write the material assignments
	UE::FUsdStage AssetStage;

	// Using payload: Convert mesh data through the asset stage (that references the payload) so that we can
	// author mesh data on the payload layer and material data on the asset layer
	if (Options->MeshAssetOptions.bUsePayload)
	{
		AssetStage = UnrealUSDWrapper::NewStage(*UExporter::CurrentFilename);
		if (AssetStage)
		{
			UsdUtils::SetUsdStageMetersPerUnit(AssetStage, Options->StageOptions.MetersPerUnit);
			UsdUtils::SetUsdStageUpAxis(AssetStage, Options->StageOptions.UpAxis);

			if (UE::FUsdPrim AssetRootPrim = AssetStage.DefinePrim(UE::FSdfPath(*RootPrimPath)))
			{
				AssetStage.SetDefaultPrim(AssetRootPrim);

				UsdUtils::AddPayload(AssetRootPrim, *PayloadFilename);
			}
		}
	}
	// Not using payload: Just author everything on the current edit target of the payload (== asset) layer
	else
	{
		AssetStage = UsdStage;
	}

	UnrealToUsd::ConvertGeometryCache(GeometryCache, RootPrim, &AssetStage);

	if (UE::FUsdPrim AssetDefaultPrim = AssetStage.GetDefaultPrim())
	{
		if (Options->MetadataOptions.bExportAssetInfo)
		{
			FUsdUnrealAssetInfo Info;
			Info.Name = GeometryCache->GetName();
			Info.Identifier = UExporter::CurrentFilename;
			Info.Version = DDCKeyHash;
			Info.UnrealContentPath = GeometryCache->GetPathName();
			Info.UnrealAssetType = GeometryCache->GetClass()->GetName();
			Info.UnrealExportTime = FDateTime::Now().ToString();
			Info.UnrealEngineVersion = FEngineVersion::Current().ToString();

			UsdUtils::SetPrimAssetInfo(AssetDefaultPrim, Info);
		}

		if (Options->MetadataOptions.bExportAssetMetadata)
		{
			if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(GeometryCache))
			{
				UnrealToUsd::ConvertMetadata(
					UserData,
					AssetDefaultPrim,
					Options->MetadataOptions.BlockedPrefixFilters,
					Options->MetadataOptions.bInvertFilters
				);
			}
		}
	}

	// Bake materials and replace unrealMaterials with references to the baked files.
	if (Options->MeshAssetOptions.bBakeMaterials)
	{
		TSet<UMaterialInterface*> MaterialsToBake;
		for (UMaterialInterface* MaterialInterface : GeometryCache->Materials)
		{
			MaterialsToBake.Add(MaterialInterface);
		}

		const bool bIsAssetLayer = true;
		UMaterialExporterUsd::ExportMaterialsForStage(
			MaterialsToBake.Array(),
			Options->MeshAssetOptions.MaterialBakingOptions,
			Options->MetadataOptions,
			AssetStage.GetRootLayer().GetRealPath(),
			bIsAssetLayer,
			Options->MeshAssetOptions.bUsePayload,
			ExportTask->bReplaceIdentical,
			Options->bReExportIdenticalAssets,
			ExportTask->bAutomated
		);
	}

	UsdStage.GetRootLayer().Save();
	if (AssetStage && UsdStage != AssetStage)
	{
		AssetStage.GetRootLayer().Save();
	}

	// Analytics
	{
		bool bAutomated = ExportTask ? ExportTask->bAutomated : false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(UExporter::CurrentFilename);

		UE::GeometryCacheExporterUSD::Private::SendAnalytics(
			Object,
			Options,
			bAutomated,
			ElapsedSeconds,
			UsdUtils::GetUsdStageNumFrames(AssetStage),
			Extension
		);
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
