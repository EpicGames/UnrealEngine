// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheExporterUSDOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCacheExporterUSDOptions)

void UsdUtils::AddAnalyticsAttributes(const UGeometryCacheExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	UsdUtils::AddAnalyticsAttributes(Options.StageOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MeshAssetOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
}

void UsdUtils::HashForGeometryCacheExport(const UGeometryCacheExporterUSDOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForExport(Options.StageOptions, HashToUpdate);
	UsdUtils::HashForMeshExport(Options.MeshAssetOptions, HashToUpdate);
}
