// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelExporterUSD.h"

#include "LevelExporterUSDOptions.h"
#include "UnrealUSDWrapper.h"
#include "USDOptionsWindow.h"

#include "AssetExportTask.h"
#include "Engine/World.h"
#include "IPythonScriptPlugin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelExporterUSD)

ULevelExporterUSD::ULevelExporterUSD()
{
#if USE_USD_SDK
	UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(FormatExtension, FormatDescription);
	SupportedClass = UWorld::StaticClass();
	bText = false;
#endif	  // #if USE_USD_SDK
}

bool ULevelExporterUSD::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
#if USE_USD_SDK
	UWorld* World = Cast<UWorld>(Object);
	if (!World)
	{
		return false;
	}

	ULevelExporterUSDOptions* Options = nullptr;
	if (ExportTask)
	{
		Options = Cast<ULevelExporterUSDOptions>(ExportTask->Options);
	}

	if (!Options)
	{
		Options = GetMutableDefault<ULevelExporterUSDOptions>();
	}
	if (!Options)
	{
		return false;
	}

	if (ExportTask)
	{
		// Set this early so that we can use the Object from it (the world/level to export) when showing
		// the level export options dialog.
		// Note that we must also consider how LevelSequenceExporterUSD may call down to us and provide the
		// export task that we must use. In that case we will be an automated export too
		Options->CurrentTask = ExportTask;

		// There is a dedicated "Export selected" option that sets this, so let's sync to it
		Options->Inner.bSelectionOnly = ExportTask->bSelected;
	}

	// Manual export: Show dialog options
	if (!ExportTask || !ExportTask->bAutomated)
	{
		Options->Inner.AssetFolder.Path = FPaths::Combine(FPaths::GetPath(UExporter::CurrentFilename), TEXT("Assets"));

		const bool bContinue = SUsdOptionsWindow::ShowExportOptions(*Options);
		if (!bContinue)
		{
			return false;
		}
	}

	// Note how we don't explicitly pass the Options down to Python here: We stash our desired export options as the CDO, and
	// those are read from Python by executing export_with_cdo_options().
	if (IPythonScriptPlugin::Get()->IsPythonAvailable())
	{
		ULevelExporterUSDOptions* OptionsOriginalCDO = GetMutableDefault<ULevelExporterUSDOptions>();
		TStrongObjectPtr<ULevelExporterUSDOptions> CDOPin;

		const bool bNeedCDOSwap = Options != OptionsOriginalCDO;
		if (bNeedCDOSwap)
		{
			CDOPin.Reset(OptionsOriginalCDO);
			ULevelExporterUSDOptions::StaticClass()->SetDefaultObject(Options);
		}

		IPythonScriptPlugin::Get()->ExecPythonCommand(TEXT("import usd_unreal.level_exporter; usd_unreal.level_exporter.export_with_cdo_options()"));

		if (bNeedCDOSwap)
		{
			ULevelExporterUSDOptions::StaticClass()->SetDefaultObject(OptionsOriginalCDO);
		}
	}
	Options->CurrentTask = nullptr;

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}
