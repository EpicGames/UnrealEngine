// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelExporterUSDOptions.h"

#include "UnrealUSDWrapper.h"
#include "USDAssetOptions.h"
#include "USDExporterModule.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelExporterUSDOptions)

TArray<FString> ULevelExporterUSDOptions::GetUsdExtensions()
{
	TArray<FString> Extensions = UnrealUSDWrapper::GetNativeFileFormats();
	Extensions.Remove(TEXT("usdz"));
	return Extensions;
}

void UsdUtils::AddAnalyticsAttributes(const FLevelExporterUSDOptionsInner& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("SelectionOnly"), Options.bSelectionOnly);
	InOutAttributes.Emplace(TEXT("ExportActorFolders"), Options.bExportActorFolders);
	InOutAttributes.Emplace(TEXT("IgnoreSequencerAnimations"), Options.bIgnoreSequencerAnimations);
	InOutAttributes.Emplace(TEXT("ExportFoliageOnActorsLayer"), Options.bExportFoliageOnActorsLayer);
	InOutAttributes.Emplace(TEXT("bExportToNestedSubFolders"), Options.bExportToNestedSubFolders);
	UsdUtils::AddAnalyticsAttributes(Options.AssetOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MetadataOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("LowestLandscapeLOD"), LexToString(Options.LowestLandscapeLOD));
	InOutAttributes.Emplace(TEXT("HighestLandscapeLOD"), LexToString(Options.HighestLandscapeLOD));
	InOutAttributes.Emplace(TEXT("LandscapeBakeResolution"), Options.LandscapeBakeResolution.ToString());
	InOutAttributes.Emplace(TEXT("ExportSublayers"), LexToString(Options.bExportSublayers));
	InOutAttributes.Emplace(TEXT("NumLevelsToIgnore"), LexToString(Options.LevelsToIgnore.Num()));
}

void UsdUtils::AddAnalyticsAttributes(const ULevelExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	UsdUtils::AddAnalyticsAttributes(Options.StageOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("StartTimeCode"), LexToString(Options.StartTimeCode));
	InOutAttributes.Emplace(TEXT("EndTimeCode"), LexToString(Options.EndTimeCode));
	UsdUtils::AddAnalyticsAttributes(Options.Inner, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalLevelsAndSequences"), Options.bReExportIdenticalLevelsAndSequences);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
}

void UsdUtils::HashForLevelExport(const FLevelExporterUSDOptionsInner& Options, FSHA1& HashToUpdate)
{
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bSelectionOnly), sizeof(Options.bSelectionOnly));
	if (Options.bSelectionOnly)
	{
		IUsdExporterModule::HashEditorSelection(HashToUpdate);
	}

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportActorFolders), sizeof(Options.bExportActorFolders));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bIgnoreSequencerAnimations), sizeof(Options.bIgnoreSequencerAnimations));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportFoliageOnActorsLayer), sizeof(Options.bExportFoliageOnActorsLayer));

	// If we changed where we want the assets exported we need to re-export them and update the reference paths
	HashToUpdate.UpdateWithString(*Options.AssetFolder.Path, Options.AssetFolder.Path.Len());

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportToNestedSubFolders), sizeof(Options.bExportToNestedSubFolders));

	// This affects how we author material overrides
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.AssetOptions.bBakeMaterials), sizeof(Options.AssetOptions.bBakeMaterials));

	// This affects how we author material overrides and landscapes
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.AssetOptions.bUsePayload), sizeof(Options.AssetOptions.bUsePayload));
	if (Options.AssetOptions.bUsePayload)
	{
		HashToUpdate.UpdateWithString(*Options.AssetOptions.PayloadFormat, Options.AssetOptions.PayloadFormat.Len());
	}

	// If we changed to/from exporting LODs, how we author material overrides may also need to change
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.AssetOptions.LowestMeshLOD), sizeof(Options.AssetOptions.LowestMeshLOD));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.AssetOptions.HighestMeshLOD), sizeof(Options.AssetOptions.HighestMeshLOD));

	UsdUtils::HashForExport(Options.MetadataOptions, HashToUpdate);

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.LowestLandscapeLOD), sizeof(Options.LowestLandscapeLOD));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.HighestLandscapeLOD), sizeof(Options.HighestLandscapeLOD));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.LandscapeBakeResolution), sizeof(Options.LandscapeBakeResolution));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportSublayers), sizeof(Options.bExportSublayers));
}

void UsdUtils::HashForLevelExport(const ULevelExporterUSDOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForExport(Options.StageOptions, HashToUpdate);
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.StartTimeCode), sizeof(Options.StartTimeCode));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.EndTimeCode), sizeof(Options.EndTimeCode));
	UsdUtils::HashForLevelExport(Options.Inner, HashToUpdate);
}
