// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelExporterUSDOptionsCustomization.h"

#include "LevelExporterUSDOptions.h"
#include "LevelSequenceExporterUSDOptions.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"

#include "Brushes/SlateColorBrush.h"
#include "CoreMinimal.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Engine/LevelStreaming.h"
#include "Modules/ModuleManager.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "LevelExporterUSDOptionsCustomization"

namespace UE::LevelExporterOptions::Private
{
	class SLevelPickerRow : public STableRow<TSharedRef<FString>>
	{
	public:
		SLATE_BEGIN_ARGS(SLevelPickerRow)
		{
		}
		SLATE_END_ARGS()

		void Construct(
			const FArguments& InArgs,
			const TSharedRef<STableViewBase>& OwnerTableView,
			TWeakPtr<FString> InEntry,
			FLevelExporterUSDOptionsInner* Inner
		)
		{
			STableRow::Construct(STableRow::FArguments(), OwnerTableView);

			SetBorderBackgroundColor(FLinearColor::Transparent);

			FString LevelName;
			if (TSharedPtr<FString> PinnedEntry = InEntry.Pin())
			{
				LevelName = *PinnedEntry;
			}

			// clang-format off
					SetRowContent(
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.MaxWidth(20)
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([LevelName, Inner]()
							{
								if (Inner)
								{
									return Inner->LevelsToIgnore.Contains(LevelName)
										? ECheckBoxState::Unchecked
										: ECheckBoxState::Checked;
								}

								return ECheckBoxState::Undetermined;
							})
							.OnCheckStateChanged_Lambda([LevelName, Inner](ECheckBoxState State)
							{
								if (Inner)
								{
									if (State == ECheckBoxState::Checked)
									{
										Inner->LevelsToIgnore.Remove(LevelName);
									}
									else if (State == ECheckBoxState::Unchecked)
									{
										Inner->LevelsToIgnore.Add(LevelName);
									}
								}
							})
						]

						+SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(FText::FromString(LevelName))
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					);
			// clang-format on
		}
	};

	class SLevelPickerList : public SListView<TSharedRef<FString>>
	{
	public:
		void Construct(const FArguments& InArgs, FLevelExporterUSDOptionsInner* Inner, UWorld* WorldToExport)
		{
			if (!Inner || !WorldToExport)
			{
				return;
			}

			if (ULevel* PersistentLevel = WorldToExport->PersistentLevel)
			{
				const FString LevelName = TEXT("Persistent Level");
				RootItems.Add(MakeShared<FString>(LevelName));
				if (!PersistentLevel->bIsVisible)
				{
					Inner->LevelsToIgnore.Add(LevelName);
				}
			}

			for (ULevelStreaming* StreamingLevel : WorldToExport->GetStreamingLevels())
			{
				if (StreamingLevel)
				{
					const FString LevelName = FPaths::GetBaseFilename(StreamingLevel->GetWorldAssetPackageName());

					RootItems.Add(MakeShared<FString>(*LevelName));

					if (!StreamingLevel->GetShouldBeVisibleInEditor())
					{
						Inner->LevelsToIgnore.Add(LevelName);
					}
				}
			}

			SListView::Construct(SListView::FArguments()
									 .ListItemsSource(&RootItems)
									 .SelectionMode(ESelectionMode::None)
									 .OnGenerateRow(this, &SLevelPickerList::OnGenerateRow, Inner));

			static FSlateColorBrush TransparentBrush{FLinearColor::Transparent};
			SetBackgroundBrush(&TransparentBrush);
		}

	private:
		TSharedRef<ITableRow> OnGenerateRow(
			TSharedRef<FString> InEntry,
			const TSharedRef<STableViewBase>& OwnerTable,
			FLevelExporterUSDOptionsInner* Inner
		) const
		{
			return SNew(SLevelPickerRow, OwnerTable, InEntry, Inner);
		}

		TArray<TSharedRef<FString>> RootItems;
	};
}	 // namespace UE::LevelExporterOptions::Private
namespace LevelExporterUSDImpl = UE::LevelExporterOptions::Private;

TSharedRef<IDetailCustomization> FLevelExporterUSDOptionsCustomization::MakeInstance()
{
	return MakeShared<FLevelExporterUSDOptionsCustomization>();
}

void FLevelExporterUSDOptionsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	TSharedPtr<LevelExporterUSDImpl::SLevelPickerList> PickerTree = nullptr;
	TStrongObjectPtr<UObject> OptionsPtr;
	FName LevelFilterPropName;
	FName ExportSublayersPropName;
	FName TexturesDirPropName;
	FName SublayersCategoryName;
	FName RootPrimNamePropName;
	TAttribute<bool> SublayersEditCondition;

	if (ULevelExporterUSDOptions* Options = Cast<ULevelExporterUSDOptions>(SelectedObjects[0].Get()))
	{
		OptionsPtr.Reset(Options);

		UAssetExportTask* Task = Options->CurrentTask.Get();

		UWorld* World = Task ? Cast<UWorld>(Task->Object) : GEditor->GetEditorWorldContext().World();

		PickerTree = SNew(LevelExporterUSDImpl::SLevelPickerList, &Options->Inner, World);
		LevelFilterPropName = TEXT("Inner.LevelsToIgnore");
		ExportSublayersPropName = TEXT("Inner.bExportSublayers");
		TexturesDirPropName = TEXT("Inner.AssetOptions.MaterialBakingOptions.TexturesDir");
		SublayersCategoryName = TEXT("Sublayers");
		RootPrimNamePropName = TEXT("Inner.RootPrimName");

		SublayersEditCondition = true;
	}
	else if (ULevelSequenceExporterUsdOptions* LevelSequenceOptions = Cast<ULevelSequenceExporterUsdOptions>(SelectedObjects[0].Get()))
	{
		OptionsPtr.Reset(LevelSequenceOptions);

		PickerTree = SNew(LevelExporterUSDImpl::SLevelPickerList, &LevelSequenceOptions->LevelExportOptions, LevelSequenceOptions->Level->GetWorld());
		LevelFilterPropName = TEXT("LevelExportOptions.LevelsToIgnore");
		ExportSublayersPropName = TEXT("LevelExportOptions.bExportSublayers");
		TexturesDirPropName = TEXT("LevelExportOptions.AssetOptions.MaterialBakingOptions.TexturesDir");
		SublayersCategoryName = TEXT("Level Export");
		RootPrimNamePropName = TEXT("LevelExportOptions.RootPrimName");

		// Refresh the dialog whenever we pick a new world to export, so that we can show this world's sublevels in the sublevel picker
		FSimpleDelegate RebuildDisplayDelegate = FSimpleDelegate::CreateLambda(
			[&DetailLayoutBuilder]()
			{
				DetailLayoutBuilder.ForceRefreshDetails();
			}
		);
		TSharedRef<IPropertyHandle> LevelToExportProp = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(ULevelSequenceExporterUsdOptions, Level)
		);
		LevelToExportProp->SetOnPropertyValueChanged(RebuildDisplayDelegate);

		// Only let us pick the sublayer options if we're exporting a level with the level sequence
		TSharedRef<IPropertyHandle> ExportLevelProp = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(ULevelSequenceExporterUsdOptions, bExportLevel)
		);
		SublayersEditCondition = TAttribute<bool>::Create(
			[ExportLevelProp]()
			{
				bool bExportingLevel = true;
				ExportLevelProp->GetValue(bExportingLevel);
				return bExportingLevel;
			}
		);
	}
	else
	{
		return;
	}

	TSharedRef<IPropertyHandle> LevelFilterProp = DetailLayoutBuilder.GetProperty(LevelFilterPropName);
	TSharedRef<IPropertyHandle> ExportSublayersProp = DetailLayoutBuilder.GetProperty(ExportSublayersPropName);
	TSharedRef<IPropertyHandle> RootPrimNameProp = DetailLayoutBuilder.GetProperty(RootPrimNamePropName);

	// Auto-sanitize Root Prim Name
	TWeakObjectPtr<UObject> OptionsObject = SelectedObjects[0];
	FSimpleDelegate SanitizeRootPrimName = FSimpleDelegate::CreateLambda(
		[&DetailLayoutBuilder, OptionsObject]()
		{
			if (ULevelExporterUSDOptions* Options = Cast<ULevelExporterUSDOptions>(OptionsObject.Get()))
			{
				Options->Inner.RootPrimName = UsdUtils::SanitizeUsdIdentifier(*Options->Inner.RootPrimName);
			}
			else if (ULevelSequenceExporterUsdOptions* LevelSequenceOptions = Cast<ULevelSequenceExporterUsdOptions>(OptionsObject.Get()))
			{
				LevelSequenceOptions->LevelExportOptions.RootPrimName = UsdUtils::SanitizeUsdIdentifier(
					*LevelSequenceOptions->LevelExportOptions.RootPrimName
				);
			}

			DetailLayoutBuilder.ForceRefreshDetails();
		}
	);
	RootPrimNameProp->SetOnPropertyValueChanged(SanitizeRootPrimName);

	// Touch these properties and categories to enforce this ordering
	DetailLayoutBuilder.EditCategory(TEXT("Stage options"));
	DetailLayoutBuilder.EditCategory(TEXT("Export settings"));
	IDetailCategoryBuilder& AssetOptionsCategory = DetailLayoutBuilder.EditCategory(TEXT("Asset options"));
	IDetailCategoryBuilder& MetadataOptionsCategory = DetailLayoutBuilder.EditCategory(TEXT("Metadata options"));

	// Promote all AssetOptions up a level on LevelExportUsdOptions or else we'll end up with a property named AssetOptions inside the AssetOptions
	// category This is the same effect as ShowOnlyInnerProperties, but in this case we need to do it manually as it doesn't work recursively
	if (TSharedPtr<IPropertyHandle> AssetOptionsProperty = DetailLayoutBuilder.GetProperty(TEXT("Inner.AssetOptions")))
	{
		DetailLayoutBuilder.HideProperty(AssetOptionsProperty);

		uint32 NumChildren = 0;
		if (AssetOptionsProperty->GetNumChildren(NumChildren) == FPropertyAccess::Result::Success)
		{
			for (uint32 Index = 0; Index < NumChildren; ++Index)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = AssetOptionsProperty->GetChildHandle(Index);
				AssetOptionsCategory.AddProperty(ChildProperty);
			}
		}
	}
	// Do the same for MetadataOptions...
	if (TSharedPtr<IPropertyHandle> MetadataOptions = DetailLayoutBuilder.GetProperty(TEXT("Inner.MetadataOptions")))
	{
		DetailLayoutBuilder.HideProperty(MetadataOptions);

		uint32 NumChildren = 0;
		if (MetadataOptions->GetNumChildren(NumChildren) == FPropertyAccess::Result::Success)
		{
			for (uint32 Index = 0; Index < NumChildren; ++Index)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = MetadataOptions->GetChildHandle(Index);
				MetadataOptionsCategory.AddProperty(ChildProperty);
			}
		}
	}

	// Hide the textures dir property because we'll add multiple textures folders (one next to each exported material)
	TSharedPtr<IPropertyHandle> TexturesDirProperty = DetailLayoutBuilder.GetProperty(TexturesDirPropName);
	if (TexturesDirProperty->IsValidHandle())
	{
		DetailLayoutBuilder.HideProperty(TexturesDirProperty);
	}

	DetailLayoutBuilder.EditCategory(TEXT("Landscape options"));

	// Customize the level filter picker widget
	DetailLayoutBuilder.HideProperty(LevelFilterProp);
	IDetailCategoryBuilder& CatBuilder = DetailLayoutBuilder.EditCategory(SublayersCategoryName);
	CatBuilder.AddProperty(ExportSublayersProp).EditCondition(SublayersEditCondition, nullptr);

	// clang-format off
	CatBuilder.AddCustomRow(LevelFilterProp->GetPropertyDisplayName())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Levels To Export")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
		.ValueContent()
		.MinDesiredWidth(300)
		[
			SNew(SScrollBorder, PickerTree.ToSharedRef())
			[
				SNew(SBox)	  // Prevent the list from expanding freely
				.MaxDesiredHeight(200)[PickerTree.ToSharedRef()]
			]
		]
		.EditCondition(SublayersEditCondition, nullptr);
	// clang-format on
}

#undef LOCTEXT_NAMESPACE
