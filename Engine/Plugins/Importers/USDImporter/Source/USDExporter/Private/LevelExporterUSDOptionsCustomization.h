// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

class FLevelExporterUSDOptionsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface
};
