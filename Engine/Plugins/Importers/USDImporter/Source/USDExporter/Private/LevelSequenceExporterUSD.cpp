// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceExporterUSD.h"

#include "LevelExporterUSDOptions.h"
#include "LevelSequenceExporterUSDOptions.h"
#include "USDSetUtils.h"
#include "UnrealUSDWrapper.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDExporterModule.h"
#include "USDExportUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDLayerUtils.h"
#include "USDObjectUtils.h"
#include "USDOptionsWindow.h"
#include "USDPrimConversion.h"
#include "USDStageActor.h"
#include "USDUnrealAssetInfo.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "AssetExportTask.h"
#include "Bindings/MovieSceneReplaceableDirectorBlueprintBinding.h"
#include "Bindings/MovieSceneSpawnableDirectorBlueprintBinding.h"
#include "CameraRig_Rail.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SpotLightComponent.h"
#include "Editor.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SphereReflectionCapture.h"
#include "EngineAnalytics.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "LevelEditorSequencerIntegration.h"
#include "LevelSequence.h"
#include "Misc/LevelSequenceEditorSpawnRegister.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSpawnRegister.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Selection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceExporterUSD)

#define LOCTEXT_NAMESPACE "LevelSequenceExporterUSD"

static bool bExportAnimationsFromAllComponents = true;
static FAutoConsoleVariableRef CVarExportAnimationsFromAllComponents(
	TEXT("USD.ExportAnimationsFromAllComponents"),
	bExportAnimationsFromAllComponents,
	TEXT(
		"If true it means that whenever we export LevelSequences to USD we may try exporting transforms and skeletal animations from all components of actors bound to the Sequence, even if those components aren't directly bound themselves. This is useful when using attach sockets or animation blueprints"
	)
);

namespace UE::LevelSequenceExporterUSD::Private
{
	void SendAnalytics(
		const ULevelSequence* LevelSequence,
		const ULevelSequenceExporterUsdOptions* Options,
		const TArray<UE::FUsdStage>& ExportedStages,
		bool bAutomated,
		double ElapsedSeconds,
		const FString& Extension,
		TArray<FAnalyticsEventAttribute> EventAttributes
	)
	{
		if (!LevelSequence || ExportedStages.Num() == 0 || !FEngineAnalytics::IsAvailable())
		{
			return;
		}

		const FString ClassName = IUsdClassesModule::GetClassNameForAnalytics(LevelSequence);

		EventAttributes.Emplace(TEXT("AssetType"), ClassName);
		EventAttributes.Emplace(TEXT("NumExportedLevelSequenceLayers"), ExportedStages.Num());

		int32 NumFrames = 0;
		if (Options)
		{
			NumFrames = Options->EndFrame - Options->StartFrame;
			UsdUtils::AddAnalyticsAttributes(*Options, EventAttributes);
		}

		// Go through a map because we'll get some overlapping attributes here due to reading analytics
		// produced by the level import. Whenever they conflict, we want the LevelSequence values.
		// This will naturally happen here anyway because we add all the LevelSequence attributes afterward,
		// which will overwrite in the map
		TMap<FString, FAnalyticsEventAttribute> AttributesByName;
		AttributesByName.Reserve(EventAttributes.Num());
		for (const FAnalyticsEventAttribute& Attribute : EventAttributes)
		{
			AttributesByName.FindOrAdd(Attribute.GetName()) = Attribute;
		}
		TArray<FAnalyticsEventAttribute> UniqueAttributes;
		AttributesByName.GenerateValueArray(UniqueAttributes);

		IUsdClassesModule::SendAnalytics(
			MoveTemp(UniqueAttributes),
			FString::Printf(TEXT("Export.%s"), *ClassName),
			bAutomated,
			ElapsedSeconds,
			NumFrames,
			Extension
		);
	}

#if USE_USD_SDK

	using FSpawnedInstanceKey = TPair<FGuid, int32>;

	// Custom spawn register so that when DestroySpawnedObject is called while bDestroyingJustHides is true we
	// actually just hide the objects, so that we can keep a live reference to components within the bakers.
	// We're going to convert the spawnable tracks into visibility tracks when exporting to USD, which
	// also works well with this approach
	class FLevelSequenceHidingSpawnRegister : public FLevelSequenceEditorSpawnRegister
	{
	public:

		bool bDestroyingJustHides = true;
		bool bExportSeparatePrimsPerSpawnableInstance = true;

		virtual UObject* SpawnObject(
			const FGuid& Guid,
			UMovieScene& MovieScene,
			FMovieSceneSequenceIDRef TemplateID,
			TSharedRef<const FSharedPlaybackState> SharedPlaybackState,
			int32 BindingIndex
		) override
		{
			FSpawnedInstanceKey InstanceKey(Guid, BindingIndex);

			// Never spawn ASphereReflectionCapture actors. These are useless in USD anyway, and we run into
			// trouble after we're done exporting them because on the tick where they're destroyed the editor
			// will still attempt to update their captures and some downstream code doesn't like that their
			// components are pending kill (check UE-167593 for more info)
			if (Cast<ASphereReflectionCapture>(
					MovieSceneHelpers::GetObjectTemplate(MovieScene.GetTypedOuter<UMovieSceneSequence>(), Guid, SharedPlaybackState)
				))
			{
				return nullptr;
			}

			UMovieSceneSequence* RootSequence = SharedPlaybackState->GetRootSequence();
			if (!RootSequence)
			{
				return nullptr;
			}

			UObject* Object = nullptr;

			TArray<UObject*>& ExistingInstancesForGuid = SpawnableInstances.FindOrAdd(InstanceKey);

			TMap<FMovieSceneSequenceID, TMap<FSpawnedInstanceKey, int32>>& SequenceInstanceToSpawnableIndices = RootSequenceToSpawnableInstanceIndices
																													.FindOrAdd(RootSequence);
			TMap<FSpawnedInstanceKey, int32>& SpawnableIndices = SequenceInstanceToSpawnableIndices.FindOrAdd(TemplateID);

			// Already have an instance of this spawnable for this movie scene sequence instance
			if (int32* ExistingIndex = SpawnableIndices.Find(InstanceKey))
			{
				Object = ExistingInstancesForGuid[*ExistingIndex];
			}
			else if (!bExportSeparatePrimsPerSpawnableInstance && ExistingInstancesForGuid.Num() > 0)
			{
				Object = ExistingInstancesForGuid[0];
				SpawnableIndices.Add(InstanceKey, 0);
			}

			// We don't have an instance of the spawnable spawned for this exact movie sequence ID, but try to see if we can
			// reuse any of the existing spawns for it.
			// We have to reuse these because the exported level will just contain prims that correspond to the spawnables spawned for
			// the top-level level sequence. If we don't reuse these, recursive exports of the subsequences would attempt to spawn
			// their own spawnables (as we always need an instance for each movie scene sequence ID), which wouldn't correspond to
			// any prim that was exported on the level as the actor names would get sanitized away from any name used by the top-level export
			if (!Object && ExistingInstancesForGuid.Num() > 0)
			{
				TArray<bool> UsedIndices;
				UsedIndices.SetNumZeroed(ExistingInstancesForGuid.Num());

				for (const TPair<FMovieSceneSequenceID, TMap<FSpawnedInstanceKey, int32>>& Pair : SequenceInstanceToSpawnableIndices)
				{
					if (const int32* UsedIndex = Pair.Value.Find(InstanceKey))
					{
						UsedIndices[*UsedIndex] = true;
					}
				}

				int32 IndexToReuse = INDEX_NONE;
				for (int32 Index = 0; Index < UsedIndices.Num(); ++Index)
				{
					if (!UsedIndices[Index])
					{
						IndexToReuse = Index;
						break;
					}
				}

				if (IndexToReuse != INDEX_NONE)
				{
					Object = ExistingInstancesForGuid[IndexToReuse];
					SpawnableIndices.Add(InstanceKey, IndexToReuse);
				}
			}

			// We keep track of spawned objects here on our derived class, but the base classes expect the Register to have
			// an entry while the spawnable is spawned, and to not have one when it is not spawned, so here we must synchronize it
			if (Object)
			{
				ESpawnOwnership SpawnOwnership = ESpawnOwnership::InnerSequence;
				FMovieSceneSpawnRegisterKey Key(TemplateID, Guid, BindingIndex);
				Register.Add(Key, FSpawnedObject(Guid, *Object, SpawnOwnership));
			}
			// Don't even have anything we can reuse: We need to spawn a brand new instance of this spawnable
			else
			{
				// SpawnObject will add an entry into the Register for us
				Object = FMovieSceneSpawnRegister::SpawnObject(Guid, MovieScene, TemplateID, SharedPlaybackState, BindingIndex);

				ExistingInstancesForGuid.Add(Object);
				SpawnableIndices.Add(InstanceKey, ExistingInstancesForGuid.Num() - 1);
			}

			if (Object)
			{
				USceneComponent* Component = nullptr;

				if (AActor* Actor = Cast<AActor>(Object))
				{
					const bool bIsHidden = false;
					Actor->SetActorHiddenInGame(bIsHidden);

					Component = Actor->GetRootComponent();
				}
				else
				{
					Component = Cast<USceneComponent>(Object);
				}

				if (Component)
				{
					const bool bNewHidden = false;
					Component->SetHiddenInGame(bNewHidden);
				}
			}

			return Object;
		}

		virtual void PreDestroyObject(UObject& Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID) override
		{
			// Don't let the FLevelSequenceEditorSpawnRegister's overload run as it will mess with our editor selection
			if (bDestroyingJustHides)
			{
				FLevelSequenceSpawnRegister::PreDestroyObject(Object, BindingId, BindingIndex, TemplateID);
				return;
			}

			FLevelSequenceEditorSpawnRegister::PreDestroyObject(Object, BindingId, BindingIndex, TemplateID);
		}

		virtual void DestroySpawnedObject(UObject& Object, UMovieSceneSpawnableBindingBase* CustomSpawnableBinding) override
		{
			// We don't have to clean up the Register here, the caller to DestroySpawnedObject will do that

			if (bDestroyingJustHides)
			{
				USceneComponent* Component = nullptr;

				if (AActor* Actor = Cast<AActor>(&Object))
				{
					const bool bIsHidden = true;
					Actor->SetActorHiddenInGame(bIsHidden);

					Component = Actor->GetRootComponent();
				}
				else
				{
					Component = Cast<USceneComponent>(&Object);
				}

				// Make sure we toggle both actor and component as the component is the one
				// that is monitored by the level exporter, and we want our spawnables to be hidden
				// by default
				if (Component)
				{
					const bool bNewHidden = true;
					Component->SetHiddenInGame(bNewHidden);
				}
			}
			else
			{
				// We shouldn't need to do this because we only ever fully delete when we're cleaning up,
				// and by then we'll delete all of these maps anyway
				for (TPair<FSpawnedInstanceKey, TArray<UObject*>>& Pair : SpawnableInstances)
				{
					const FSpawnedInstanceKey& InstanceKey = Pair.Key;
					TArray<UObject*>& InstancesForGuid = Pair.Value;

					int32 IndexToDelete = INDEX_NONE;
					for (int32 Index = 0; Index < InstancesForGuid.Num(); ++Index)
					{
						if (InstancesForGuid[Index] == &Object)
						{
							IndexToDelete = Index;
							break;
						}
					}

					if (IndexToDelete != INDEX_NONE)
					{
						for (TPair<const UMovieSceneSequence*, TMap<FMovieSceneSequenceID, TMap<FSpawnedInstanceKey, int32>>>& RootSequencePair :
							 RootSequenceToSpawnableInstanceIndices)
						{
							for (TPair<FMovieSceneSequenceID, TMap<FSpawnedInstanceKey, int32>>& SequenceIDPair : RootSequencePair.Value)
							{
								TMap<FSpawnedInstanceKey, int32>& GuidToInstance = SequenceIDPair.Value;
								if (int32* InstanceIndex = GuidToInstance.Find(InstanceKey))
								{
									if (*InstanceIndex == IndexToDelete)
									{
										GuidToInstance.Remove(InstanceKey);
									}
								}
							}
						}

						InstancesForGuid.RemoveAt(IndexToDelete);
						break;
					}
				}

				FLevelSequenceEditorSpawnRegister::DestroySpawnedObject(Object, CustomSpawnableBinding);
			}
		}

		bool HasSpawnedObject(const FSpawnedInstanceKey& InstanceKey) const
		{
			return SpawnableInstances.Contains(InstanceKey);
		}

		void DeleteSpawns(TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
		{
			bDestroyingJustHides = false;
			CleanUp(SharedPlaybackState);

			// If we still have existing spawns it may be because our base class' Register member didn't contain an
			// entry for a spawnable before we called CleanUp (check its implementation: It just iterates over that Register).
			// This is expected in some scenarios because we're sort of abusing this inheritance: The base class expects
			// CleanUp to delete the object already and so removes its Register entry, but we'll keep the instances
			// alive when bDestroyingJustHides=true. Because of this we must explicitly clean up these "abandoned" spawns here,
			// which resynchronizes us with Register:
			TArray<UObject*> ObjectsToDelete;
			for (TPair<FSpawnedInstanceKey, TArray<UObject*>>& Pair : SpawnableInstances)
			{
				ObjectsToDelete.Append(Pair.Value);
			}
			for (UObject* Object : ObjectsToDelete)
			{
				if (Object)
				{
					// TODO: I think this will likely be fine for most cases, but I could see it potentially being problematic in all cases to destroy
					// an object in the default way this way. However, to do this 'correctly' would also involve storing the UCustomBinding object in
					// this process, which I'm skipping for now.
					DestroySpawnedObject(*Object, nullptr);
				}
			}
		}

	private:
		// Ensures all of our new spawns have unique names
		TSet<FString> UsedActorLabels;

		// Tracks all instances we created for a given spawnable guid and binding index
		TMap<FSpawnedInstanceKey, TArray<UObject*>> SpawnableInstances;

		// Tracks the indices into SpawnableInstances for each spawnable guid, used by each sequence ID, in the hierarchy of each root
		// sequence
		TMap<const UMovieSceneSequence*, TMap<FMovieSceneSequenceID, TMap<FSpawnedInstanceKey, int32>>> RootSequenceToSpawnableInstanceIndices;
	};

	// Contain all of the baker lambda functions for a given component. Only one baker per baking type is allowed.
	struct FCombinedComponentBakers
	{
		UnrealToUsd::EBakingType CombinedBakingType = UnrealToUsd::EBakingType::None;
		TArray<UnrealToUsd::FComponentBaker> Bakers;
	};

	class FLevelSequenceExportContext
	{
	public:
		FLevelSequenceExportContext(
			ULevelSequence& InSequence,
			TSharedRef<ISequencer> InSequencer,
			TSharedRef<FLevelSequenceHidingSpawnRegister> InSpawnRegister
		)
			: RootSequence(InSequence)
			, Sequencer(InSequencer)
			, SpawnRegister(InSpawnRegister)
		{
		}

		FLevelSequenceExportContext() = delete;
		FLevelSequenceExportContext(const FLevelSequenceExportContext& Other) = delete;
		FLevelSequenceExportContext(const FLevelSequenceExportContext&& Other) = delete;
		FLevelSequenceExportContext& operator=(const FLevelSequenceExportContext& Other) = delete;
		FLevelSequenceExportContext& operator=(const FLevelSequenceExportContext&& Other) = delete;

	public:
		// The actual content asset that is being exported
		ULevelSequence& RootSequence;

		ULevelSequenceExporterUsdOptions* ExportOptions;

		// If ExportOptions->bSelectionOnly is true, this specifies the actors whose bindings we should export
		TSet<AActor*> SelectedActors;

		// Where we store our ExportTask's bReplaceIdentical, which indicates if we should overwrite files or not
		bool bReplaceIdentical;

		// Our own read-only sequencer that we use to play the level sequences while we bake them out one frame at a time
		TSharedRef<ISequencer> Sequencer;

		// Object that manages spawned instances for FMovieScenePossessables
		TSharedRef<FLevelSequenceHidingSpawnRegister> SpawnRegister;

		// Used to keep track of which sequences we already baked
		TMap<UMovieSceneSequence*, FString> ExportedMovieScenes;

		// File paths that we already used during this export.
		// Used so that we can prevent conflicts between files emitted for this export, but can still
		// overwrite other files on disk
		TSet<FString> UsedFilePaths;

		// File path of the exported USD root layer, in case we also exported the level along with the level sequence
		FString LevelFilePath;

		// World that is being animated for the LevelSequence bake
		UWorld* World = nullptr;

	public:
		~FLevelSequenceExportContext()
		{
			SpawnRegister->DeleteSpawns(Sequencer->GetSharedPlaybackState());
		}
	};

	bool IsTrackAnimated(const UMovieSceneTrack* Track)
	{
		for (const UMovieSceneSection* Section : Track->GetAllSections())
		{
			// We can't just check whether a section has range here because it may just have a bunch
			// of channels with modified default values instead (and so no range or even keys, but can still affect the level).
			// Sadly there's no way of telling if the default value in the channel has been modified or not (that is, whether it
			// matches the unanimated value or not), so we'll just have to export any track with an active section for now.
			// This will emit a bunch of unwanted tracks, but later on we may want to do a post-processing pass on our exported
			// data to make sure we don't emit intermediary keys on linearly interpolated sections, and that would naturally fix
			// this too.
			if (Section->IsActive())
			{
				return true;
			}
		}

		return false;
	}

	TMap<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>> GetSequenceHierarchyInstances(
		UMovieSceneSequence& Sequence,
		ISequencer& Sequencer,
		FMovieSceneSequenceHierarchy& InOutHierarchyCache
	)
	{
		TMap<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>> SequenceInstances;

		UMovieSceneCompiledDataManager::CompileHierarchy(&Sequence, &InOutHierarchyCache, EMovieSceneServerClientMask::All);

		SequenceInstances.FindOrAdd(&Sequence).Add(Sequencer.GetRootTemplateID());
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : InOutHierarchyCache.AllSubSequenceData())
		{
			if (ULevelSequence* SubSequence = Cast<ULevelSequence>(Pair.Value.GetSequence()))
			{
				SequenceInstances.FindOrAdd(SubSequence).Add(Pair.Key);
			}
		}

		return SequenceInstances;
	}

	// Spawn and hide all spawnables for Sequence hierarchy
	TMap<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>> PreSpawnSpawnables(
		FLevelSequenceExportContext& Context,
		UMovieSceneSequence& RootSequence
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULevelSequenceExporterUsd::PreSpawnSpawnables);

		FMovieSceneSequenceHierarchy HierarchyCache;
		TMap<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>> SequenceInstances = GetSequenceHierarchyInstances(
			RootSequence,
			Context.Sequencer.Get(),
			HierarchyCache
		);

		UMovieSceneSequence* OrigRootSequence = Context.Sequencer->GetRootMovieSceneSequence();
		Context.Sequencer->ResetToNewRootSequence(RootSequence);

		// Spawn everything for this Sequence hierarchy
		for (TPair<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>>& SequenceInstancePair : SequenceInstances)
		{
			const UMovieSceneSequence* Sequence = SequenceInstancePair.Key;
			if (!Sequence)
			{
				continue;
			}

			UMovieScene* MovieScene = Sequence->GetMovieScene();
			if (!MovieScene)
			{
				continue;
			}

			// Spawn everything for this instance
			for (FMovieSceneSequenceID SequenceInstance : SequenceInstancePair.Value)
			{
				if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
				{
					int32 BindingIndex = 0;
					FGuid LastGuid;
					for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
					{
						if (LastGuid != BindingReference.ID)
						{
							LastGuid = BindingReference.ID;
							BindingIndex = 0;
						}
						if (BindingReference.CustomBinding
							&& BindingReference.CustomBinding->WillSpawnObject(Context.Sequencer->GetSharedPlaybackState()))
						{
							StaticCastSharedRef<FMovieSceneSpawnRegister>(Context.SpawnRegister)
								->SpawnObject(
									BindingReference.ID,
									*MovieScene,
									SequenceInstance,
									Context.Sequencer->GetSharedPlaybackState(),
									BindingIndex++
								);
						}
					}
				}
			}
		}

		// Recurse into subsequences if we'll export them as sublayers. This because when exporting each, we'll reset our sequencer
		// to have each subsequence as the root level sequence again, which means that the sequence ids will all change. We need to
		// prepare our SpawnRegister to be able to reuse the previous spawns for these new instance ids
		if (Context.ExportOptions->bExportSubsequencesAsLayers)
		{
			for (TPair<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>>& SequenceInstancePair : SequenceInstances)
			{
				UMovieSceneSequence* Sequence = SequenceInstancePair.Key;
				if (!Sequence || Sequence == &RootSequence)
				{
					continue;
				}

				PreSpawnSpawnables(Context, *Sequence);
			}
		}

		// Put this back to what it was before this call
		if (OrigRootSequence)
		{
			Context.Sequencer->ResetToNewRootSequence(*OrigRootSequence);
		}

		return SequenceInstances;
	}

	// Collect selected actors and components before we do anything.
	// Note that this is especially needed for spawnables since we're going to have to replicate the selection on
	// the analogous spawnables that belong to our TempSequencer
	void CollectEditorSelection(
		const FLevelSequenceHidingSpawnRegister& SpawnRegister,
		TArray<AActor*>& OutSelectedSpawnableActors,
		TArray<AActor*>& OutSelectedNonSpawnableActors,
		TArray<UE::MovieScene::FFixedObjectBindingID>& OutSelectedSpawnableBindings
	)
	{
		OutSelectedSpawnableActors.Reset();
		OutSelectedNonSpawnableActors.Reset();
		OutSelectedSpawnableBindings.Reset();

		// We'll start off with all actors inside OutSelectedNonSpawnableActors, and then remove the
		// spawnables and put them on OutSelectedSpawnableActors
		USelection* ActorSelection = GEditor->GetSelectedActors();
		ActorSelection->GetSelectedObjects(OutSelectedNonSpawnableActors);

		OutSelectedSpawnableActors.Reserve(OutSelectedNonSpawnableActors.Num());

		TArray<UE::MovieScene::FFixedObjectBindingID> SpawnableBindings;
		for (const TWeakPtr<ISequencer>& Sequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
		{
			if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
			{
				// Components are never "spawnable", so we only need to get these bindings for actors
				for (int32 Index = OutSelectedNonSpawnableActors.Num() - 1; Index >= 0; --Index)
				{
					AActor* Actor = OutSelectedNonSpawnableActors[Index];

					TArray<FMovieSceneObjectBindingID> Bindings;
					PinnedSequencer->GetEvaluationState()->FilterObjectBindings(Actor, *PinnedSequencer, &Bindings);

					OutSelectedSpawnableBindings.Reserve(Bindings.Num() + OutSelectedSpawnableBindings.Num());
					for (const FMovieSceneObjectBindingID& Binding : Bindings)
					{
						// Using the spawn register is an easy way of telling if a Guid is a spawnable or not,
						// but it's more appropriate because we really only ever care about the spawnables that
						// we have spawned on our TempSequencer
						const bool bIsSpawnable = SpawnRegister.HasSpawnedObject(FSpawnedInstanceKey(Binding.GetGuid(), 0));
						if (!bIsSpawnable)
						{
							continue;
						}

						// We must keep fixed bindings only as those are kind of like "absolute file paths" and can be used
						// to identify any of the bindings within this LevelSequence's template hierarchy. The alternative
						// would be a relative binding ID, which would only be useful within the context of a particular
						// subsequence of the root sequence
						UE::MovieScene::FFixedObjectBindingID NewBinding;
						if (Binding.IsFixedBinding())
						{
							NewBinding = Binding.ReinterpretAsFixed();
						}
						else
						{
							NewBinding = Binding.ResolveToFixed(Binding.GetRelativeSequenceID(), *PinnedSequencer);
						}
						OutSelectedSpawnableBindings.Add(NewBinding);
						OutSelectedSpawnableActors.Add(Actor);

						const int32 Count = 1;
						OutSelectedNonSpawnableActors.RemoveAt(Index, Count, EAllowShrinking::No);
					}
				}
			}
		}
	}

	void TransferSpawnableSelection(
		ISequencer& Sequencer,
		const TArray<UE::MovieScene::FFixedObjectBindingID>& SelectedSpawnableBindings,
		TArray<AActor*>& OutSelectedActors
	)
	{
		USelection* ActorSelection = GEditor->GetSelectedActors();
		if (!ensure(ActorSelection))
		{
			return;
		}
		ActorSelection->Modify();
		ActorSelection->BeginBatchSelectOperation();

		OutSelectedActors.Reset();

		for (const UE::MovieScene::FFixedObjectBindingID& FixedBinding : SelectedSpawnableBindings)
		{
			FMovieSceneObjectBindingID BindingID{FixedBinding};
			TArrayView<TWeakObjectPtr<UObject>> BoundObjects = BindingID.ResolveBoundObjects(FixedBinding.SequenceID, Sequencer);
			for (const TWeakObjectPtr<UObject>& BoundObject : BoundObjects)
			{
				if (AActor* Actor = Cast<AActor>(BoundObject.Get()))
				{
					ActorSelection->Select(Actor);

					OutSelectedActors.Add(Actor);
				}
			}
		}

		const bool bNotify = true;
		ActorSelection->EndBatchSelectOperation(bNotify);
	}

	void RestoreEditorSelection(
		const TArray<AActor*>& SelectedNonSpawnableActors,
		const TArray<UE::MovieScene::FFixedObjectBindingID>& SelectedSpawnableBindings
	)
	{
		USelection* ActorSelection = GEditor->GetSelectedActors();
		if (!ensure(ActorSelection))
		{
			return;
		}
		ActorSelection->Modify();
		ActorSelection->BeginBatchSelectOperation();
		ActorSelection->DeselectAll();

		for (AActor* Actor : SelectedNonSpawnableActors)
		{
			ActorSelection->Select(Actor);
		}

		// Note that we're not tracking *which* sequencer produced which spawnable originally, because realistically the
		// user will only ever have a single sequencer editing a particular LevelSequence at a time, if that
		for (const TWeakPtr<ISequencer>& Sequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
		{
			if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
			{
				for (const UE::MovieScene::FFixedObjectBindingID& FixedBinding : SelectedSpawnableBindings)
				{
					FMovieSceneObjectBindingID BindingID{FixedBinding};
					TArrayView<TWeakObjectPtr<UObject>> BoundObjects = BindingID.ResolveBoundObjects(FixedBinding.SequenceID, *PinnedSequencer);
					for (const TWeakObjectPtr<UObject>& BoundObject : BoundObjects)
					{
						if (AActor* Actor = Cast<AActor>(BoundObject.Get()))
						{
							ActorSelection->Select(Actor);
						}
					}
				}
			}
		}

		const bool bNotify = true;
		ActorSelection->EndBatchSelectOperation(bNotify);
	}

	// Export the provided AudioTrack to Prim as UsdMediaSpatialAudio attributes
	//
	// Exporting this track type takes a different approach because unlike all other animation types, there is no actual
	// change on the component on the level while the LevelSequence plays audio. This means that reading the "final output"
	// of the sequence on the component every EvalFrame with a baker (like all other track cases do) doesn't really do
	// anything for us, and we actually need to traverse the Sequencer tracks themselves.
	//
	// Of course, we won't get the benefit of the previous approach here: If we have multiple audio tracks for the same
	// actor/component, even if they're placed within different Subsequences, they *will* conflict on the USD files.
	// There is not much we can do about that at this point other than to emit a warning, but hopefully having multiple
	// audio tracks on the same audio component is something that doesn't happen very often in practice anyway. If that
	// is ever requested, in the future we could handle it by creating a separate UsdMediaSpatialAudio prim per audio section,
	// but that will make a bit of a mess and harm roundtripping, so for now we only handle one section.
	void ExportAudioTrack(
		const UMovieSceneAudioTrack& AudioTrack,
		const FMovieSceneSequenceTransform& SequenceTransform,
		UE::FUsdPrim& Prim,
		UsdUtils::TCaseSensitiveStringMap<int32>& AudioTracksPerPrim
	)
	{
		FString PrimPath = Prim.GetPrimPath().GetString();

		const TArray<UMovieSceneSection*>& Sections = AudioTrack.GetAudioSections();
		if (Sections.Num() > 1)
		{
			// We only support one audio section per track because we need a full UsdMediaSpatialAudio prim for each
			// section. If we tried exporting another section here we'd need a fully separate prim for it, which opens
			// a can of worms as we so far only had one prim per binding. Furthermore we'd need to pay attention to this
			// split when opening the stage as well, otherwise we'd roundtrip the audio track incorrectly
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"MultipleAudioSections",
					"The audio track '{0}' has {1} sections, but only the first audio section of an audio track can be written out to USD for now"
				),
				FText::FromString(AudioTrack.GetPathName()),
				Sections.Num()
			));
		}

		if (Sections.Num() > 0)
		{
			if (UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Sections[0]))
			{
				UnrealToUsd::ConvertAudioSection(*AudioSection, SequenceTransform, Prim);
				AudioTracksPerPrim.FindOrAdd(PrimPath) += 1;
			}
		}

		if (int32* SourceTracksForPrim = AudioTracksPerPrim.Find(PrimPath))
		{
			if (*SourceTracksForPrim > 1)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"MultipleAudioTracks",
						"Exporting multiple audio tracks (like '{0}') to the same prim ('{1}') is currently not supported and may lead to incorrect output"
					),
					FText::FromString(AudioTrack.GetPathName()),
					FText::FromString(PrimPath)
				));
			}
		}
	}

	// Appends to InOutComponentBakers all of the component bakers for all components bound to MovieSceneSequence.
	// In the process it will generate the output prims for each of these components, and keep track of them
	// within the bakers themselves
	void GenerateBakersForMovieScene(
		FLevelSequenceExportContext& Context,
		UMovieSceneSequence& MovieSceneSequence,
		const TMap<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>>& SequenceInstances,
		const FMovieSceneSequenceHierarchy& HierarchyCache,
		UE::FUsdStage& UsdStage,
		TMap<USceneComponent*, FCombinedComponentBakers>& InOutComponentBakers
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULevelSequenceExporterUsd::GenerateBakersForMovieScene);

		UMovieScene* MovieScene = MovieSceneSequence.GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		// Find all objects bound to the movie scene. We'll track UObjects here because we may have tracks
		// bound to root components but also separate tracks bound directly to the actors, and we want to
		// capture both.
		// Index from UObject to FGuid because we may have multiple spawned objects for a given spawnable Guid
		TMap<UObject*, FSpawnedInstanceKey> BoundObjects;

		// Collect any USD-related DynamicBinding. The idea being that if we find any, we're likely looking at a
		// loaded USD Stage that's going to be exported, and the possessable is one of the transient actors and
		// components. It that's the case, we don't want to just come up with a random name for the prim based on
		// the actor/component path, but instead want to use the prim path that it has been given on the dynamic binding,
		// if any
		TMap<FSpawnedInstanceKey, const FMovieSceneDynamicBinding*> DynamicBindings;

		const TArray<FMovieSceneSequenceID>* InstancesOfThisSequence = SequenceInstances.Find(&MovieSceneSequence);
		if (!InstancesOfThisSequence)
		{
			return;
		}

		UMovieSceneSequence* RootSequence = Context.Sequencer->GetRootMovieSceneSequence();
		if (!RootSequence)
		{
			return;
		}

		// Force spawn spawnables again here so that they exist on the Register map when we rely on FindBoundObjects
		// to resolve bindings
		PreSpawnSpawnables(Context, MovieSceneSequence);

		for (FMovieSceneSequenceID SequenceInstance : *InstancesOfThisSequence)
		{
			FMovieSceneObjectCache& ObjectCache = Context.Sequencer->GetEvaluationState()->GetObjectCache(SequenceInstance);

			// Possessables
			int32 NumPossessables = MovieScene->GetPossessableCount();
			for (int32 Index = 0; Index < NumPossessables; ++Index)
			{
				const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
				const FGuid& Guid = Possessable.GetGuid();

				UObject* BoundObject = nullptr;

				// We need to check for custom spawnables here as well.
				// Note: Now all sequencer bindings are possessables, even the old spawnables. This is why we loop
				// over all binding references here, and will attempt to use the SpawnRegister even for possessables
				if (const FMovieSceneBindingReferences* BindingReferences = MovieSceneSequence.GetBindingReferences())
				{
					int32 BindingIndex = 0;
					FGuid LastGuid;
					for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
					{
						if (LastGuid != BindingReference.ID)
						{
							LastGuid = BindingReference.ID;
							BindingIndex = 0;
						}
						if (BindingReference.CustomBinding)
						{
							if (BindingReference.CustomBinding->WillSpawnObject(Context.Sequencer->GetSharedPlaybackState()))
							{
								BoundObject = Context.SpawnRegister->FindSpawnedObject(Guid, SequenceInstance, BindingIndex++).Get();
								if (!BoundObject)
								{
									continue;
								}

								if (BoundObject && (BoundObject->IsA<USceneComponent>() || BoundObject->IsA<AActor>()))
								{
									BoundObjects.Add(BoundObject, FSpawnedInstanceKey(Guid, BindingIndex));
								}
							}

							if (UMovieSceneSpawnableDirectorBlueprintBinding* SpawnableDirectorBlueprintBinding = Cast<
									UMovieSceneSpawnableDirectorBlueprintBinding>(
									BindingReference.CustomBinding->AsSpawnable(Context.Sequencer->GetSharedPlaybackState())
								))
							{
								if (SpawnableDirectorBlueprintBinding->DynamicBinding.Function)
								{
									DynamicBindings.Add(FSpawnedInstanceKey(Guid, BindingIndex), &SpawnableDirectorBlueprintBinding->DynamicBinding);
								}
							}

							if (UMovieSceneReplaceableDirectorBlueprintBinding* ReplaceableDirectorBlueprintBinding = Cast<
									UMovieSceneReplaceableDirectorBlueprintBinding>(BindingReference.CustomBinding))
							{
								if (ReplaceableDirectorBlueprintBinding->DynamicBinding.Function)
								{
									DynamicBindings.Add(
										FSpawnedInstanceKey(Guid, BindingIndex),
										&ReplaceableDirectorBlueprintBinding->DynamicBinding
									);
								}
							}
						}
					}
				}

				// Go through FMovieSceneObjectCache and FindBoundObjects because that will also evaluate DynamicBindings.
				// The idea here is that FindBoundObjects will manage to find the binding even it their "parent context" is
				// a spawnable (e.g. if it's a possessable component of a spawnable) and also uses DynamicBindings, which is great!
				// For it to be able to find our spawns however, the spawnables must be *currently* spawned.
				TArrayView<TWeakObjectPtr<UObject>> ObjectWeakPtrs = ObjectCache.FindBoundObjects(Guid, *Context.Sequencer);
				if (ObjectWeakPtrs.Num() > 0)
				{
					BoundObject = ObjectWeakPtrs[0].Get();
				}

				if (BoundObject && (BoundObject->IsA<USceneComponent>() || BoundObject->IsA<AActor>()))
				{
					BoundObjects.Add(BoundObject, FSpawnedInstanceKey(Guid, 0));
				}
			}
		}

		// Expand BoundObjects to include all components of all of its bound actors (even those without any binding to the LevelSequence).
		// The idea here is that even if these don't have any tracks, the attach socket and AnimBlueprint fallbacks at the bottom of the
		// loop below will still be triggered, letting us automatically capture the animations of these components that are "indirectly
		// animated"
		if (bExportAnimationsFromAllComponents)
		{
			TMap<UObject*, FSpawnedInstanceKey> NewEntries;
			for (const TPair<UObject*, FSpawnedInstanceKey>& Pair : BoundObjects)
			{
				if (AActor* Actor = Cast<AActor>(Pair.Key))
				{
					if (USceneComponent* Root = Actor->GetRootComponent())
					{
						const bool bIncludeAllDescendants = true;
						TArray<USceneComponent*> Children;
						Root->GetChildrenComponents(bIncludeAllDescendants, Children);

						for (USceneComponent* Child : Children)
						{
							// Skip hidden billboards/arrows/camera mesh components, etc.
							if (!Child || !Child->IsVisibleInEditor() || Child->IsVisualizationComponent())
							{
								continue;
							}

							NewEntries.Add(Child, FSpawnedInstanceKey{});
						}
					}
				}
			}
			NewEntries.Append(BoundObjects);	// Prefer values from BoundObjects
			NewEntries.Remove(nullptr);
			Swap(NewEntries, BoundObjects);
		}

		// Generate bakers
		for (const TPair<UObject*, FSpawnedInstanceKey>& Pair : BoundObjects)
		{
			UObject* BoundObject = Pair.Key;
			const FSpawnedInstanceKey& InstanceKey = Pair.Value;

			// We always use components here because when exporting actors and components to USD we basically
			// just ignore actors altogether and export the component attachment hierarchy instead
			USceneComponent* BoundComponent = nullptr;
			AActor* BoundActor = nullptr;
			if (USceneComponent* Component = Cast<USceneComponent>(BoundObject))
			{
				BoundComponent = Component;
				BoundActor = BoundComponent->GetOwner();
			}
			else if (AActor* Actor = Cast<AActor>(BoundObject))
			{
				BoundActor = Actor;
				BoundComponent = Actor->GetRootComponent();
			}

			if (!BoundComponent || !BoundActor)
			{
				continue;
			}
			if (Context.ExportOptions->bSelectionOnly && !Context.SelectedActors.Contains(BoundActor))
			{
				continue;
			}

			FString PrimPath = UsdUtils::GetPrimPathForObject(
				BoundComponent,
				TEXT(""),
				Context.ExportOptions && Context.ExportOptions->LevelExportOptions.bExportActorFolders,
				Context.ExportOptions->LevelExportOptions.RootPrimName
			);
			if (PrimPath.IsEmpty())
			{
				continue;
			}

			// If this binding has one of our dynamic bindings set up pointing to a valid prim path, let's use that path
			// instead of using our generated PrimPath, as that one will better match the prim paths that we'll get when
			// opening a referenced stage via an exported UsdStageActor
			if (const FMovieSceneDynamicBinding* DynamicBinding = DynamicBindings.FindRef(InstanceKey))
			{
				if (const FMovieSceneDynamicBindingPayloadVariable* FoundPrimPathPayload = DynamicBinding->PayloadVariables.Find(TEXT("PrimPath")))
				{
					FString PrimPathInSourceStage = FoundPrimPathPayload->Value;
					if (!PrimPathInSourceStage.IsEmpty())
					{
						AActor* PossibleParentStageActor = BoundActor;
						while (PossibleParentStageActor && !PossibleParentStageActor->IsA<AUsdStageActor>())
						{
							PossibleParentStageActor = PossibleParentStageActor->GetAttachParentActor();
						}

						// Our possessable has a dynamic binding with a "PrimPath" payload variable and is a child of a stage actor,
						// for now let's consider this enough to consider this is one of our dynamic bindings
						if (AUsdStageActor* StageActor = Cast<AUsdStageActor>(PossibleParentStageActor))
						{
							FString ParentStageActorPrimPathOnExport = UsdUtils::GetPrimPathForObject(
								StageActor,
								TEXT(""),
								Context.ExportOptions && Context.ExportOptions->LevelExportOptions.bExportActorFolders,
								Context.ExportOptions->LevelExportOptions.RootPrimName
							);
							if (!ParentStageActorPrimPathOnExport.IsEmpty())
							{
								UE::FSdfPath SdfPrimPathInSourceStage{*PrimPathInSourceStage};

								UE::FUsdStage LoadedStage = StageActor->GetBaseUsdStage();
								UE::FSdfPath DefaultPrimPath = LoadedStage.GetDefaultPrim().GetPrimPath();

								if (!LoadedStage.GetRootLayer().IsAnonymous() && !DefaultPrimPath.IsEmpty()
									&& SdfPrimPathInSourceStage.HasPrefix(DefaultPrimPath))
								{
									// Note that it's perfectly fine if this ends up being just "."
									UE::FSdfPath RelativePrimPathInSourceStage = SdfPrimPathInSourceStage.MakeRelativePath(DefaultPrimPath);

									FString PrimPathRelativeToParentStageActor = UE::FSdfPath{*ParentStageActorPrimPathOnExport}
																					 .AppendPath(RelativePrimPathInSourceStage)
																					 .GetString();

									if (!PrimPathRelativeToParentStageActor.IsEmpty())
									{
										PrimPath = PrimPathRelativeToParentStageActor;
									}
								}
								else
								{
									USD_LOG_USERWARNING(FText::Format(
										LOCTEXT(
											"NonIdealComposition",
											"Exported animation for prim '{0}' may not compose correctly with the prims from referenced layer '{1}' on the exported stage for the LevelSequence '{2}'. For best results, make sure the referenced layer is saved to disk (i.e. not anonymous), has a defaultPrim setup, and that the animation tracks are only bound to prims that are descendents of the defaultPrim."
										),
										FText::FromString(PrimPathInSourceStage),
										FText::FromString(LoadedStage.GetRootLayer().GetIdentifier()),
										FText::FromString(Context.RootSequence.GetPathName())
									));
								}
							}
						}
					}
				}
			}

			auto GetPrimForComponent = [&Context, &UsdStage](const USceneComponent& Component, FString* PrimPathStr = nullptr) -> UE::FUsdPrim
			{
				FString PrimPath = PrimPathStr ? *PrimPathStr
											   : UsdUtils::GetPrimPathForObject(
													 &Component,
													 TEXT(""),
													 Context.ExportOptions && Context.ExportOptions->LevelExportOptions.bExportActorFolders,
													 Context.ExportOptions->LevelExportOptions.RootPrimName
												 );
				if (PrimPath.IsEmpty())
				{
					return {};
				}

				FString SchemaName = UsdUtils::GetSchemaNameForComponent(Component);
				if (SchemaName.IsEmpty())
				{
					return {};
				}

				// We will define a prim here so that we can apply schemas and use the shortcut CreateXAttribute functions,
				// and not have to worry about attribute names and types. Later on we will convert these prims back into just 'overs' though
				UE::FUsdPrim Prim = UsdStage.DefinePrim(UE::FSdfPath{*PrimPath}, *SchemaName);

				// For the spot light case we also have to author our API schema here, or else our property bakers will fail to see this
				// as a valid prim for a spot light
				if (Prim && Component.IsA<USpotLightComponent>())
				{
					Prim.ApplyAPI(TEXT("ShapingAPI"));
				}

				return Prim;
			};

			UE::FUsdPrim Prim = GetPrimForComponent(*BoundComponent, &PrimPath);
			if (!Prim)
			{
				continue;
			}

			TFunction<void(UnrealToUsd::FComponentBaker&)> AddBaker = [&InOutComponentBakers, &BoundComponent](UnrealToUsd::FComponentBaker& Baker)
			{
				// If we made a baker and we don't have one of this type for this component yet, add its lambda to the array
				FCombinedComponentBakers& ExistingBakers = InOutComponentBakers.FindOrAdd(BoundComponent);
				if (Baker.BakerType != UnrealToUsd::EBakingType::None && !EnumHasAnyFlags(ExistingBakers.CombinedBakingType, Baker.BakerType))
				{
					ExistingBakers.Bakers.Add(Baker);
					ExistingBakers.CombinedBakingType |= Baker.BakerType;
				}
			};

			const bool bBakeAsSkeletal = !Context.ExportOptions->LevelExportOptions.AssetOptions.bConvertSkeletalToNonSkeletal;
			TFunction<void(UnrealToUsd::FComponentBaker&)> GenerateSkeletalBaker =
				[&UsdStage, &BoundComponent, &PrimPath, &Prim, bBakeAsSkeletal](UnrealToUsd::FComponentBaker& InOutBaker)
			{
				if (USkeletalMeshComponent* SkeletalBoundComponent = Cast<USkeletalMeshComponent>(BoundComponent))
				{
					if (bBakeAsSkeletal)
					{
						UE::FUsdPrim SkelAnimPrim = UsdStage.DefinePrim(UE::FSdfPath{*PrimPath}.AppendChild(TEXT("Anim")), TEXT("SkelAnimation"));

						UE::FUsdPrim SkeletonPrim = UsdStage.DefinePrim(
							UE::FSdfPath{*PrimPath}.AppendChild(UnrealIdentifiers::ExportedSkeletonPrimName),
							TEXT("Skeleton")
						);

						if (SkelAnimPrim && SkeletonPrim)
						{
							UnrealToUsd::CreateSkeletalAnimationBaker(SkeletonPrim, SkelAnimPrim, *SkeletalBoundComponent, InOutBaker);
						}
						else
						{
							USD_LOG_WARNING(TEXT("Failed to generate Skeleton or SkelAnimation prim when baking out SkelRoot '%s'"), *PrimPath);
						}
					}
					else
					{
						// Convert the prim for the skeletal mesh component from SkelRoot to Mesh
						Prim.SetTypeName(TEXT("Mesh"));
						UnrealToUsd::CreateSkeletalAnimationToMeshBaker(Prim, *SkeletalBoundComponent, InOutBaker);
					}
				}
			};

			bool bHasTransformBaker = false;
			bool bHasSkeletalBaker = false;

			if (ACameraRig_Rail* RailActor = Cast<ACameraRig_Rail>(BoundObject))
			{
				// In the case of a CameraRig_Rail, what we want to bake is the transform animation of its
				// RailMountComponent since that is where children camera will be attached to.
				const static FString TransformPropertyPath = UnrealIdentifiers::TransformPropertyName.ToString();
				USceneComponent* RailMountComponent = RailActor->GetDefaultAttachComponent();
				UE::FUsdPrim RailMountPrim = GetPrimForComponent(*RailMountComponent);
				if (RailMountPrim)
				{
					UnrealToUsd::FComponentBaker Baker;
					if (UnrealToUsd::CreateComponentPropertyBaker(RailMountPrim, *RailMountComponent, TransformPropertyPath, Baker))
					{
						AddBaker(Baker);
						bHasTransformBaker = true;
					}
				}
			}

			UsdUtils::TCaseSensitiveStringMap<int32> AudioTracksPerPrim;
			if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InstanceKey.Key))
			{
				for (const UMovieSceneTrack* Track : Binding->GetTracks())
				{
					// Let even non-animated spawn tracks through because even if they have zero keyframes we'll still need
					// to bake the visibility of the spawnables despawning when e.g. a subsequence shot is over
					if (!Track || (!IsTrackAnimated(Track) && !Track->IsA<UMovieSceneSpawnTrack>()))
					{
						continue;
					}

					UnrealToUsd::FComponentBaker Baker;

					if (const UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(Track))
					{
						const FString PropertyPath = PropertyTrack->GetPropertyPath().ToString();
						UnrealToUsd::CreateComponentPropertyBaker(Prim, *BoundComponent, PropertyPath, Baker);
					}
					else if (Track->IsA<UMovieSceneSpawnTrack>())
					{
						// Just handle spawnable tracks as if they're visibility tracks, and hide the prim when not "spawned"
						// Remember that our spawn register just hides the spawnables when they're not spawned anyway, so this
						// is essentially the same
						const static FString PropertyPath = UnrealIdentifiers::HiddenPropertyName.ToString();
						UnrealToUsd::CreateComponentPropertyBaker(Prim, *BoundComponent, PropertyPath, Baker);
					}
					// Check for the control rig tracks too, because if the user did "Bake to Control Rig" the controlrig code will silently
					// set the original skeletal animation track sections as disabled, so they'd fail the "IsTrackAnimated" check above
					else if (Track->IsA<UMovieSceneSkeletalAnimationTrack>() || Track->IsA<UMovieSceneControlRigParameterTrack>())
					{
						GenerateSkeletalBaker(Baker);
					}
					// If we have an attach track that attaches the object to somewhere else, then we'll need to bake in that transform
					// change, as we can't export "hierarchy changes" otherwise
					else if (Track->IsA<UMovieScene3DAttachTrack>())
					{
						const static FString PropertyPath = UnrealIdentifiers::TransformPropertyName.ToString();
						UnrealToUsd::CreateComponentPropertyBaker(Prim, *BoundComponent, PropertyPath, Baker);
					}
					else if (const UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(Track))
					{
						FMovieSceneSequenceID InstanceID = Context.Sequencer->GetRootTemplateID();
						if (InstancesOfThisSequence && InstancesOfThisSequence->Num() > 0)
						{
							InstanceID = (*InstancesOfThisSequence)[0];
						}

						FMovieSceneSequenceTransform SequenceTransform;
						if (const FMovieSceneSubSequenceData* SubSequenceData = HierarchyCache.FindSubData(InstanceID))
						{
							SequenceTransform = SubSequenceData->RootToSequenceTransform;
						}

						// This is awkwardly handled here within GenerateBakersForMovieScene (even though it doesn't generate a baker) for two
						// reasons:
						//  - It's the first place you'd go do in order to search for how audio is exported, since literally every other type
						//    of track we support goes through here
						//  - Getting the Prim to export the audio track *to* is very much non-trivial and requires looking into
						//    DynamicBindings and etc., which this function already does
						ExportAudioTrack(*AudioTrack, SequenceTransform, Prim, AudioTracksPerPrim);
					}

					AddBaker(Baker);

					if (Baker.BakerType == UnrealToUsd::EBakingType::Transform)
					{
						bHasTransformBaker = true;
					}
					else if (Baker.BakerType == UnrealToUsd::EBakingType::Skeletal)
					{
						bHasSkeletalBaker = true;
					}
				}
			}

			// If our component is attached to a specific socket of its parent, make sure its transform is baked.
			// This because if this parent has any AnimSequence animating it, BoundComponent's world transform may change
			// without it ever having an animated transform, and we don't have any form of rigging/socket attachment on USD that
			// would cause the parent prim's skeletal animation to also affect its child prims.
			// Ideally we'd actually search through the tracks to know for sure whether our parent has a SkeletalAnimation section,
			// but it's probably safer to just do this in case it is hidden behind N subsequences or some obscure feature
			if (!bHasTransformBaker)
			{
				// If any ancestor component has an attach socket, we may need to bake our transform
				bool bHasSocketAttachment = false;
				USceneComponent* Iterator = BoundComponent;
				while (Iterator)
				{
					if (Iterator->GetAttachSocketName() != NAME_None)
					{
						bHasSocketAttachment = true;
						break;
					}

					if (AActor* OwnerActor = Iterator->GetOwner())
					{
						if (OwnerActor->GetRootComponent() == Iterator)
						{
							// Don't climb out of the actor
							break;
						}
					}

					Iterator = Iterator->GetAttachParent();
				}

				if (bHasSocketAttachment)
				{
					UnrealToUsd::FComponentBaker Baker;

					const static FString PropertyPath = UnrealIdentifiers::TransformPropertyName.ToString();
					UnrealToUsd::CreateComponentPropertyBaker(Prim, *BoundComponent, PropertyPath, Baker);

					AddBaker(Baker);
				}
			}

			// There are many different ways in which SkeletalMeshComponents may animate their joints without having any Sequencer track
			// or even any binding, and this check here tries filling in that gap and generating a skeletal baker if needed.
			// (Search for bExportAnimationsFromAllComponents in this file to see how we can get in here without having a binding)
			if (!bHasSkeletalBaker)
			{
				if (USkeletalMeshComponent* SkeletalBoundComponent = Cast<USkeletalMeshComponent>(BoundComponent))
				{
					const bool bNeedsSkeletalBaker = SkeletalBoundComponent->HasValidAnimationInstance()
													 || SkeletalBoundComponent->LeaderPoseComponent.IsValid()
													 || (SkeletalBoundComponent->GetAnimationMode() == EAnimationMode::AnimationBlueprint
														 && SkeletalBoundComponent->AnimClass);

					if (bNeedsSkeletalBaker)
					{
						UnrealToUsd::FComponentBaker Baker;

						GenerateSkeletalBaker(Baker);

						AddBaker(Baker);
					}
				}
			}
		}

		// We always have to generate bakers for subsequences (even if we're not exporting separate files for subsequences) since we
		// will bake the full combined, composed level sequence as the main output USD layer. This because USD doesn't allow any form
		// of animation blending, and so composing individual USD layers exported for each subsequence with the same result as the sequencer
		// is impossible... we have to settle for having each USD layer represent the full effect of it's level sequence.
		// Also note that we can't share these bakers with our parent movie scenes in case we're a subsequence, unfortunately, because our
		// bakers will contain lambdas that write directly to a given prim, and those prims are specific to each layer that we're exporting
		// (e.g. our parent level sequence will export to different prims than this movie scene will)
		for (const UMovieSceneTrack* Track : MovieScene->GetTracks())
		{
			const UMovieSceneSubTrack* SubTrack = Cast<const UMovieSceneSubTrack>(Track);
			if (!SubTrack)
			{
				continue;
			}

			for (const UMovieSceneSection* Section : SubTrack->GetAllSections())
			{
				const UMovieSceneSubSection* SubSection = Cast<const UMovieSceneSubSection>(Section);
				if (!SubSection)
				{
					continue;
				}

				UMovieSceneSequence* SubSequence = SubSection->GetSequence();
				if (!SubSequence)
				{
					continue;
				}

				GenerateBakersForMovieScene(Context, *SubSequence, SequenceInstances, HierarchyCache, UsdStage, InOutComponentBakers);
			}
		}
	}

	// Steps through MovieSceneSequence frame by frame, invoking all baker lambdas every frame
	void BakeMovieSceneSequence(
		FLevelSequenceExportContext& Context,
		UMovieSceneSequence& MovieSceneSequence,
		UE::FUsdStage& UsdStage,
		const TMap<USceneComponent*, FCombinedComponentBakers>& ComponentBakers
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULevelSequenceExporterUsd::BakeMovieSceneSequence);

		UMovieScene* MovieScene = MovieSceneSequence.GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		// Hide all our spawns again so we can pretend they haven't actually spawned and let the sequence spawn them
		// as it plays
		Context.SpawnRegister->bDestroyingJustHides = true;
		Context.SpawnRegister->CleanUp(*Context.Sequencer);

		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		const FFrameRate Resolution = MovieScene->GetTickResolution();
		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		const double StageTimeCodesPerSecond = UsdStage.GetTimeCodesPerSecond();
		const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

		const FFrameTime Interval = FFrameRate::TransformTime(1, DisplayRate, Resolution);
		FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
		FFrameNumber EndFrame = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

		if (Context.ExportOptions && Context.ExportOptions->bOverrideExportRange)
		{
			StartFrame = FFrameRate::TransformTime(Context.ExportOptions->StartFrame, DisplayRate, Resolution).FloorToFrame();
			EndFrame = FFrameRate::TransformTime(Context.ExportOptions->EndFrame, DisplayRate, Resolution).CeilToFrame();
		}

		FFrameTime StartFrameUETime = FFrameRate::Snap(StartFrame, Resolution, DisplayRate).FloorToFrame();
		double StartTimeCode = FFrameRate::TransformTime(StartFrameUETime, Resolution, StageFrameRate).AsDecimal();
		UsdStage.SetStartTimeCode(StartTimeCode);

		FFrameTime EndFrameUETime = FFrameRate::Snap(EndFrame, Resolution, DisplayRate).FloorToFrame();
		double EndTimeCode = FFrameRate::TransformTime(EndFrameUETime, Resolution, StageFrameRate).AsDecimal();
		UsdStage.SetEndTimeCode(EndTimeCode);

		const FConstraintsManagerController& Controller = FConstraintsManagerController::Get(Context.World);

		// Collect and sort the bakers: We need all skeletal animations evaluated first, as we need to manually force
		// the component to update and attached components will only get the correct values if they are evaluated
		// after that
		TArray<UnrealToUsd::FComponentBaker> SortedBakers;
		SortedBakers.Reserve(ComponentBakers.Num() * UnrealToUsd::NumBakingTypes);
		for (const TPair<USceneComponent*, FCombinedComponentBakers>& Pair : ComponentBakers)
		{
			for (const UnrealToUsd::FComponentBaker& Baker : Pair.Value.Bakers)
			{
				if (Baker.BakerFunction)
				{
					SortedBakers.Add(Baker);
				}
			}
		}
		SortedBakers.Sort(
			[](const UnrealToUsd::FComponentBaker& LHS, const UnrealToUsd::FComponentBaker& RHS) -> bool
			{
				if (LHS.BakerType != RHS.BakerType)
				{
					if (LHS.BakerType == UnrealToUsd::EBakingType::Skeletal)
					{
						// We want all skeletal bakers first so that the joints are updated before we refresh transforms attached
						// to joints and sockets
						return true;
					}
					else if (RHS.BakerType == UnrealToUsd::EBakingType::Skeletal)
					{
						return false;
					}
				}

				// Parents should go first (so that if we have two nested skeletal mesh components attached to each other, we refresh the
				// parent first). To be honest this is likely not necessary as child joint transforms shouldn't depend on parent joint
				// transforms, but we should enforce some consistent ordering anyway so might as well use this
				return LHS.ComponentPath < RHS.ComponentPath;
			}
		);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PlaySequence);

			for (FFrameTime EvalTime = StartFrame; EvalTime <= EndFrame; EvalTime += Interval)
			{
				Context.Sequencer->SetLocalTimeDirectly(EvalTime);
				Context.Sequencer->ForceEvaluate();

				// Evaluate constraints (these run on tick in the editor, so here we must trigger them manually)
				// Can't iterate through a pre-sorted list since the parenting of the constraints can change between frames
				Controller.EvaluateAllConstraints();

				FFrameTime KeyTime = FFrameRate::Snap(EvalTime, Resolution, DisplayRate).FloorToFrame();
				double UsdTimeCode = FFrameRate::TransformTime(KeyTime, Resolution, StageFrameRate).AsDecimal();

				for (const UnrealToUsd::FComponentBaker& Baker : SortedBakers)
				{
					Baker.BakerFunction(UsdTimeCode);
				}
			}
		}

		// Convert all prims back to typeless overs (so that this layer doesn't define anything on a stage that doesn't
		// previously have it - it's only supposed to carry animation data)
		// We need to do this in this way because when going from 'def' to 'over' we need to do it from leaf towards the
		// root, as USD doesn't like a parent 'over' with a child 'def'
		TFunction<void(FUsdPrim&)> RecursiveSetOver;
		RecursiveSetOver = [&RecursiveSetOver](FUsdPrim& Prim)
		{
			for (FUsdPrim& Child : Prim.GetChildren())
			{
				RecursiveSetOver(Child);
			}

			// Retain SkelAnimations though: Those are actually authored directly on the LevelSequence layer
			if (!Prim.IsA("SkelAnimation"))
			{
				Prim.ClearTypeName();
				Prim.SetSpecifier(ESdfSpecifier::Over);
			}
		};
		FUsdPrim Root = UsdStage.GetPseudoRoot();
		for (FUsdPrim& TopLevelPrim : Root.GetChildren())
		{
			RecursiveSetOver(TopLevelPrim);
		}
	}

	void ExportMovieSceneSequence(
		FLevelSequenceExportContext& Context,
		UMovieSceneSequence& MovieSceneSequence,
		const FString& FilePath,
		TArray<UE::FUsdStage>& InOutExportedStages
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ULevelSequenceExporterUsd::ExportMovieSceneSequence);

		if (FilePath.IsEmpty() || Context.ExportedMovieScenes.Contains(&MovieSceneSequence))
		{
			return;
		}

		UMovieScene* MovieScene = MovieSceneSequence.GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		// Make sure we don't overwrite a file we just wrote *during this export*.
		// Overwriting other files is OK, as we want to allow a "repeatedly export over the same files" workflow
		FString UniqueFilePath = UsdUnreal::ObjectUtils::GetUniqueName(FilePath, Context.UsedFilePaths);

		// Try exporting subsequences if needed
		if (Context.ExportOptions)
		{
			if (Context.ExportOptions->bExportSubsequencesAsLayers)
			{
				FString Directory;
				FString FileName;
				FString Extension;
				FPaths::Split(UniqueFilePath, Directory, FileName, Extension);

				for (const UMovieSceneTrack* Track : MovieScene->GetTracks())
				{
					const UMovieSceneSubTrack* SubTrack = Cast<const UMovieSceneSubTrack>(Track);
					if (!SubTrack)
					{
						continue;
					}

					for (const UMovieSceneSection* Section : SubTrack->GetAllSections())
					{
						const UMovieSceneSubSection* SubSection = Cast<const UMovieSceneSubSection>(Section);
						if (!SubSection)
						{
							continue;
						}

						UMovieSceneSequence* SubSequence = SubSection->GetSequence();
						if (!SubSequence)
						{
							continue;
						}

						FString SubSequencePath = FPaths::Combine(Directory, FString::Printf(TEXT("%s.%s"), *SubSequence->GetName(), *Extension));

						ExportMovieSceneSequence(Context, *SubSequence, SubSequencePath, InOutExportedStages);

						// For now we don't want to actually add the subsequence layers as sublayers since each exported level sequence
						// contains the full baked result anyway, but this is how we'd do it:
						/*
						double Offset = 0.0;
						if ( SubSection->HasStartFrame() )
						{
							FFrameNumber LowerBound = SubSection->GetTrueRange().GetLowerBoundValue();
							FFrameTime LowerBoundTime{ FFrameRate::Snap( LowerBound, Resolution, DisplayRate ).FloorToFrame() };
							Offset = FFrameRate::TransformTime( LowerBoundTime, Resolution, StageFrameRate ).AsDecimal();
						}
						float Scale = SubSection->Parameters.TimeScale;
						const int32 Index = -1;
						UsdUtils::InsertSubLayer( UsdStage.GetRootLayer(), *SubSequencePath, Index, Offset, Scale );
						*/
					}
				}
			}
		}

		if (!IUsdExporterModule::CanExportToLayer(UniqueFilePath))
		{
			return;
		}

		FString LevelSequenceVersion = MovieSceneSequence.GetSignature().ToString();
		{
			// We could just use the GUID directly but all other asset types end up with SHA hash size so lets be
			// consistent
			FSHA1 SHA1;
			SHA1.UpdateWithString(*LevelSequenceVersion, LevelSequenceVersion.Len());
			if (Context.ExportOptions)
			{
				UsdUtils::HashForLevelSequenceExport(*Context.ExportOptions, SHA1);
			}
			SHA1.Final();
			FSHAHash Hash;
			SHA1.GetHash(&Hash.Hash[0]);
			LevelSequenceVersion = Hash.ToString();
		}

		// Check if we already have exported what we plan on exporting anyway
		if (FPaths::FileExists(UniqueFilePath))
		{
			if (!Context.bReplaceIdentical)
			{
				USD_LOG_USERINFO(FText::Format(
					LOCTEXT("FileAlreadyExists", "Skipping export of asset '{0}' as the target file '{1}' already exists."),
					FText::FromString(MovieSceneSequence.GetPathName()),
					FText::FromString(UExporter::CurrentFilename)
				));
				return;
			}
			// If we don't want to re-export this level sequence we need to check if its the same version
			else if (!Context.ExportOptions->bReExportIdenticalLevelsAndSequences)
			{
				// Don't use the stage cache here as we want this stage to close within this scope in case
				// we have to overwrite its files due to e.g. missing payload or anything like that
				const bool bUseStageCache = false;
				const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadNone;
				if (UE::FUsdStage TempStage = UnrealUSDWrapper::OpenStage(*UniqueFilePath, InitialLoadSet, bUseStageCache))
				{
					if (UE::FUsdPrim RootPrim = TempStage.GetDefaultPrim())
					{
						FUsdUnrealAssetInfo Info = UsdUtils::GetPrimAssetInfo(RootPrim);

						const bool bVersionMatches = !Info.Version.IsEmpty() && Info.Version == LevelSequenceVersion;

						const bool bAssetTypeMatches = !Info.UnrealAssetType.IsEmpty()
													   && Info.UnrealAssetType == MovieSceneSequence.GetClass()->GetName();

						if (bVersionMatches && bAssetTypeMatches)
						{
							USD_LOG_USERINFO(FText::Format(
								LOCTEXT(
									"FileUpToDate",
									"Skipping export of asset '{0}' as the target file '{1}' already contains up-to-date exported data."
								),
								FText::FromString(MovieSceneSequence.GetPathName()),
								FText::FromString(UExporter::CurrentFilename)
							));
							return;
						}
					}
				}
			}
		}

		UE::FUsdStage UsdStage = UnrealUSDWrapper::NewStage(*UniqueFilePath);
		if (!UsdStage)
		{
			return;
		}

		if (Context.ExportOptions)
		{
			UsdUtils::SetUsdStageMetersPerUnit(UsdStage, Context.ExportOptions->StageOptions.MetersPerUnit);
			UsdUtils::SetUsdStageUpAxis(UsdStage, Context.ExportOptions->StageOptions.UpAxis);
			UsdStage.SetTimeCodesPerSecond(Context.ExportOptions->TimeCodesPerSecond);
		}

		// Set this so that if we open this exported sequence back up in UE the Sequencer will start up showing
		// the same DisplayRate as the original exported sequence
		UsdStage.SetFramesPerSecond(MovieScene->GetDisplayRate().AsDecimal());

		const FString RootPrimPath = TEXT("/") + UsdUtils::SanitizeUsdIdentifier(*Context.ExportOptions->LevelExportOptions.RootPrimName);
		UE::FUsdPrim RootPrim = UsdStage.OverridePrim(UE::FSdfPath(*RootPrimPath));
		if (!RootPrim)
		{
			return;
		}

		UsdStage.SetDefaultPrim(RootPrim);

		// Currently we bake the full composed result of each level sequence into a single layer,
		// because USD can't compose individual layers in the same way (with blending and so on). So here
		// we make sure that our sequencer exports each MovieSceneSequence as if it was a root, emitting the
		// same result as if we had exported that subsequence's LevelSequence by itself
		Context.Sequencer->ResetToNewRootSequence(MovieSceneSequence);

		FMovieSceneSequenceHierarchy HierarchyCache;
		TMap<UMovieSceneSequence*, TArray<FMovieSceneSequenceID>> SequenceInstances = GetSequenceHierarchyInstances(
			MovieSceneSequence,
			Context.Sequencer.Get(),
			HierarchyCache
		);

		TMap<USceneComponent*, FCombinedComponentBakers> Bakers;
		GenerateBakersForMovieScene(Context, MovieSceneSequence, SequenceInstances, HierarchyCache, UsdStage, Bakers);

		// Bake this MovieScene
		// We bake each MovieScene individually instead of doing one large simultaneous bake because this way
		// not only we avoid having to handle FMovieSceneSequenceTransforms when writing out the UsdTimeCodes,
		// we guarantee we'll get the same result as if we exported each subsequence individually.
		// They could have differed, for example, if we had a limited the playback range of a subsequence
		BakeMovieSceneSequence(Context, MovieSceneSequence, UsdStage, Bakers);

		// We can add the level as a sublayer to every exported subsequence, so that each can be opened individually and
		// automatically load the level layer. It doesn't matter much if the parent stage has composed the level
		// sublayer multiple times (in case we add subsequence layers as sublayers in the future), as the prims will
		// just all override each other with the same data
		if (Context.ExportOptions && Context.ExportOptions->bUseExportedLevelAsSublayer && FPaths::FileExists(Context.LevelFilePath))
		{
			UsdUtils::InsertSubLayer(UsdStage.GetRootLayer(), *Context.LevelFilePath);
		}

		if (UE::FUsdPrim AssetDefaultPrim = UsdStage.GetDefaultPrim())
		{
			if (Context.ExportOptions->LevelExportOptions.MetadataOptions.bExportAssetInfo)
			{
				FUsdUnrealAssetInfo Info;
				Info.Name = MovieSceneSequence.GetName();
				Info.Identifier = UniqueFilePath;
				Info.Version = LevelSequenceVersion;
				Info.UnrealContentPath = MovieSceneSequence.GetPathName();
				Info.UnrealAssetType = MovieSceneSequence.GetClass()->GetName();
				Info.UnrealExportTime = FDateTime::Now().ToString();
				Info.UnrealEngineVersion = FEngineVersion::Current().ToString();

				UsdUtils::SetPrimAssetInfo(AssetDefaultPrim, Info);
			}

			if (Context.ExportOptions->LevelExportOptions.MetadataOptions.bExportAssetMetadata)
			{
				if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(Cast<ULevelSequence>(&MovieSceneSequence)))
				{
					UnrealToUsd::ConvertMetadata(
						UserData,
						AssetDefaultPrim,
						Context.ExportOptions->LevelExportOptions.MetadataOptions.BlockedPrefixFilters,
						Context.ExportOptions->LevelExportOptions.MetadataOptions.bInvertFilters
					);
				}
			}
		}

		Context.ExportedMovieScenes.Add(&MovieSceneSequence, UniqueFilePath);
		Context.UsedFilePaths.Add(UniqueFilePath);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Save);
			UsdStage.GetRootLayer().Save();
		}

		InOutExportedStages.Add(UsdStage);
	}
#endif	  // USE_USD_SDK
}	 // namespace UE::LevelSequenceExporterUSD::Private

ULevelSequenceExporterUsd::ULevelSequenceExporterUsd()
{
#if USE_USD_SDK
	UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(FormatExtension, FormatDescription);
	SupportedClass = ULevelSequence::StaticClass();
	bText = false;
#endif	  // #if USE_USD_SDK
}

bool ULevelSequenceExporterUsd::ExportBinary(
	UObject* Object,
	const TCHAR* Type,
	FArchive& Ar,
	FFeedbackContext* Warn,
	int32 FileIndex,
	uint32 PortFlags
)
{
	namespace LevelSequenceExporterImpl = UE::LevelSequenceExporterUSD::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelSequenceExporterUsd::ExportBinary);

#if USE_USD_SDK
	ULevelSequence* LevelSequence = Cast<ULevelSequence>(Object);
	if (!GEditor || !GIsEditor || !LevelSequence)
	{
		return false;
	}

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	ULevelSequenceExporterUsdOptions* Options = nullptr;
	if (ExportTask)
	{
		Options = Cast<ULevelSequenceExporterUsdOptions>(ExportTask->Options);
	}
	if (!Options)
	{
		Options = GetMutableDefault<ULevelSequenceExporterUsdOptions>();

		// Prefill the level to export with the current level
		if (!Options->Level.Get())
		{
			Options->Level = IUsdClassesModule::GetCurrentWorld();
		}

		// Prompt with an options dialog if we can
		if (Options && (!ExportTask || !ExportTask->bAutomated))
		{
			Options->LevelExportOptions.AssetFolder.Path = FPaths::Combine(FPaths::GetPath(UExporter::CurrentFilename), TEXT("Assets"));
			Options->TimeCodesPerSecond = MovieScene->GetDisplayRate().AsDecimal();

			const bool bContinue = SUsdOptionsWindow::ShowExportOptions(*Options);
			if (!bContinue)
			{
				return false;
			}
		}
	}
	if (!Options)
	{
		return false;
	}

	// See comment on the analogous line within StaticMeshExporterUSD.cpp
	ExportTask->bPrompt = false;

	double StartTime = FPlatformTime::Cycles64();

	TSharedPtr<LevelSequenceExporterImpl::FLevelSequenceHidingSpawnRegister>
		SpawnRegister = MakeShared<LevelSequenceExporterImpl::FLevelSequenceHidingSpawnRegister>();
	if (!SpawnRegister.IsValid())
	{
		return false;
	}

	UsdUnreal::ExportUtils::FUniquePathScope UniquePathScope;
	FScopedUsdMessageLog UsdMessageLog;

	FSequencerInitParams Params;
	Params.RootSequence = LevelSequence;
	Params.SpawnRegister = SpawnRegister;
	Params.ViewParams.bReadOnly = true;
	Params.bEditWithinLevelEditor = false;

	// UE-132538: Use a getter for the playback context instead of just binding the world directly because
	// FSequencer::UpdateCachedPlaybackContextAndClient will ignore the attribute's value and only check the getter
	UWorld* World = Options->Level.Get();
	if (!World)
	{
		const bool bEditorWorldsOnly = true;
		World = IUsdClassesModule::GetCurrentWorld(bEditorWorldsOnly);
	}
	Params.PlaybackContext = TAttribute<UObject*>::Create(TAttribute<UObject*>::FGetter::CreateLambda(
		[World]()
		{
			return World;
		}
	));

	// Set to read only or else CreateSequencer will change the playback range of the moviescene without even calling Modify()
	const bool bOldReadOnly = MovieScene->IsReadOnly();
	const bool bNewReadOnly = true;
	MovieScene->SetReadOnly(bNewReadOnly);
	TSharedPtr<ISequencer> TempSequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(Params);
	MovieScene->SetReadOnly(bOldReadOnly);
	if (!TempSequencer.IsValid())
	{
		return false;
	}

	TempSequencer->EnterSilentMode();
	TempSequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);

	SpawnRegister->SetSequencer(TempSequencer);
	SpawnRegister->bExportSeparatePrimsPerSpawnableInstance = Options->bExportSeparatePrimsPerSpawnableInstance;

	LevelSequenceExporterImpl::FLevelSequenceExportContext Context{*LevelSequence, TempSequencer.ToSharedRef(), SpawnRegister.ToSharedRef()};
	Context.ExportOptions = Options;
	Context.bReplaceIdentical = ExportTask->bReplaceIdentical;
	Context.World = World;

	// Spawn (but hide) all spawnables so that they will also show up on the level export if we need them to.
	// We have to traverse the template IDs when spawning spawnables, because we'll want to force each individual spawnable of each
	// FMovieSceneSequenceID to spawn a separate object, so that they can become separate prims. Without doing this, if we used the same
	// subsequence with spawnables multiple times within a parent sequence we'd only get one prim out, as the spawnable bindings
	// would be the exact same between all instances of the child sequence (same FGuid)
	LevelSequenceExporterImpl::PreSpawnSpawnables(Context, *LevelSequence);

	TArray<AActor*> OriginalSelectedSpawnableActors;
	TArray<AActor*> OriginalSelectedNonSpawnableActors;
	TArray<UE::MovieScene::FFixedObjectBindingID> OriginalSelectedSpawnableBindings;
	LevelSequenceExporterImpl::CollectEditorSelection(
		*SpawnRegister,
		OriginalSelectedSpawnableActors,
		OriginalSelectedNonSpawnableActors,
		OriginalSelectedSpawnableBindings
	);

	if (Options->bSelectionOnly || Options->LevelExportOptions.bSelectionOnly)
	{
		// If the user has any spawnable selected on their Sequencer, we need to transfer that selection to our temp sequencer's spawnables
		TArray<AActor*> TransferredSelectedSpawnableActors;
		LevelSequenceExporterImpl::TransferSpawnableSelection(*TempSequencer, OriginalSelectedSpawnableBindings, TransferredSelectedSpawnableActors);

		if (Options->bSelectionOnly)
		{
			Context.SelectedActors.Reset();
			Context.SelectedActors.Append(OriginalSelectedNonSpawnableActors);
			Context.SelectedActors.Append(TransferredSelectedSpawnableActors);
		}
	}

	// Capture this first because when we launch UExporter::RunAssetExportTask the CurrentFileName will change
	const FString TargetFileName = UExporter::CurrentFilename;

	// Always close all opened sequencers since it doesn't look like we're supposed to have more than one opened at a time.
	// Without this, the ResetToNewRootSequence call may actually evaluate our opened subsequence at the playhead position of
	// *other* sequencers, for whatever reason. We also can't call RestorePreAnimatedState on the sequence we're exporting
	// after that either, as it hasn't stored anything yet.
	// Additionally, it seems the sequencer will also attempt to interpolate between all opened sequencers when evaluating a track,
	// which could affect the sequence we're exporting.
	// Closing the Sequencers before exporting the Level is important in order to despawn their spawnables. We'll spawn our own,
	// so we would have ended up with duplicates otherwise
	TArray<UObject*> AssetsToReopenEditorsFor;
	UAssetEditorSubsystem* AssetEditorSubsystem = nullptr;
	{
		AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (AssetEditorSubsystem && !IsEngineExitRequested())
		{
			for (const TWeakPtr<ISequencer>& Sequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
			{
				if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
				{
					if (PinnedSequencer == TempSequencer)
					{
						continue;
					}

					ULevelSequence* OpenedSequence = Cast<ULevelSequence>(PinnedSequencer->GetRootMovieSceneSequence());
					if (!OpenedSequence)
					{
						continue;
					}

					AssetsToReopenEditorsFor.Add(OpenedSequence);
					AssetEditorSubsystem->CloseAllEditorsForAsset(OpenedSequence);
				}
			}
		}
	}

	TArray<FAnalyticsEventAttribute> AnalyticsAttributes;

	// Export level if we need to
	if (Options && Options->bExportLevel)
	{
		if (UWorld* WorldToExport = Options->Level.Get())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ULevelSequenceExporterUsd::LevelExport);

			// Come up with a file path for the level
			FString Directory;
			FString Filename;
			FString Extension;
			FPaths::Split(TargetFileName, Directory, Filename, Extension);
			Context.LevelFilePath = FPaths::Combine(Directory, FString::Printf(TEXT("%s.%s"), *WorldToExport->GetFName().ToString(), *Extension));

			Context.UsedFilePaths.Add(Context.LevelFilePath);

			ULevelExporterUSDOptions* LevelOptions = GetMutableDefault<ULevelExporterUSDOptions>();
			LevelOptions->StageOptions = Options->StageOptions;
			LevelOptions->Inner = Options->LevelExportOptions;
			LevelOptions->bReExportIdenticalAssets = Options->bReExportIdenticalAssets;
			LevelOptions->bReExportIdenticalLevelsAndSequences = Options->bReExportIdenticalLevelsAndSequences;

			UAssetExportTask* LevelExportTask = NewObject<UAssetExportTask>();
			FGCObjectScopeGuard ExportTaskGuard(LevelExportTask);
			LevelExportTask->Object = WorldToExport;
			LevelExportTask->Options = LevelOptions;
			LevelExportTask->Exporter = nullptr;
			LevelExportTask->Filename = Context.LevelFilePath;
			LevelExportTask->bSelected = LevelOptions->Inner.bSelectionOnly;	// Move this as the level exporter will favor bSelected
			LevelExportTask->bReplaceIdentical = ExportTask->bReplaceIdentical;
			LevelExportTask->bPrompt = false;
			LevelExportTask->bUseFileArchive = false;
			LevelExportTask->bWriteEmptyFiles = false;
			LevelExportTask->bAutomated = true;	   // Pretend this is an automated task so it doesn't pop the options dialog

			// We don't want the level export to send a separate event
			IUsdClassesModule::BlockAnalyticsEvents();
			{
				UExporter::RunAssetExportTask(LevelExportTask);
			}
			IUsdClassesModule::ResumeAnalyticsEvents();

			// Grab the analytics that the level exporter produced, because that also includes some asset counts that would be
			// awkward to collect from here
			if (const TArray<FAnalyticsEventAttribute>* LevelAnalytics = IUsdClassesModule::GetAccumulatedAnalytics().Find(TEXT("Export.World")))
			{
				AnalyticsAttributes = *LevelAnalytics;
			}
		}
	}

	TArray<UE::FUsdStage> ExportedStages;
	LevelSequenceExporterImpl::ExportMovieSceneSequence(Context, *LevelSequence, TargetFileName, ExportedStages);

	// Set this back to Stopped or else it will keep the editor viewport controls permanently hidden
	TempSequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
	TempSequencer->Close();

	if (AssetEditorSubsystem)
	{
		AssetEditorSubsystem->OpenEditorForAssets(AssetsToReopenEditorsFor);
	}

	LevelSequenceExporterImpl::RestoreEditorSelection(OriginalSelectedNonSpawnableActors, OriginalSelectedSpawnableBindings);

	// Analytics
	{
		bool bAutomated = ExportTask ? ExportTask->bAutomated : false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(UExporter::CurrentFilename);

		UE::LevelSequenceExporterUSD::Private::SendAnalytics(
			LevelSequence,
			Options,
			ExportedStages,
			bAutomated,
			ElapsedSeconds,
			Extension,
			AnalyticsAttributes
		);
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
