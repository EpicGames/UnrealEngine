// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceExporterUSDOptions.h"

#include "LevelExporterUSDOptions.h"
#include "USDExporterModule.h"

#include "AnalyticsEventAttribute.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceExporterUSDOptions)

void UsdUtils::AddAnalyticsAttributes(const ULevelSequenceExporterUsdOptions& Options, TArray<struct FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("TimeCodesPerSecond"), LexToString(Options.TimeCodesPerSecond));
	InOutAttributes.Emplace(TEXT("OverrideExportRange"), Options.bOverrideExportRange);
	if (Options.bOverrideExportRange)
	{
		InOutAttributes.Emplace(TEXT("StartFrame"), LexToString(Options.StartFrame));
		InOutAttributes.Emplace(TEXT("EndFrame"), LexToString(Options.EndFrame));
	}
	InOutAttributes.Emplace(TEXT("LevelSequenceSelectionOnly"), Options.bSelectionOnly);
	InOutAttributes.Emplace(TEXT("ExportSubsequencesAsLayers"), Options.bExportSubsequencesAsLayers);
	InOutAttributes.Emplace(TEXT("ExportLevel"), Options.bExportLevel);
	if (Options.bExportLevel)
	{
		InOutAttributes.Emplace(TEXT("UseExportedLevelAsSublayer"), Options.bUseExportedLevelAsSublayer);
	}
	InOutAttributes.Emplace(TEXT("ExportSeparatePrimsPerSpawnableInstance"), Options.bExportSeparatePrimsPerSpawnableInstance);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalLevelsAndSequences"), Options.bReExportIdenticalLevelsAndSequences);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
	if (Options.bExportLevel)
	{
		UsdUtils::AddAnalyticsAttributes(Options.LevelExportOptions, InOutAttributes);
	}
}

void UsdUtils::HashForLevelSequenceExport(const ULevelSequenceExporterUsdOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForExport(Options.StageOptions, HashToUpdate);
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.TimeCodesPerSecond), sizeof(Options.TimeCodesPerSecond));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bOverrideExportRange), sizeof(Options.bOverrideExportRange));
	if (Options.bOverrideExportRange)
	{
		HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.StartFrame), sizeof(Options.StartFrame));
		HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.EndFrame), sizeof(Options.EndFrame));
	}

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bSelectionOnly), sizeof(Options.bSelectionOnly));
	if (Options.bSelectionOnly)
	{
		IUsdExporterModule::HashEditorSelection(HashToUpdate);
	}

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportSubsequencesAsLayers), sizeof(Options.bExportSubsequencesAsLayers));

	const bool bUsingLevelSublayer = Options.bExportLevel && Options.bUseExportedLevelAsSublayer;
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&bUsingLevelSublayer), sizeof(bUsingLevelSublayer));

	HashToUpdate.Update(
		reinterpret_cast<const uint8*>(&Options.bExportSeparatePrimsPerSpawnableInstance),
		sizeof(Options.bExportSeparatePrimsPerSpawnableInstance)
	);

	static IConsoleVariable* Cvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.LevelSequenceExport.SkipConstantValues"));
	const bool bSkipConstantValues = Cvar && Cvar->GetBool();
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&bSkipConstantValues), sizeof(bSkipConstantValues));
}
