// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExporterUSD.h"

#include "MaterialExporterUSDOptions.h"
#include "UnrealUSDWrapper.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDExporterModule.h"
#include "USDExportUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDMetadataExportOptions.h"
#include "USDObjectUtils.h"
#include "USDOptionsWindow.h"
#include "USDPrimConversion.h"
#include "USDShadeConversion.h"
#include "USDUnrealAssetInfo.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "AssetExportTask.h"
#include "Engine/Font.h"
#include "Engine/Texture.h"
#include "EngineAnalytics.h"
#include "MaterialOptions.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "Misc/EngineVersion.h"
#include "Misc/Paths.h"
#include "UObject/UObjectGlobals.h"
#include "VT/RuntimeVirtualTexture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExporterUSD)

#define LOCTEXT_NAMESPACE "MaterialExporterUSD"

namespace UE::MaterialExporterUSD::Private
{
	void SendAnalytics(
		const UMaterialInterface& Material,
		const FUsdMaterialBakingOptions& Options,
		bool bAutomated,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension
	)
	{
		if (!FEngineAnalytics::IsAvailable())
		{
			return;
		}

		const FString ClassName = IUsdClassesModule::GetClassNameForAnalytics(&Material);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Emplace(TEXT("AssetType"), ClassName);

		UsdUtils::AddAnalyticsAttributes(Options, EventAttributes);

		IUsdClassesModule::SendAnalytics(
			MoveTemp(EventAttributes),
			FString::Printf(TEXT("Export.%s"), *ClassName),
			bAutomated,
			ElapsedSeconds,
			NumberOfFrames,
			Extension
		);
	}

	void HashMaterial(const UMaterialInterface& Material, FSHA1& VersionHash)
	{
		if (const UMaterial* Parent = Material.GetMaterial())
		{
			// Add StateId to the combined hash, which is updated on every compile
			VersionHash.Update(reinterpret_cast<const uint8*>(&Parent->StateId[0]), sizeof(FGuid));

			// Add material hash to the combined hash, which is updated when switching some material settings
			if (const FMaterialResource* MaterialResource = Parent->GetMaterialResource(GMaxRHIShaderPlatform))
			{
				MaterialResource = Parent->GetMaterialResource(GMaxRHIShaderPlatform);

				FMaterialShaderMapId ShaderMapID;
				MaterialResource->BuildShaderMapId(ShaderMapID, nullptr);

				uint64 MaterialHash = ShaderMapID.GetMaterialHash();
				VersionHash.Update(reinterpret_cast<const uint8*>(&MaterialHash), sizeof(MaterialHash));
			}
			else
			{
				// We should have resources by now, but if we don't just add some random value to the hash so that
				// we never consider this material similar to the one we previously exported, since we can't guarantee
				// that at all without looking at its shader map ID hash
				FGuid NewGuid = FGuid::NewGuid();
				VersionHash.Update(reinterpret_cast<const uint8*>(&NewGuid[0]), sizeof(FGuid));
			}
		}

		// Add material override parameters to the hash
		if (const UMaterialInstanceConstant* InstanceConstant = Cast<const UMaterialInstanceConstant>(&Material))
		{
			// MIC already have a nice GUID we can use
			VersionHash.Update(reinterpret_cast<const uint8*>(&InstanceConstant->ParameterStateId[0]), sizeof(FGuid));
		}
		else if (const UMaterialInstance* Instance = Cast<const UMaterialInstance>(&Material))
		{
			// Manually hash everything...
			for (const FScalarParameterValue& Parameter : Instance->ScalarParameterValues)
			{
				uint32 NameHash = GetTypeHash(Parameter.ParameterInfo.Name);
				VersionHash.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(uint32));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Association), sizeof(Parameter.ParameterInfo.Association));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Index), sizeof(Parameter.ParameterInfo.Index));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterValue), sizeof(Parameter.ParameterValue));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ExpressionGUID), sizeof(Parameter.ExpressionGUID));
			}
			for (const FVectorParameterValue& Parameter : Instance->VectorParameterValues)
			{
				uint32 NameHash = GetTypeHash(Parameter.ParameterInfo.Name);
				VersionHash.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(uint32));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Association), sizeof(Parameter.ParameterInfo.Association));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Index), sizeof(Parameter.ParameterInfo.Index));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterValue), sizeof(Parameter.ParameterValue));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ExpressionGUID), sizeof(Parameter.ExpressionGUID));
			}
			for (const FDoubleVectorParameterValue& Parameter : Instance->DoubleVectorParameterValues)
			{
				uint32 NameHash = GetTypeHash(Parameter.ParameterInfo.Name);
				VersionHash.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(uint32));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Association), sizeof(Parameter.ParameterInfo.Association));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Index), sizeof(Parameter.ParameterInfo.Index));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterValue), sizeof(Parameter.ParameterValue));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ExpressionGUID), sizeof(Parameter.ExpressionGUID));
			}
			for (const FTextureParameterValue& Parameter : Instance->TextureParameterValues)
			{
				uint32 NameHash = GetTypeHash(Parameter.ParameterInfo.Name);
				VersionHash.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(uint32));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Association), sizeof(Parameter.ParameterInfo.Association));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Index), sizeof(Parameter.ParameterInfo.Index));

				FString TexturePath = Parameter.ParameterValue ? Parameter.ParameterValue->GetPathName() : FString{};
				VersionHash.UpdateWithString(*TexturePath, TexturePath.Len());
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ExpressionGUID), sizeof(Parameter.ExpressionGUID));
			}
			for (const FRuntimeVirtualTextureParameterValue& Parameter : Instance->RuntimeVirtualTextureParameterValues)
			{
				uint32 NameHash = GetTypeHash(Parameter.ParameterInfo.Name);
				VersionHash.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(uint32));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Association), sizeof(Parameter.ParameterInfo.Association));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Index), sizeof(Parameter.ParameterInfo.Index));

				FString TexturePath = Parameter.ParameterValue ? Parameter.ParameterValue->GetPathName() : FString{};
				VersionHash.UpdateWithString(*TexturePath, TexturePath.Len());
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ExpressionGUID), sizeof(Parameter.ExpressionGUID));
			}
			for (const FFontParameterValue& Parameter : Instance->FontParameterValues)
			{
				uint32 NameHash = GetTypeHash(Parameter.ParameterInfo.Name);
				VersionHash.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(uint32));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Association), sizeof(Parameter.ParameterInfo.Association));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ParameterInfo.Index), sizeof(Parameter.ParameterInfo.Index));

				FString FontPath = Parameter.FontValue ? Parameter.FontValue->GetPathName() : FString{};
				VersionHash.UpdateWithString(*FontPath, FontPath.Len());
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.FontPage), sizeof(Parameter.FontPage));
				VersionHash.Update(reinterpret_cast<const uint8*>(&Parameter.ExpressionGUID), sizeof(Parameter.ExpressionGUID));
			}
		}
	}

	void HashOptions(const FUsdMaterialBakingOptions& Options, FSHA1& VersionHash)
	{
		VersionHash.Update(reinterpret_cast<const uint8*>(&Options.DefaultTextureSize), sizeof(Options.DefaultTextureSize));

		VersionHash.Update(reinterpret_cast<const uint8*>(&Options.bConstantColorAsSingleValue), sizeof(Options.bConstantColorAsSingleValue));

		VersionHash.Update(reinterpret_cast<const uint8*>(Options.Properties.GetData()), Options.Properties.Num() * Options.Properties.GetTypeSize());

		VersionHash.UpdateWithString(*Options.TexturesDir.Path, Options.TexturesDir.Path.Len());
	}
}	 // namespace UE::MaterialExporterUSD::Private

UMaterialExporterUsd::UMaterialExporterUsd()
{
#if USE_USD_SDK
	UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(FormatExtension, FormatDescription);
	SupportedClass = UMaterialInterface::StaticClass();
	bText = false;
#endif	  // #if USE_USD_SDK
}

bool UMaterialExporterUsd::ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex, uint32 PortFlags)
{
#if USE_USD_SDK
	UMaterialInterface* Material = Cast<UMaterialInterface>(Object);
	if (!Material)
	{
		return false;
	}

	UMaterialExporterUSDOptions* Options = nullptr;
	if (ExportTask)
	{
		Options = Cast<UMaterialExporterUSDOptions>(ExportTask->Options);
	}
	if (!Options)
	{
		Options = GetMutableDefault<UMaterialExporterUSDOptions>();

		// Prompt with an options dialog if we can
		if (Options && (!ExportTask || !ExportTask->bAutomated))
		{
			Options->MaterialBakingOptions.TexturesDir.Path = FPaths::Combine(FPaths::GetPath(UExporter::CurrentFilename), TEXT("Textures"));

			const bool bContinue = SUsdOptionsWindow::ShowExportOptions(*Options);
			if (!bContinue)
			{
				return false;
			}
		}
	}
	if (!Options)
	{
		return false;
	}

	// See comment on the analogous line within StaticMeshExporterUSD.cpp
	ExportTask->bPrompt = false;

	// Make sure the material has a resource as we'll use it to fetch some version GUIDs and ExportMaterial
	// can't do it directly due to receiving the material by const ref
	if (UMaterial* Parent = Material->GetMaterial())
	{
		const FMaterialResource* MaterialResource = Parent->GetMaterialResource(GMaxRHIShaderPlatform);
		if (MaterialResource == nullptr)
		{
			Parent->ForceRecompileForRendering();
		}
	}

	return UMaterialExporterUsd::ExportMaterial(
		*Material,
		Options->MaterialBakingOptions,
		Options->MetadataOptions,
		FFilePath{UExporter::CurrentFilename},
		ExportTask->bReplaceIdentical,
		Options->bReExportIdenticalAssets,
		ExportTask->bAutomated
	);
#else
	return false;
#endif	  // #if USE_USD_SDK
}

bool UMaterialExporterUsd::ExportMaterial(
	const UMaterialInterface& Material,
	const FUsdMaterialBakingOptions& Options,
	const FUsdMetadataExportOptions& MetadataOptions,
	const FFilePath& FilePath,
	bool bReplaceIdentical,
	bool bReExportIdenticalAssets,
	bool bIsAutomated
)
{
#if USE_USD_SDK

	FSHAHash Hash;
	FSHA1 SHA1;
	UE::MaterialExporterUSD::Private::HashMaterial(Material, SHA1);
	UE::MaterialExporterUSD::Private::HashOptions(Options, SHA1);
	SHA1.Final();
	SHA1.GetHash(&Hash.Hash[0]);
	FString MaterialHashString = Hash.ToString();

	if (!IUsdExporterModule::CanExportToLayer(FilePath.FilePath))
	{
		return false;
	}

	UsdUnreal::ExportUtils::FUniquePathScope UniquePathScope;

	// Check if we already have exported what we plan on exporting anyway
	if (FPaths::FileExists(FilePath.FilePath))
	{
		if (!bReplaceIdentical)
		{
			USD_LOG_USERINFO(FText::Format(
				LOCTEXT("FileAlreadyExists", "Skipping export of asset '{0}' as the target file '{1}' already exists."),
				FText::FromString(Material.GetPathName()),
				FText::FromString(UExporter::CurrentFilename)
			));
			return false;
		}
		// If we don't want to re-export this asset we need to check if its the same version
		else if (!bReExportIdenticalAssets)
		{
			// Don't use the asset cache here as we want this stage to close within this scope in case
			// we have to overwrite its files due to e.g. missing payload or anything like that
			const bool bUseAssetCache = false;
			const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadNone;
			if (UE::FUsdStage TempStage = UnrealUSDWrapper::OpenStage(*FilePath.FilePath, InitialLoadSet, bUseAssetCache))
			{
				if (UE::FUsdPrim DefaultPrim = TempStage.GetDefaultPrim())
				{
					FUsdUnrealAssetInfo Info = UsdUtils::GetPrimAssetInfo(DefaultPrim);

					const bool bVersionMatches = !Info.Version.IsEmpty() && Info.Version == MaterialHashString;

					const bool bAssetTypeMatches = !Info.UnrealAssetType.IsEmpty() && Info.UnrealAssetType == Material.GetClass()->GetName();

					if (bVersionMatches && bAssetTypeMatches)
					{
						USD_LOG_USERINFO(FText::Format(
							LOCTEXT(
								"FileUpToDate",
								"Skipping export of asset '{0}' as the target file '{1}' already contains up-to-date exported data."
							),
							FText::FromString(Material.GetPathName()),
							FText::FromString(UExporter::CurrentFilename)
						));

						return true;
					}
				}
			}
		}
	}

	double StartTime = FPlatformTime::Cycles64();

	UE::FUsdStage UsdStage = UnrealUSDWrapper::NewStage(*FilePath.FilePath);
	if (!UsdStage)
	{
		return false;
	}

	FString RootPrimPath = TEXT("/") + UsdUtils::SanitizeUsdIdentifier(*Material.GetName());

	UE::FUsdPrim RootPrim = UsdStage.DefinePrim(UE::FSdfPath(*RootPrimPath), TEXT("Material"));
	if (!RootPrim)
	{
		return false;
	}

	UsdStage.SetDefaultPrim(RootPrim);

	UsdUtils::SetUnrealSurfaceOutput(RootPrim, Material.GetPathName());

	UnrealToUsd::ConvertMaterialToBakedSurface(
		Material,
		Options.Properties,
		Options.DefaultTextureSize,
		Options.TexturesDir,
		RootPrim,
		Options.bConstantColorAsSingleValue
	);

	if (MetadataOptions.bExportAssetInfo)
	{
		FUsdUnrealAssetInfo Info;
		Info.Name = Material.GetName();
		Info.Identifier = UExporter::CurrentFilename;
		Info.Version = MaterialHashString;
		Info.UnrealContentPath = Material.GetPathName();
		Info.UnrealAssetType = Material.GetClass()->GetName();
		Info.UnrealExportTime = FDateTime::Now().ToString();
		Info.UnrealEngineVersion = FEngineVersion::Current().ToString();

		UsdUtils::SetPrimAssetInfo(RootPrim, Info);
	}

	if (MetadataOptions.bExportAssetMetadata)
	{
		if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(&Material))
		{
			UnrealToUsd::ConvertMetadata(UserData, RootPrim, MetadataOptions.BlockedPrefixFilters, MetadataOptions.bInvertFilters);
		}
	}

	UsdStage.GetRootLayer().Save();

	// Analytics
	{
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(FilePath.FilePath);
		double NumberOfFrames = 1 + UsdStage.GetEndTimeCode() - UsdStage.GetStartTimeCode();

		UE::MaterialExporterUSD::Private::SendAnalytics(Material, Options, bIsAutomated, ElapsedSeconds, NumberOfFrames, Extension);
	}

	return true;
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UMaterialExporterUsd::ExportMaterialsForStage(
	const TArray<UMaterialInterface*>& Materials,
	const FUsdMaterialBakingOptions& Options,
	const FUsdMetadataExportOptions& MetadataOptions,
	const FString& StageRootLayerPath,
	bool bIsAssetLayer,
	bool bUsePayload,
	bool bReplaceIdentical,
	bool bReExportIdenticalAssets,
	bool bIsAutomated
)
{
#if USE_USD_SDK
	if (StageRootLayerPath.IsEmpty())
	{
		return false;
	}

	if (Materials.Num() == 0)
	{
		return true;
	}

	UsdUnreal::ExportUtils::FUniquePathScope UniquePathScope;

	const FString ExtensionNoDot = FPaths::GetExtension(StageRootLayerPath);

	// If we have multiple materials *within this mesh* that want to be emitted to the same filepath we'll append
	// a suffix, but we will otherwise overwrite any unrelated existing files that were there before we began the export.
	// This allows the workflow of repeatedly exporting over the same files to update the results
	TSet<FString> UsedFilePathsWithoutExt;

	TMap<FString, FString> MaterialPathNameToFilePath;
	for (UMaterialInterface* Material : Materials)
	{
		if (!Material)
		{
			continue;
		}

		// Make sure the material has a resource as we'll use it to fetch some version GUIDs and ExportMaterial
		// can't do it directly due to receiving the material by const ref
		if (UMaterial* Parent = Material->GetMaterial())
		{
			const FMaterialResource* MaterialResource = Parent->GetMaterialResource(GMaxRHIShaderPlatform);
			if (MaterialResource == nullptr)
			{
				Parent->ForceRecompileForRendering();
			}
		}

		// "/Game/ContentFolder/Blue.Blue"
		FString MaterialPathName = Material->GetPathName();

		// "C:/MyFolder/Export/Blue"
		FString MaterialFilePath = FPaths::Combine(FPaths::GetPath(UExporter::CurrentFilename), FPaths::GetBaseFilename(MaterialPathName));

		// "C:/MyFolder/Export/Blue_4"
		FString FinalPathNoExt = UsdUnreal::ObjectUtils::GetUniqueName(MaterialFilePath, UsedFilePathsWithoutExt);

		// "C:/MyFolder/Export/Blue_4.usda"
		FString FinalPath = FString::Printf(TEXT("%s.%s"), *FinalPathNoExt, *ExtensionNoDot);

		if (UMaterialExporterUsd::ExportMaterial(
				*Material,
				Options,
				MetadataOptions,
				FFilePath{FinalPath},
				bReplaceIdentical,
				bReExportIdenticalAssets,
				bIsAutomated
			))
		{
			UsedFilePathsWithoutExt.Add(FinalPathNoExt);
			MaterialPathNameToFilePath.Add(MaterialPathName, FinalPath);
		}
	}

	// We can only open the stage *after* we finished exporting the materials. This because if we're exporting over
	// existing files, it could be that this stage still references the existing material layers that the individual
	// material exports would try to replace, meaning the exports would fail as those files would be currently open.
	const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadAll;
	const bool bUseStageCache = false;
	UE::FUsdStage Stage = UnrealUSDWrapper::OpenStage(*StageRootLayerPath, InitialLoadSet, bUseStageCache);
	if (!Stage)
	{
		return false;
	}

	UE::FSdfLayer RootLayer = Stage.GetRootLayer();
	if (!RootLayer)
	{
		return false;
	}

	UsdUtils::ReplaceUnrealMaterialsWithBaked(Stage, RootLayer, MaterialPathNameToFilePath, bIsAssetLayer, bUsePayload);

	RootLayer.Save();

	return true;
#else
	return false;
#endif	  // USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
