// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExporterUSDOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExporterUSDOptions)

void UsdUtils::AddAnalyticsAttributes(const UMaterialExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	UsdUtils::AddAnalyticsAttributes(Options.MaterialBakingOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MetadataOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
}

void UsdUtils::HashForMaterialExport(const UMaterialExporterUSDOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForMaterialExport(Options.MaterialBakingOptions, HashToUpdate);
	UsdUtils::HashForExport(Options.MetadataOptions, HashToUpdate);
}
