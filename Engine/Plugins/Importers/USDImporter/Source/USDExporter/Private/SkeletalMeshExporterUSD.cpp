// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshExporterUSD.h"

#include "SkeletalMeshExporterUSDOptions.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDExporterModule.h"
#include "USDExportUtils.h"
#include "USDObjectUtils.h"
#include "USDOptionsWindow.h"
#include "USDPrimConversion.h"
#include "USDSkeletalDataConversion.h"
#include "USDTypesConversion.h"
#include "USDUnrealAssetInfo.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "AssetExportTask.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "EngineAnalytics.h"
#include "MaterialExporterUSD.h"
#include "Misc/EngineVersion.h"
#include "Rendering/SkeletalMeshRenderData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshExporterUSD)

#define LOCTEXT_NAMESPACE "SkeletalMeshExporterUSD"

namespace UE::SkeletalMeshExporterUSD::Private
{
	void SendAnalytics(
		UObject* Asset,
		USkeletalMeshExporterUSDOptions* Options,
		bool bAutomated,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension
	)
	{
		if (!Asset || !FEngineAnalytics::IsAvailable())
		{
			return;
		}

		const FString ClassName = IUsdClassesModule::GetClassNameForAnalytics(Asset);

		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Emplace(TEXT("AssetType"), ClassName);

		if (Options)
		{
			UsdUtils::AddAnalyticsAttributes(*Options, EventAttributes);
		}

		IUsdClassesModule::SendAnalytics(
			MoveTemp(EventAttributes),
			FString::Printf(TEXT("Export.%s"), *ClassName),
			bAutomated,
			ElapsedSeconds,
			NumberOfFrames,
			Extension
		);
	}

	void HashSkeletalMesh(const USkeletalMesh* SkeletalMesh, FSHA1& InOutHashToUpdate)
	{
		if (const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering())
		{
			const FString& DDCKey = RenderData->DerivedDataKey;
			InOutHashToUpdate.UpdateWithString(*DDCKey, DDCKey.Len());
		}

		for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
		{
			FString MaterialPath = TEXT("None");
			if (SkeletalMaterial.MaterialInterface)
			{
				MaterialPath = SkeletalMaterial.MaterialInterface->GetPathName();
			}
			InOutHashToUpdate.UpdateWithString(*MaterialPath, MaterialPath.Len());

			// Note that we could hash the material slot name here too, but we don't because we always
			// just write out the slots with UsdGeomSubsets named "Section0", "Section1", ..., "SectionN" anyway
		}
	}
}	 // namespace UE::SkeletalMeshExporterUSD::Private

USkeletalMeshExporterUsd::USkeletalMeshExporterUsd()
{
#if USE_USD_SDK
	UnrealUSDWrapper::AddUsdExportFileFormatDescriptions(FormatExtension, FormatDescription);
	SupportedClass = USkeletalMesh::StaticClass();
	bText = false;
#endif	  // #if USE_USD_SDK
}

bool USkeletalMeshExporterUsd::ExportBinary(
	UObject* Object,
	const TCHAR* Type,
	FArchive& Ar,
	FFeedbackContext* Warn,
	int32 FileIndex,
	uint32 PortFlags
)
{
#if USE_USD_SDK
	USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(Object);
	if (!SkeletalMesh)
	{
		return false;
	}

	USkeletalMeshExporterUSDOptions* Options = nullptr;
	if (ExportTask)
	{
		Options = Cast<USkeletalMeshExporterUSDOptions>(ExportTask->Options);
	}
	if (!Options)
	{
		Options = GetMutableDefault<USkeletalMeshExporterUSDOptions>();

		// Prompt with an options dialog if we can
		if (Options && (!ExportTask || !ExportTask->bAutomated))
		{
			Options->MeshAssetOptions.MaterialBakingOptions.TexturesDir.Path = FPaths::Combine(
				FPaths::GetPath(UExporter::CurrentFilename),
				TEXT("Textures")
			);

			const bool bContinue = SUsdOptionsWindow::ShowExportOptions(*Options);
			if (!bContinue)
			{
				return false;
			}
		}
	}
	if (!Options)
	{
		return false;
	}

	// See comment on the analogous line within StaticMeshExporterUSD.cpp
	ExportTask->bPrompt = false;

	// If bUsePayload is true, we'll intercept the filename so that we write the mesh data to
	// "C:/MyFolder/file_payload.usda" and create an "asset" file "C:/MyFolder/file.usda" that uses it
	// as a payload, pointing at the default prim
	FString PayloadFilename = UExporter::CurrentFilename;
	if (Options && Options->MeshAssetOptions.bUsePayload)
	{
		FString PathPart;
		FString FilenamePart;
		FString ExtensionPart;
		FPaths::Split(PayloadFilename, PathPart, FilenamePart, ExtensionPart);

		if (FormatExtension.Contains(Options->MeshAssetOptions.PayloadFormat))
		{
			ExtensionPart = Options->MeshAssetOptions.PayloadFormat;
		}

		PayloadFilename = FPaths::Combine(PathPart, FilenamePart + TEXT("_payload.") + ExtensionPart);
	}

	if (!IUsdExporterModule::CanExportToLayer(UExporter::CurrentFilename)
		|| (Options->MeshAssetOptions.bUsePayload && !IUsdExporterModule::CanExportToLayer(PayloadFilename)))
	{
		return false;
	}

	UsdUnreal::ExportUtils::FUniquePathScope UniquePathScope;

	// Get a simple GUID hash/identifier of our mesh
	FSHA1 SHA1;
	UE::SkeletalMeshExporterUSD::Private::HashSkeletalMesh(SkeletalMesh, SHA1);
	UsdUtils::HashForSkeletalMeshExport(*Options, SHA1);
	SHA1.Final();
	FSHAHash Hash;
	SHA1.GetHash(&Hash.Hash[0]);
	FString CurrentHashString = Hash.ToString();

	// Check if we already have exported what we plan on exporting anyway
	if (FPaths::FileExists(UExporter::CurrentFilename) && FPaths::FileExists(PayloadFilename))
	{
		if (!ExportTask->bReplaceIdentical)
		{
			USD_LOG_USERINFO(FText::Format(
				LOCTEXT("FileAlreadyExists", "Skipping export of asset '{0}' as the target file '{1}' already exists."),
				FText::FromString(Object->GetPathName()),
				FText::FromString(UExporter::CurrentFilename)
			));
			return false;
		}
		// If we don't want to re-export this asset we need to check if its the same version
		else if (!Options->bReExportIdenticalAssets)
		{
			bool bSkipMeshExport = false;

			// Don't use the stage cache here as we want this stage to close within this scope in case
			// we have to overwrite its files due to e.g. missing payload or anything like that
			const bool bUseStageCache = false;
			const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadNone;
			if (UE::FUsdStage TempStage = UnrealUSDWrapper::OpenStage(*UExporter::CurrentFilename, InitialLoadSet, bUseStageCache))
			{
				if (UE::FUsdPrim DefaultPrim = TempStage.GetDefaultPrim())
				{
					FUsdUnrealAssetInfo Info = UsdUtils::GetPrimAssetInfo(DefaultPrim);

					const bool bVersionMatches = !Info.Version.IsEmpty() && Info.Version == CurrentHashString;

					const bool bAssetTypeMatches = !Info.UnrealAssetType.IsEmpty() && Info.UnrealAssetType == SkeletalMesh->GetClass()->GetName();

					if (bVersionMatches && bAssetTypeMatches)
					{
						USD_LOG_USERINFO(FText::Format(
							LOCTEXT(
								"FileUpToDate",
								"Skipping export of asset '{0}' as the target file '{1}' already contains up-to-date exported data."
							),
							FText::FromString(SkeletalMesh->GetPathName()),
							FText::FromString(UExporter::CurrentFilename)
						));

						bSkipMeshExport = true;
					}
				}
			}

			if (bSkipMeshExport)
			{
				// Even if we're not going to export the mesh, we may still need to re-bake materials
				if (Options->MeshAssetOptions.bBakeMaterials)
				{
					TSet<UMaterialInterface*> MaterialsToBake;
					for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
					{
						MaterialsToBake.Add(SkeletalMaterial.MaterialInterface);
					}

					const bool bIsAssetLayer = true;
					UMaterialExporterUsd::ExportMaterialsForStage(
						MaterialsToBake.Array(),
						Options->MeshAssetOptions.MaterialBakingOptions,
						Options->MetadataOptions,
						UExporter::CurrentFilename,
						bIsAssetLayer,
						Options->MeshAssetOptions.bUsePayload,
						ExportTask->bReplaceIdentical,
						Options->bReExportIdenticalAssets,
						ExportTask->bAutomated
					);
				}

				return true;
			}
		}
	}

	double StartTime = FPlatformTime::Cycles64();

	// UsdStage is the payload stage when exporting with payloads, or just the single stage otherwise
	UE::FUsdStage UsdStage = UnrealUSDWrapper::NewStage(*PayloadFilename);
	if (!UsdStage)
	{
		return false;
	}

	if (Options)
	{
		UsdUtils::SetUsdStageMetersPerUnit(UsdStage, Options->StageOptions.MetersPerUnit);
		UsdUtils::SetUsdStageUpAxis(UsdStage, Options->StageOptions.UpAxis);
	}

	FString RootPrimPath = (TEXT("/") + UsdUtils::SanitizeUsdIdentifier(*SkeletalMesh->GetName()));

	const bool bExportAsSkeletal = !Options->MeshAssetOptions.bConvertSkeletalToNonSkeletal;
	UE::FUsdPrim RootPrim = UsdStage.DefinePrim(UE::FSdfPath(*RootPrimPath), bExportAsSkeletal ? TEXT("SkelRoot") : TEXT("Mesh"));
	if (!RootPrim)
	{
		return false;
	}

	UsdStage.SetDefaultPrim(RootPrim);

	// Asset stage always the stage where we write the material assignments
	UE::FUsdStage AssetStage;

	// Using payload: Convert mesh data through the asset stage (that references the payload) so that we can
	// author mesh data on the payload layer and material data on the asset layer
	if (Options && Options->MeshAssetOptions.bUsePayload)
	{
		AssetStage = UnrealUSDWrapper::NewStage(*UExporter::CurrentFilename);
		if (AssetStage)
		{
			UsdUtils::SetUsdStageMetersPerUnit(AssetStage, Options->StageOptions.MetersPerUnit);
			UsdUtils::SetUsdStageUpAxis(AssetStage, Options->StageOptions.UpAxis);

			if (UE::FUsdPrim AssetRootPrim = AssetStage.DefinePrim(UE::FSdfPath(*RootPrimPath)))
			{
				AssetStage.SetDefaultPrim(AssetRootPrim);
				UsdUtils::AddPayload(AssetRootPrim, *PayloadFilename);
			}
		}
	}
	// Not using payload: Just author everything on the current edit target of the single stage
	else
	{
		AssetStage = UsdStage;
	}

	if (bExportAsSkeletal)
	{
		UnrealToUsd::ConvertSkeletalMesh(
			SkeletalMesh,
			RootPrim,
			UsdUtils::GetDefaultTimeCode(),
			&AssetStage,
			Options->MeshAssetOptions.LowestMeshLOD,
			Options->MeshAssetOptions.HighestMeshLOD
		);
	}
	else
	{
		UnrealToUsd::ConvertSkeletalMeshToStaticMesh(SkeletalMesh, RootPrim, UsdUtils::GetDefaultTimeCode(), &AssetStage);
	}

	if (UE::FUsdPrim AssetDefaultPrim = AssetStage.GetDefaultPrim())
	{
		if (Options->MetadataOptions.bExportAssetInfo)
		{
			FUsdUnrealAssetInfo Info;
			Info.Name = SkeletalMesh->GetName();
			Info.Identifier = UExporter::CurrentFilename;
			Info.Version = CurrentHashString;
			Info.UnrealContentPath = SkeletalMesh->GetPathName();
			Info.UnrealAssetType = SkeletalMesh->GetClass()->GetName();
			Info.UnrealExportTime = FDateTime::Now().ToString();
			Info.UnrealEngineVersion = FEngineVersion::Current().ToString();

			UsdUtils::SetPrimAssetInfo(AssetDefaultPrim, Info);
		}

		if (Options->MetadataOptions.bExportAssetMetadata)
		{
			if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(SkeletalMesh))
			{
				UnrealToUsd::ConvertMetadata(
					UserData,
					AssetDefaultPrim,
					Options->MetadataOptions.BlockedPrefixFilters,
					Options->MetadataOptions.bInvertFilters
				);
			}

			if (USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
			{
				if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(Skeleton))
				{
					if (UserData->StageIdentifierToMetadata.Num() > 0)
					{
						UE::FUsdPrim SkelPrim = AssetStage.OverridePrim(
							UE::FSdfPath(*RootPrimPath).AppendChild(UnrealIdentifiers::ExportedSkeletonPrimName)
						);
						UnrealToUsd::ConvertMetadata(
							UserData,
							SkelPrim,
							Options->MetadataOptions.BlockedPrefixFilters,
							Options->MetadataOptions.bInvertFilters
						);
					}
				}
			}
		}
	}

	// Bake materials and replace unrealMaterials with references to the baked files.
	if (Options->MeshAssetOptions.bBakeMaterials)
	{
		TSet<UMaterialInterface*> MaterialsToBake;
		for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
		{
			MaterialsToBake.Add(SkeletalMaterial.MaterialInterface);
		}

		const bool bIsAssetLayer = true;
		UMaterialExporterUsd::ExportMaterialsForStage(
			MaterialsToBake.Array(),
			Options->MeshAssetOptions.MaterialBakingOptions,
			Options->MetadataOptions,
			AssetStage.GetRootLayer().GetRealPath(),
			bIsAssetLayer,
			Options->MeshAssetOptions.bUsePayload,
			ExportTask->bReplaceIdentical,
			Options->bReExportIdenticalAssets,
			ExportTask->bAutomated
		);
	}

	if (AssetStage && UsdStage != AssetStage)
	{
		AssetStage.GetRootLayer().Save();
	}
	UsdStage.GetRootLayer().Save();

	// Analytics
	{
		bool bAutomated = ExportTask ? ExportTask->bAutomated : false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(UExporter::CurrentFilename);

		UE::SkeletalMeshExporterUSD::Private::SendAnalytics(
			Object,
			Options,
			bAutomated,
			ElapsedSeconds,
			UsdUtils::GetUsdStageNumFrames(AssetStage),
			Extension
		);
	}

	return true;
#else
	return false;
#endif	  // #if USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
