// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshExporterUSDOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshExporterUSDOptions)

void UsdUtils::AddAnalyticsAttributes(const USkeletalMeshExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	UsdUtils::AddAnalyticsAttributes(Options.StageOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MeshAssetOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MetadataOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
}

void UsdUtils::HashForSkeletalMeshExport(const USkeletalMeshExporterUSDOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForExport(Options.StageOptions, HashToUpdate);
	UsdUtils::HashForMeshExport(Options.MeshAssetOptions, HashToUpdate);
	// This option is only relevant to skeletal mesh export so it is not hashed as part of the mesh export
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.MeshAssetOptions.bConvertSkeletalToNonSkeletal), sizeof(Options.MeshAssetOptions.bConvertSkeletalToNonSkeletal));
	UsdUtils::HashForExport(Options.MetadataOptions, HashToUpdate);
}
