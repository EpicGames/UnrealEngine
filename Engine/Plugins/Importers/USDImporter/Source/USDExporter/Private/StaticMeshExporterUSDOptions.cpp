// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshExporterUSDOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshExporterUSDOptions)

void UsdUtils::AddAnalyticsAttributes(const UStaticMeshExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	UsdUtils::AddAnalyticsAttributes(Options.StageOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MeshAssetOptions, InOutAttributes);
	UsdUtils::AddAnalyticsAttributes(Options.MetadataOptions, InOutAttributes);
	InOutAttributes.Emplace(TEXT("ReExportIdenticalAssets"), Options.bReExportIdenticalAssets);
}

void UsdUtils::HashForStaticMeshExport(const UStaticMeshExporterUSDOptions& Options, FSHA1& HashToUpdate)
{
	UsdUtils::HashForExport(Options.StageOptions, HashToUpdate);
	UsdUtils::HashForMeshExport(Options.MeshAssetOptions, HashToUpdate);
	UsdUtils::HashForExport(Options.MetadataOptions, HashToUpdate);
}
