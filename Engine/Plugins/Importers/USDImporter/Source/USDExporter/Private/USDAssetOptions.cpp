// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDAssetOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDAssetOptions)

void UsdUtils::AddAnalyticsAttributes(const FUsdMaterialBakingOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	FString BakedPropertiesString;
	{
		const UEnum* PropertyEnum = StaticEnum<EMaterialProperty>();
		for (const FPropertyEntry& PropertyEntry : Options.Properties)
		{
			FString PropertyString = PropertyEnum->GetNameByValue(PropertyEntry.Property).ToString();
			PropertyString.RemoveFromStart(TEXT("MP_"));
			BakedPropertiesString += PropertyString + TEXT(", ");
		}

		BakedPropertiesString.RemoveFromEnd(TEXT(", "));
	}

	InOutAttributes.Emplace(TEXT("BakedProperties"), BakedPropertiesString);
	InOutAttributes.Emplace(TEXT("DefaultTextureSize"), Options.DefaultTextureSize.ToString());
	InOutAttributes.Emplace(TEXT("ConstantColorAsSingleValue"), Options.bConstantColorAsSingleValue);
}

void UsdUtils::AddAnalyticsAttributes(const FUsdMeshAssetOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("UsePayload"), LexToString(Options.bUsePayload));
	if (Options.bUsePayload)
	{
		InOutAttributes.Emplace(TEXT("PayloadFormat"), Options.PayloadFormat);
	}
	InOutAttributes.Emplace(TEXT("BakeMaterials"), Options.bBakeMaterials);
	if (Options.bBakeMaterials)
	{
		UsdUtils::AddAnalyticsAttributes(Options.MaterialBakingOptions, InOutAttributes);
	}
	InOutAttributes.Emplace(TEXT("LowestMeshLOD"), LexToString(Options.LowestMeshLOD));
	InOutAttributes.Emplace(TEXT("HighestMeshLOD"), LexToString(Options.HighestMeshLOD));
	InOutAttributes.Emplace(TEXT("ExportStaticMeshSourceData"), Options.bExportStaticMeshSourceData);
}

void UsdUtils::HashForMaterialExport(const FUsdMaterialBakingOptions& Options, FSHA1& HashToUpdate)
{
	HashToUpdate.Update(reinterpret_cast<const uint8*>(Options.Properties.GetData()), Options.Properties.Num() * Options.Properties.GetTypeSize());

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.DefaultTextureSize), sizeof(Options.DefaultTextureSize));

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bConstantColorAsSingleValue), sizeof(Options.bConstantColorAsSingleValue));

	// If we changed where we want the textures exported we need to re-export them and update the texture paths on the
	// material
	HashToUpdate.UpdateWithString(*Options.TexturesDir.Path, Options.TexturesDir.Path.Len());
}

void UsdUtils::HashForMeshExport(const FUsdMeshAssetOptions& Options, FSHA1& HashToUpdate)
{
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bUsePayload), sizeof(Options.bUsePayload));
	if (Options.bUsePayload)
	{
		HashToUpdate.UpdateWithString(*Options.PayloadFormat, Options.PayloadFormat.Len());
	}

	// These can affect the material bindings we'll write on the exported layer with the mesh
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bBakeMaterials), sizeof(Options.bBakeMaterials));

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.LowestMeshLOD), sizeof(Options.LowestMeshLOD));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.HighestMeshLOD), sizeof(Options.HighestMeshLOD));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportStaticMeshSourceData), sizeof(Options.bExportStaticMeshSourceData));
}
