// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDConversionBlueprintContext.h"

#include "PrimitiveSceneProxy.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDLightConversion.h"
#include "USDMetadata.h"
#include "USDPrimConversion.h"
#include "USDShadeConversion.h"
#include "USDUnrealAssetInfo.h"
#include "USDValueConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/VtValue.h"

#include "InstancedFoliageActor.h"
#include "LandscapeProxy.h"
#include "MaterialOptions.h"
#include "MaterialUtilities.h"
#include "MeshDescription.h"
#include "RHI.h"
#include "StaticMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDConversionBlueprintContext)

#define LOCTEXT_NAMESPACE "USDConversionBlueprintContext"

namespace UE::USDExporter::Private
{
	UE::FUsdPrim GetPrim(const UE::FUsdStage& Stage, const FString& PrimPath)
	{
		if (!Stage)
		{
			USD_LOG_USERERROR(LOCTEXT("MissingStage", "Export context has no stage set! Call SetStage with a root layer filepath first."));
			return {};
		}

		return Stage.GetPrimAtPath(UE::FSdfPath(*PrimPath));
	}

	static void ConvertPropertyEntriesIntoFlattenMaterial(
		const TArray<FPropertyEntry>& PropertiesToBake,
		const FIntPoint& DefaultTextureSize,
		FFlattenMaterial& LandscapeFlattenMaterial
	)
	{
#if USE_USD_SDK
		for (FPropertyEntry Property : PropertiesToBake)
		{
			EFlattenMaterialProperties AnalogueProperty = UsdUtils::MaterialPropertyToFlattenProperty(Property.Property);
			if (AnalogueProperty == EFlattenMaterialProperties::NumFlattenMaterialProperties)
			{
				continue;
			}

			FIntPoint Size = DefaultTextureSize;
			if (Property.bUseCustomSize)
			{
				Size = Property.CustomSize;
			}
			if (Property.bUseConstantValue)
			{
				Size = FIntPoint(0, 0);
			}

			LandscapeFlattenMaterial.SetPropertySize(AnalogueProperty, Size);
		}
#endif
	}
}	 // namespace UE::USDExporter::Private
namespace UnrealToUsdImpl = UE::USDExporter::Private;

UUsdConversionBlueprintContext::~UUsdConversionBlueprintContext()
{
	Cleanup();
}

void UUsdConversionBlueprintContext::SetStageRootLayer(FFilePath StageRootLayerPath)
{
	Cleanup();

	TArray<UE::FUsdStage> PreviouslyOpenedStages = UnrealUSDWrapper::GetAllStagesFromCache();

	Stage = UnrealUSDWrapper::OpenStage(*StageRootLayerPath.FilePath, EUsdInitialLoadSet::LoadAll);
	bEraseFromStageCache = !PreviouslyOpenedStages.Contains(Stage);
}

FFilePath UUsdConversionBlueprintContext::GetStageRootLayer()
{
	if (Stage)
	{
		return FFilePath{Stage.GetRootLayer().GetRealPath()};
	}

	USD_LOG_USERERROR(LOCTEXT("NoStage", "There is no stage currently open!"));
	return {};
}

void UUsdConversionBlueprintContext::SetEditTarget(FFilePath EditTargetLayerPath)
{
	if (Stage)
	{
		if (UE::FSdfLayer EditTargetLayer = UE::FSdfLayer::FindOrOpen(*EditTargetLayerPath.FilePath))
		{
			Stage.SetEditTarget(EditTargetLayer);
		}
		else
		{
			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("FailedToOpenPath", "Failed to find or open USD layer with filepath '{0}'!"),
				FText::FromString(EditTargetLayerPath.FilePath)
			));
		}
	}
	else
	{
		USD_LOG_USERERROR(LOCTEXT("NoStage", "There is no stage currently open!"));
	}
}

FFilePath UUsdConversionBlueprintContext::GetEditTarget()
{
	if (Stage)
	{
		return FFilePath{Stage.GetEditTarget().GetRealPath()};
	}

	USD_LOG_USERERROR(LOCTEXT("NoStage", "There is no stage currently open!"));
	return {};
}

void UUsdConversionBlueprintContext::Cleanup()
{
	if (Stage)
	{
		if (bEraseFromStageCache)
		{
			UnrealUSDWrapper::EraseStageFromCache(Stage);
		}

		Stage = UE::FUsdStage();
	}
}

bool UUsdConversionBlueprintContext::ConvertLightComponent(const ULightComponentBase* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertLightComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertDirectionalLightComponent(
	const UDirectionalLightComponent* Component,
	const FString& PrimPath,
	float TimeCode
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertDirectionalLightComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertRectLightComponent(const URectLightComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertRectLightComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertPointLightComponent(const UPointLightComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertPointLightComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertSkyLightComponent(const USkyLightComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertSkyLightComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertSpotLightComponent(const USpotLightComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertSpotLightComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertDrawModeComponent(const UUsdDrawModeComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	const bool bWriteExtents = true;
	return UnrealToUsd::ConvertDrawModeComponent(*Component, Prim, bWriteExtents, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertAudioComponent(
	const UAudioComponent* Component,
	const FString& PrimPath,
	float TimeCode,
	bool bFilePathOnly
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertAudioComponent(*Component, Prim, bFilePathOnly, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertSceneComponent(const USceneComponent* Component, const FString& PrimPath)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertSceneComponent(Stage, Component, Prim);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertIsmComponent(const UInstancedStaticMeshComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertInstancedStaticMeshComponent(Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertHismComponent(
	const UHierarchicalInstancedStaticMeshComponent* Component,
	const FString& PrimPath,
	float TimeCode
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertHierarchicalInstancedStaticMeshComponent(
		Component,
		Prim,
		TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode
	);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertMeshComponent(const UMeshComponent* Component, const FString& PrimPath)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertMeshComponent(Stage, Component, Prim);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertCineCameraComponent(const UCineCameraComponent* Component, const FString& PrimPath, float TimeCode)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Component)
	{
		return false;
	}

	return UnrealToUsd::ConvertCameraComponent(*Component, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertInstancedFoliageActor(
	const AInstancedFoliageActor* Actor,
	const FString& PrimPath,
	ULevel* InstancesLevel,
	float TimeCode
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Actor)
	{
		return false;
	}

	return UnrealToUsd::ConvertInstancedFoliageActor(*Actor, Prim, TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode, InstancesLevel);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertLandscapeProxyActorMesh(
	const ALandscapeProxy* Actor,
	const FString& PrimPath,
	int32 LowestLOD,
	int32 HighestLOD,
	float TimeCode /*= 3.402823466e+38F */
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Actor)
	{
		return false;
	}

	// Make sure they're both >= 0 (the options dialog slider is clamped, but this may be called directly)
	LowestLOD = FMath::Max(LowestLOD, 0);
	HighestLOD = FMath::Max(HighestLOD, 0);

	// Make sure Lowest <= Highest
	int32 Temp = FMath::Min(LowestLOD, HighestLOD);
	HighestLOD = FMath::Max(LowestLOD, HighestLOD);
	LowestLOD = Temp;

	// Make sure it's at least 1 LOD level
	int32 NumLODs = FMath::Max(HighestLOD - LowestLOD + 1, 1);

	TArray<FMeshDescription> LODMeshDescriptions;
	LODMeshDescriptions.SetNum(NumLODs);

	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		FMeshDescription& MeshDescription = LODMeshDescriptions[LODIndex];

		FStaticMeshAttributes Attributes(MeshDescription);
		Attributes.Register();

		ALandscapeProxy::FRawMeshExportParams ExportParams;
		ExportParams.ExportLOD = LODIndex + LowestLOD;
		if (!Actor->ExportToRawMesh(ExportParams, MeshDescription))
		{
			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("FailedLODLandscape", "Failed to convert LOD {0} of landscape actor '{1}''s mesh data"),
				LODIndex,
				FText::FromString(Actor->GetName())
			));
			return false;
		}
	}

	// Inverse through FMatrix as non-uniform scalings are common for landscapes
	// If we're a streaming proxy, we have an additional transform wrt. the parent ALandscapeProxy actor.
	// If we were to use LandscapeActorToWorld here, it would use ActorToWorld() internally, but automatically compensate the offset from
	// the parent actor, generating the same transform as the one you'd get from calling LandscapeActorToWorld() directly on it.
	// We don't want this here: We want to specifically compensate the proxy actor's transform ourselves, so we need just ActorToWorld.
	FMatrix ActorToWorldInv = Actor->ActorToWorld().ToInverseMatrixWithScale();
	if (!UnrealToUsd::ConvertMeshDescriptions(
			LODMeshDescriptions,
			Prim,
			ActorToWorldInv,
			TimeCode == FLT_MAX ? UsdUtils::GetDefaultTimeCode() : TimeCode
		))
	{
		return false;
	}

	return true;
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertLandscapeProxyActorMaterial(
	ALandscapeProxy* Actor,
	const FString& PrimPath,
	const TArray<FPropertyEntry>& PropertiesToBake,
	const FIntPoint& DefaultTextureSize,
	const FDirectoryPath& TexturesDir,
	float TimeCode /*= 3.402823466e+38F */
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !Actor)
	{
		return false;
	}

	FFlattenMaterial LandscapeFlattenMaterial;
	UnrealToUsdImpl::ConvertPropertyEntriesIntoFlattenMaterial(PropertiesToBake, DefaultTextureSize, LandscapeFlattenMaterial);

	// FMaterialUtilities::ExportLandscapeMaterial will basically render the scene with an orthographic transform. We need to hide
	// each and every actor and component that is not part of the landscape for that render, or else they will show up on the bake
	TSet<FPrimitiveComponentId> PrimitivesToHide;
	for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
	{
		UPrimitiveComponent* PrimitiveComp = *It;
		const bool bTargetPrim = (PrimitiveComp->GetOuter() == Actor);

		if (!bTargetPrim && PrimitiveComp->IsRegistered() && PrimitiveComp->SceneProxy)
		{
			PrimitivesToHide.Add(PrimitiveComp->SceneProxy->GetPrimitiveComponentId());
		}
	}
	FMaterialUtilities::ExportLandscapeMaterial(Actor, PrimitivesToHide, LandscapeFlattenMaterial);

	// If we asked to bake a texture with some size and it's just a constant value, FMaterialUtilities::ExportLandscapeMaterial will
	// emit the full texture anyway, so we need to collapse it back into being just one pixel
	UsdUtils::CollapseConstantChannelsToSinglePixel(LandscapeFlattenMaterial);

	// Synchronize the intended bake size for channels that FMaterialUtilities::ExportLandscapeMaterial didn't bake (or that we collapsed),
	// or else downstream code may assume that there was some form of error
	for (FPropertyEntry PropertyToBake : PropertiesToBake)
	{
		EFlattenMaterialProperties AnalogueProperty = UsdUtils::MaterialPropertyToFlattenProperty(PropertyToBake.Property);
		if (AnalogueProperty == EFlattenMaterialProperties::NumFlattenMaterialProperties)
		{
			continue;
		}

		const TArray<FColor>& Samples = LandscapeFlattenMaterial.GetPropertySamples(AnalogueProperty);
		if (Samples.Num() == 0)
		{
			LandscapeFlattenMaterial.SetPropertySize(AnalogueProperty, FIntPoint(0, 0));
		}
	}

	bool bSuccess = UnrealToUsd::ConvertFlattenMaterial(Actor->GetActorLabel(), LandscapeFlattenMaterial, PropertiesToBake, TexturesDir, Prim);

	// FMaterialUtilities::ExportLandscapeMaterial always bakes WorldNormals, so we need to write to this material that it's meant to be using
	// world-space normals. On import, we check for this and set the proper scalar parameter on our UsdPreviewSurface material to compensate for it.
	// This is done on the shader via a scalar parameter to avoid quantizing the normal info twice (which converting on the CPU would do) and to allow
	// usage during runtime (which wouldn't be possible with static switch parameters)
	if (bSuccess)
	{
		bSuccess &= UsdUtils::MarkMaterialPrimWithWorldSpaceNormals(Prim);
	}

	return bSuccess;
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::ConvertMaterialOverrides(
	const UObject* MeshAsset,
	const TArray<UMaterialInterface*> MaterialOverrides,
	const FString& PrimPath,
	int32 LowestLOD,
	int32 HighestLOD,
	bool bCheckAssetUserData
)
{
#if USE_USD_SDK
	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim || !MeshAsset)
	{
		return false;
	}

	return UnrealToUsd::ConvertMaterialOverrides(MeshAsset, MaterialOverrides, Prim, LowestLOD, HighestLOD, bCheckAssetUserData);
#else
	return false;
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintContext::ReplaceUnrealMaterialsWithBaked(
	const FFilePath& LayerToAuthorIn,
	const TMap<FString, FString>& BakedMaterials,
	bool bIsAssetLayer,
	bool bUsePayload,
	bool bRemoveUnrealMaterials
)
{
#if USE_USD_SDK
	if (!Stage || LayerToAuthorIn.FilePath.IsEmpty())
	{
		return;
	}

	UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(*LayerToAuthorIn.FilePath);
	if (!Layer)
	{
		return;
	}

	UsdUtils::ReplaceUnrealMaterialsWithBaked(Stage, Layer, BakedMaterials, bIsAssetLayer, bUsePayload);
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintContext::AuthorUnrealMaterialBinding(
	const FString& MeshLikePrimPath,
	const FString& UnrealMaterialPathName,
	bool bStrongerThanDescendants
)
{
#if USE_USD_SDK
	if (!Stage || UnrealMaterialPathName.IsEmpty())
	{
		return;
	}

	UE::FUsdPrim Prim = Stage.GetPrimAtPath(UE::FSdfPath{*MeshLikePrimPath});
	if (!Prim)
	{
		return;
	}

	UsdUtils::AuthorUnrealMaterialBinding(Prim, UnrealMaterialPathName, bStrongerThanDescendants);
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintContext::RemoveUnrealSurfaceOutput(const FString& PrimPath, const FFilePath& LayerToAuthorIn)
{
	return false;
}

int32 UUsdConversionBlueprintContext::GetUsdStageNumFrames()
{
#if USE_USD_SDK
	if (!Stage)
	{
		return 0;
	}

	return UsdUtils::GetUsdStageNumFrames(Stage);
#else
	return 0;
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintContext::SetPrimAssetInfo(const FString& PrimPath, const FUsdUnrealAssetInfo& Info)
{
#if USE_USD_SDK
	if (!Stage)
	{
		return;
	}

	UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath);
	if (!Prim)
	{
		return;
	}

	UsdUtils::SetPrimAssetInfo(Prim, Info);
#endif	  // USE_USD_SDK
}

FUsdUnrealAssetInfo UUsdConversionBlueprintContext::GetPrimAssetInfo(const FString& PrimPath)
{
#if USE_USD_SDK
	if (Stage)
	{
		if (UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath))
		{
			return UsdUtils::GetPrimAssetInfo(Prim);
		}
	}
#endif	  // USE_USD_SDK

	return {};
}

void UUsdConversionBlueprintContext::SetPrimMetadata(
	const FString& PrimPath,
	const FUsdCombinedPrimMetadata& Metadata,
	const TArray<FString>& BlockedPrefixFilter,
	bool bInvertFilter
)
{
#if USE_USD_SDK
	if (Stage)
	{
		if (UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath))
		{
			UnrealToUsd::ConvertMetadata(Metadata, Prim, BlockedPrefixFilter, bInvertFilter);
		}
	}
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintContext::SetPrimMetadataFromUserData(
	const FString& PrimPath,
	const UUsdAssetUserData* UserData,
	const TArray<FString>& BlockedPrefixFilter,
	bool bInvertFilter
)
{
#if USE_USD_SDK
	if (Stage && UserData)
	{
		if (UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath))
		{
			UnrealToUsd::ConvertMetadata(UserData, Prim, BlockedPrefixFilter, bInvertFilter);
		}
	}
#endif	  // USE_USD_SDK
}

FUsdCombinedPrimMetadata UUsdConversionBlueprintContext::GetPrimMetadata(
	const FString& PrimPath,
	const TArray<FString>& BlockedPrefixFilter,
	bool bInvertFilter,
	bool bCollectFromEntireSubtrees
)
{
	FUsdCombinedPrimMetadata Result;

#if USE_USD_SDK
	if (Stage)
	{
		if (UE::FUsdPrim Prim = UnrealToUsdImpl::GetPrim(Stage, PrimPath))
		{
			bool bSuccess = UsdToUnreal::ConvertMetadata(Prim, Result, BlockedPrefixFilter, bInvertFilter, bCollectFromEntireSubtrees);
			if (!bSuccess)
			{
				USD_LOG_USERWARNING(
					FText::Format(LOCTEXT("FailedToGetMetadata", "Failed to get metadata from prim '{0}'"), FText::FromString(PrimPath))
				);
				Result = {};
			}
		}
	}
#endif	  // USE_USD_SDK

	return Result;
}

#undef LOCTEXT_NAMESPACE
