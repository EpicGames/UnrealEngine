// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDConversionBlueprintLibrary.h"

#include "LevelExporterUSDOptions.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDExporterModule.h"
#include "USDExportUtils.h"
#include "USDLayerUtils.h"
#include "USDObjectUtils.h"
#include "USDValueConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "AnalyticsBlueprintLibrary.h"
#include "AnalyticsEventAttribute.h"
#include "AssetCompilingManager.h"
#include "ContentStreaming.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "InstancedFoliageActor.h"
#include "ISequencer.h"
#include "LevelEditorSequencerIntegration.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDConversionBlueprintLibrary)

#define LOCTEXT_NAMESPACE "USDConversionBlueprintLibrary"

namespace UE::UsdConversionBlueprintLibrary::Private
{
	void WaitForAllAsyncAndSteamingTasks(UWorld* World)
	{
		FlushAsyncLoading();

		if (World)
		{
			World->BlockTillLevelStreamingCompleted();

			if (!FPlatformProperties::RequiresCookedData())
			{
				UMaterialInterface::SubmitRemainingJobsForWorld(World);
				FAssetCompilingManager::Get().FinishAllCompilation();
			}
		}

		UTexture::ForceUpdateTextureStreaming();

		IStreamingManager::Get().StreamAllResources(0.0f);
	}

	void StreamInLevels(ULevel* Level, const TSet<FString>& LevelsToIgnore)
	{
		UWorld* InnerWorld = Level->GetTypedOuter<UWorld>();
		if (!InnerWorld || InnerWorld->GetStreamingLevels().Num() == 0)
		{
			return;
		}

		// Ensure our world to export has a context so that level streaming doesn't crash.
		// This is needed exclusively to be able to load other levels from python scripts using just `load_asset`
		// and have them be exportable.
		bool bCreatedContext = false;
		FWorldContext* Context = GEngine->GetWorldContextFromWorld(InnerWorld);
		if (!Context)
		{
			FWorldContext& NewContext = GEngine->CreateNewWorldContext(EWorldType::EditorPreview);
			NewContext.SetCurrentWorld(InnerWorld);

			bCreatedContext = true;
		}

		// Mark all sublevels that we need to be loaded
		for (ULevelStreaming* StreamingLevel : InnerWorld->GetStreamingLevels())
		{
			if (StreamingLevel)
			{
				// Note how we will always load the sublevel's package even if will ignore this level. This is a workaround
				// to a level streaming bug/quirk:
				// As soon as we first load the sublevels's package the component scene proxies will be (incorrectly?) placed on
				// the vestigial worlds' FScene. When exporting sublevels though, we need the scene proxies to be on the owning
				// world, especially for landscapes as their materials will be baked by essentially taking a camera screenshot
				// from the FScene.
				// Because of that, we have to always at least load the sublevels here so that the FlushLevelStreaming call below can
				// call World->RemoveFromWorld from within ULevelStreaming::UpdateStreamingState when we call FlushLevelStreaming
				// below and leave the loaded level's component's unregistered.
				// This ensures that if we later try exporting this same sublevel, we won't get the components first registered
				// during the process of actually first loading the sublevel (which would have placed them on the vestigial world), but
				// instead, since the level *is already loaded*, a future call to to FlushLevelStreaming below can trigger the landscape
				// components to be registered on the owning world by the World->AddToWorld call.
				const FString StreamingLevelWorldAssetPackageName = StreamingLevel->GetWorldAssetPackageName();
				const FName StreamingLevelWorldAssetPackageFName = StreamingLevel->GetWorldAssetPackageFName();
				ULevel::StreamedLevelsOwningWorld.Add(StreamingLevelWorldAssetPackageFName, InnerWorld);
				UPackage* Package = LoadPackage(nullptr, *StreamingLevelWorldAssetPackageName, LOAD_None);
				ULevel::StreamedLevelsOwningWorld.Remove(StreamingLevelWorldAssetPackageFName);

				const FString LevelName = FPaths::GetBaseFilename(StreamingLevel->GetWorldAssetPackageName());
				if (LevelsToIgnore.Contains(LevelName))
				{
					continue;
				}

				const bool bInShouldBeLoaded = true;
				StreamingLevel->SetShouldBeLoaded(bInShouldBeLoaded);

				// This is a workaround to a level streaming bug/quirk:
				// We must set these to false here in order to get both our streaming level's current and target
				// state to LoadedNotVisible. If the level is already visible when we call FlushLevelStreaming,
				// our level will go directly to the LoadedVisible state.
				// This may seem desirable, but it means the second FlushLevelStreaming call below won't really do anything.
				// We need ULevelStreaming::UpdateStreamingState to call World->RemoveFromWorld and World->AddToWorld though,
				// as that is the only thing that will force the sublevel components' scene proxies to being added to the
				// *owning* world's FScene, as opposed to being left at the vestigial world's FScenes instead.
				// If we don't do this, we may get undesired effects when exporting anything that relies on the actual FScene,
				// like baking landscape materials (see UE-126953)
				{
					const bool bShouldBeVisible = false;
					StreamingLevel->SetShouldBeVisible(bShouldBeVisible);
					StreamingLevel->SetShouldBeVisibleInEditor(bShouldBeVisible);
				}
			}
		}

		// Synchronously stream in levels
		InnerWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);

		// Mark all sublevels that we need to be made visible
		// For whatever reason this needs to be done with two separate flushes: One for loading, one for
		// turning visible. If we do this with a single flush the levels will not be synchronously loaded *and* made visible
		// on this same exact frame, and if we're e.g. baking a landscape immediately after this,
		// its material will be baked incorrectly (check test_export_level_landscape_bake.py)
		for (ULevelStreaming* StreamingLevel : InnerWorld->GetStreamingLevels())
		{
			if (StreamingLevel)
			{
				const FString LevelName = FPaths::GetBaseFilename(StreamingLevel->GetWorldAssetPackageName());
				if (LevelsToIgnore.Contains(LevelName))
				{
					continue;
				}

				const bool bInShouldBeVisible = true;
				StreamingLevel->SetShouldBeVisible(bInShouldBeVisible);
				StreamingLevel->SetShouldBeVisibleInEditor(bInShouldBeVisible);
			}
		}

		// Synchronously show levels right now
		InnerWorld->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);

		WaitForAllAsyncAndSteamingTasks(InnerWorld);

		if (bCreatedContext)
		{
			GEngine->DestroyWorldContext(InnerWorld);
		}
	}
}	 // namespace UE::UsdConversionBlueprintLibrary::Private

int32 UUsdConversionBlueprintLibrary::GetNumLevelsToExport(UWorld* World, const TSet<FString>& LevelsToIgnore)
{
	int32 Count = 0;

	if (!World)
	{
		return Count;
	}

	if (!LevelsToIgnore.Contains("Persistent Level"))
	{
		Count += 1;
	}

	for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
	{
		if (StreamingLevel)
		{
			const FString LevelName = FPaths::GetBaseFilename(StreamingLevel->GetWorldAssetPackageName());
			if (!LevelsToIgnore.Contains(LevelName))
			{
				continue;
			}

			Count += 1;
		}
	}

	return Count;
}

void UUsdConversionBlueprintLibrary::StreamInRequiredLevels(UWorld* World, const TSet<FString>& LevelsToIgnore)
{
	if (!World)
	{
		return;
	}

	if (ULevel* PersistentLevel = World->PersistentLevel)
	{
		UE::UsdConversionBlueprintLibrary::Private::StreamInLevels(PersistentLevel, LevelsToIgnore);
	}
}

void UUsdConversionBlueprintLibrary::RevertSequencerAnimations()
{
	for (const TWeakPtr<ISequencer>& Sequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
	{
		if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
		{
			PinnedSequencer->EnterSilentMode();
			PinnedSequencer->RestorePreAnimatedState();
		}
	}
}

void UUsdConversionBlueprintLibrary::ReapplySequencerAnimations()
{
	for (const TWeakPtr<ISequencer>& Sequencer : FLevelEditorSequencerIntegration::Get().GetSequencers())
	{
		if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
		{
			PinnedSequencer->InvalidateCachedData();
			PinnedSequencer->ForceEvaluate();
			PinnedSequencer->ExitSilentMode();
		}
	}
}

TArray<FString> UUsdConversionBlueprintLibrary::GetLoadedLevelNames(UWorld* World)
{
	TArray<FString> Result;

	for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
	{
		if (StreamingLevel && StreamingLevel->IsLevelLoaded())
		{
			Result.Add(StreamingLevel->GetWorldAssetPackageName());
		}
	}

	return Result;
}

TArray<FString> UUsdConversionBlueprintLibrary::GetVisibleInEditorLevelNames(UWorld* World)
{
	TArray<FString> Result;

	for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
	{
		if (StreamingLevel && StreamingLevel->GetShouldBeVisibleInEditor())
		{
			Result.Add(StreamingLevel->GetWorldAssetPackageName());
		}
	}

	return Result;
}

void UUsdConversionBlueprintLibrary::StreamOutLevels(
	UWorld* OwningWorld,
	const TArray<FString>& LevelNamesToStreamOut,
	const TArray<FString>& LevelNamesToHide
)
{
	if (LevelNamesToStreamOut.Num() == 0 && LevelNamesToHide.Num() == 0)
	{
		return;
	}

	bool bCreatedContext = false;
	FWorldContext* Context = GEngine->GetWorldContextFromWorld(OwningWorld);
	if (!Context)
	{
		FWorldContext& NewContext = GEngine->CreateNewWorldContext(EWorldType::EditorPreview);
		NewContext.SetCurrentWorld(OwningWorld);

		bCreatedContext = true;
	}

	for (ULevelStreaming* StreamingLevel : OwningWorld->GetStreamingLevels())
	{
		if (StreamingLevel)
		{
			const FString& LevelName = StreamingLevel->GetWorldAssetPackageName();

			if (LevelNamesToHide.Contains(LevelName))
			{
				const bool bInShouldBeVisible = false;
				StreamingLevel->SetShouldBeVisible(bInShouldBeVisible);
				StreamingLevel->SetShouldBeVisibleInEditor(bInShouldBeVisible);
			}

			if (LevelNamesToStreamOut.Contains(LevelName))
			{
				const bool bInShouldBeLoaded = false;
				StreamingLevel->SetShouldBeVisible(bInShouldBeLoaded);
				StreamingLevel->SetShouldBeLoaded(bInShouldBeLoaded);
			}
		}
	}

	UE::UsdConversionBlueprintLibrary::Private::WaitForAllAsyncAndSteamingTasks(OwningWorld);

	if (bCreatedContext)
	{
		GEngine->DestroyWorldContext(OwningWorld);
	}
}

TSet<AActor*> UUsdConversionBlueprintLibrary::GetActorsToConvert(UWorld* World)
{
	TSet<AActor*> Result;
	if (!World)
	{
		return Result;
	}

	auto CollectActors = [&Result](ULevel* Level)
	{
		if (!Level)
		{
			return;
		}

		Result.Append(ObjectPtrDecay(Level->Actors));
	};

	CollectActors(World->PersistentLevel);

	for (ULevelStreaming* StreamingLevel : World->GetStreamingLevels())
	{
		if (StreamingLevel && StreamingLevel->IsLevelLoaded() && StreamingLevel->GetShouldBeVisibleInEditor())
		{
			if (ULevel* Level = StreamingLevel->GetLoadedLevel())
			{
				CollectActors(Level);
			}
		}
	}

	Result.Remove(nullptr);

	// Remove transient actors here because it is not possible to do this via Python
	TSet<AActor*> ActorsToRemove;
	ActorsToRemove.Reserve(Result.Num());
	for (AActor* Actor : Result)
	{
		// Actors marked with this tag are transient because they're spawnables: We still want to export those, in case
		// we're exporting a level for a LevelSequence export with spawnables.
		if (Actor->HasAnyFlags(EObjectFlags::RF_Transient) && !Actor->Tags.Contains(TEXT("SequencerActor")))
		{
			ActorsToRemove.Add(Actor);
		}
	}
	Result = Result.Difference(ActorsToRemove);
	return Result;
}

FString UUsdConversionBlueprintLibrary::GenerateObjectVersionString(const UObject* ObjectToExport, UObject* ExportOptions)
{
	if (!ObjectToExport)
	{
		return {};
	}

	FSHA1 SHA1;

	if (!IUsdClassesModule::HashObjectPackage(ObjectToExport, SHA1))
	{
		return {};
	}

	if (ULevelExporterUSDOptions* LevelExportOptions = Cast<ULevelExporterUSDOptions>(ExportOptions))
	{
		UsdUtils::HashForLevelExport(*LevelExportOptions, SHA1);
	}

	FSHAHash Hash;
	SHA1.Final();
	SHA1.GetHash(&Hash.Hash[0]);
	return Hash.ToString();
}

bool UUsdConversionBlueprintLibrary::CanExportToLayer(const FString& TargetFilePath)
{
	return IUsdExporterModule::CanExportToLayer(TargetFilePath);
}

FString UUsdConversionBlueprintLibrary::MakePathRelativeToLayer(const FString& AnchorLayerPath, const FString& PathToMakeRelative)
{
#if USE_USD_SDK
	if (UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(*AnchorLayerPath))
	{
		FString Path = PathToMakeRelative;
		UsdUtils::MakePathRelativeToLayer(Layer, Path);
		return Path;
	}
	else
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("FailedToFindAnchor", "Failed to find a layer with path '{0}' to make the path '{1}' relative to"),
			FText::FromString(AnchorLayerPath),
			FText::FromString(PathToMakeRelative)
		));
		return PathToMakeRelative;
	}
#else
	return FString();
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintLibrary::InsertSubLayer(const FString& ParentLayerPath, const FString& SubLayerPath, int32 Index /*= -1 */)
{
#if USE_USD_SDK
	if (ParentLayerPath.IsEmpty() || SubLayerPath.IsEmpty())
	{
		return;
	}

	if (UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(*ParentLayerPath))
	{
		UsdUtils::InsertSubLayer(Layer, *SubLayerPath, Index);
	}
	else
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("FailedToFindParent", "Failed to find a parent layer '{0}' when trying to insert sublayer '{1}'"),
			FText::FromString(ParentLayerPath),
			FText::FromString(SubLayerPath)
		));
	}
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintLibrary::AddReference(
	const FString& ReferencingStagePath,
	const FString& ReferencingPrimPath,
	const FString& TargetStagePath,
	const FString& TargetPrimPath,
	double TimeCodeOffset,
	double TimeCodeScale,
	bool bUseProjectDefaultTypeHandling,
	EReferencerTypeHandling ReferencerTypeHandling
)
{
#if USE_USD_SDK
	TArray<UE::FUsdStage> PreviouslyOpenedStages = UnrealUSDWrapper::GetAllStagesFromCache();

	// Open using the stage cache as it's very likely this stage is already in there anyway
	UE::FUsdStage ReferencingStage = UnrealUSDWrapper::OpenStage(*ReferencingStagePath, EUsdInitialLoadSet::LoadAll);
	if (ReferencingStage)
	{
		if (UE::FUsdPrim ReferencingPrim = ReferencingStage.GetPrimAtPath(UE::FSdfPath(*ReferencingPrimPath)))
		{
			UsdUtils::AddReference(
				ReferencingPrim,
				*TargetStagePath,
				bUseProjectDefaultTypeHandling ? TOptional<EReferencerTypeHandling>{} : ReferencerTypeHandling,
				TargetPrimPath.IsEmpty() ? UE::FSdfPath{} : UE::FSdfPath{*TargetPrimPath},
				TimeCodeOffset,
				TimeCodeScale
			);
		}
	}

	// Cleanup or else the stage cache will keep these stages open forever
	if (!PreviouslyOpenedStages.Contains(ReferencingStage))
	{
		UnrealUSDWrapper::EraseStageFromCache(ReferencingStage);
	}
#endif	  // USE_USD_SDK
}

void UUsdConversionBlueprintLibrary::AddPayload(
	const FString& ReferencingStagePath,
	const FString& ReferencingPrimPath,
	const FString& TargetStagePath,
	const FString& TargetPrimPath,
	double TimeCodeOffset,
	double TimeCodeScale,
	bool bUseProjectDefaultTypeHandling,
	EReferencerTypeHandling ReferencerTypeHandling
)
{
#if USE_USD_SDK
	TArray<UE::FUsdStage> PreviouslyOpenedStages = UnrealUSDWrapper::GetAllStagesFromCache();

	// Open using the stage cache as it's very likely this stage is already in there anyway
	UE::FUsdStage ReferencingStage = UnrealUSDWrapper::OpenStage(*ReferencingStagePath, EUsdInitialLoadSet::LoadAll);
	if (ReferencingStage)
	{
		if (UE::FUsdPrim ReferencingPrim = ReferencingStage.GetPrimAtPath(UE::FSdfPath(*ReferencingPrimPath)))
		{
			UsdUtils::AddPayload(
				ReferencingPrim,
				*TargetStagePath,
				bUseProjectDefaultTypeHandling ? TOptional<EReferencerTypeHandling>{} : ReferencerTypeHandling,
				TargetPrimPath.IsEmpty() ? UE::FSdfPath{} : UE::FSdfPath{*TargetPrimPath},
				TimeCodeOffset,
				TimeCodeScale
			);
		}
	}

	// Cleanup or else the stage cache will keep these stages open forever
	if (!PreviouslyOpenedStages.Contains(ReferencingStage))
	{
		UnrealUSDWrapper::EraseStageFromCache(ReferencingStage);
	}
#endif	  // USE_USD_SDK
}

FString UUsdConversionBlueprintLibrary::GetPrimPathForObject(
	const UObject* ActorOrComponent,
	const FString& ParentPrimPath,
	bool bUseActorFolders,
	const FString& RootPrimName
)
{
#if USE_USD_SDK
	return UsdUtils::GetPrimPathForObject(ActorOrComponent, ParentPrimPath, bUseActorFolders, RootPrimName);
#else
	return {};
#endif	  // USE_USD_SDK
}

FString UUsdConversionBlueprintLibrary::GetSchemaNameForComponent(const USceneComponent* Component)
{
#if USE_USD_SDK
	if (Component)
	{
		return UsdUtils::GetSchemaNameForComponent(*Component);
	}
#endif	  // USE_USD_SDK

	return {};
}

AInstancedFoliageActor* UUsdConversionBlueprintLibrary::GetInstancedFoliageActorForLevel(bool bCreateIfNone /*= false */, ULevel* Level /*= nullptr */)
{
	if (!Level)
	{
		const bool bEnsureIsGWorld = false;
		UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext(bEnsureIsGWorld).World() : nullptr;
		if (!EditorWorld)
		{
			return nullptr;
		}

		Level = EditorWorld->GetCurrentLevel();
		if (!Level)
		{
			return nullptr;
		}
	}

	return AInstancedFoliageActor::GetInstancedFoliageActorForLevel(Level, bCreateIfNone);
}

TArray<UFoliageType*> UUsdConversionBlueprintLibrary::GetUsedFoliageTypes(AInstancedFoliageActor* Actor)
{
	TArray<UFoliageType*> Result;
	if (!Actor)
	{
		return Result;
	}

	for (const TPair<UFoliageType*, TUniqueObj<FFoliageInfo>>& FoliagePair : Actor->GetFoliageInfos())
	{
		Result.Add(FoliagePair.Key);
	}

	return Result;
}

UObject* UUsdConversionBlueprintLibrary::GetSource(UFoliageType* FoliageType)
{
	if (FoliageType)
	{
		return FoliageType->GetSource();
	}

	return nullptr;
}

TArray<FTransform> UUsdConversionBlueprintLibrary::GetInstanceTransforms(
	AInstancedFoliageActor* Actor,
	UFoliageType* FoliageType,
	ULevel* InstancesLevel
)
{
	TArray<FTransform> Result;
	if (!Actor || !FoliageType)
	{
		return Result;
	}

	// Modified from AInstancedFoliageActor::GetInstancesForComponent to limit traversal only to our FoliageType

	if (const TUniqueObj<FFoliageInfo>* FoundInfo = Actor->GetFoliageInfos().Find(FoliageType))
	{
		const FFoliageInfo& Info = (*FoundInfo).Get();

		// Collect IDs of components that are on the same level as the actor's level. This because later on we'll have level-by-level
		// export, and we'd want one point instancer per level
		for (const TPair<FFoliageInstanceBaseId, FFoliageInstanceBaseInfo>& FoliageInstancePair : Actor->InstanceBaseCache.InstanceBaseMap)
		{
			UActorComponent* Comp = FoliageInstancePair.Value.BasePtr.Get();
			if (!Comp || (InstancesLevel && (Comp->GetComponentLevel() != InstancesLevel)))
			{
				continue;
			}

			if (const auto* InstanceSet = Info.ComponentHash.Find(FoliageInstancePair.Key))
			{
				Result.Reserve(Result.Num() + InstanceSet->Num());
				for (int32 InstanceIndex : *InstanceSet)
				{
					const FFoliageInstancePlacementInfo* Instance = &Info.Instances[InstanceIndex];
					Result.Emplace(FQuat(Instance->Rotation), Instance->Location, (FVector)Instance->DrawScale3D);
				}
			}
		}
	}

	return Result;
}

TArray<FAnalyticsEventAttr> UUsdConversionBlueprintLibrary::GetAnalyticsAttributes(const ULevelExporterUSDOptions* Options)
{
	TArray<FAnalyticsEventAttr> Attrs;
	if (Options)
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		UsdUtils::AddAnalyticsAttributes(*Options, Attributes);

		Attrs.Reserve(Attributes.Num());
		for (const FAnalyticsEventAttribute& Attribute : Attributes)
		{
			FAnalyticsEventAttr& NewAttr = Attrs.Emplace_GetRef();
			NewAttr.Name = Attribute.GetName();
			NewAttr.Value = Attribute.GetValue();
		}
	}
	return Attrs;
}

void UUsdConversionBlueprintLibrary::SendAnalytics(
	const TArray<FAnalyticsEventAttr>& Attrs,
	const FString& EventName,
	bool bAutomated,
	double ElapsedSeconds,
	double NumberOfFrames,
	const FString& Extension
)
{
	TArray<FAnalyticsEventAttribute> Converted;
	Converted.Reserve(Attrs.Num());
	for (const FAnalyticsEventAttr& Attr : Attrs)
	{
		Converted.Emplace(Attr.Name, Attr.Value);
	}

	IUsdClassesModule::SendAnalytics(MoveTemp(Converted), EventName, bAutomated, ElapsedSeconds, NumberOfFrames, Extension);
}

void UUsdConversionBlueprintLibrary::BlockAnalyticsEvents()
{
	IUsdClassesModule::BlockAnalyticsEvents();
}

void UUsdConversionBlueprintLibrary::ResumeAnalyticsEvents()
{
	IUsdClassesModule::ResumeAnalyticsEvents();
}

void UUsdConversionBlueprintLibrary::BeginUniquePathScope()
{
	UsdUnreal::ExportUtils::BeginUniquePathScope();
}

void UUsdConversionBlueprintLibrary::EndUniquePathScope()
{
	UsdUnreal::ExportUtils::EndUniquePathScope();
}

FString UUsdConversionBlueprintLibrary::GetUniqueFilePathForExport(const FString& DesiredPathWithExtension)
{
	return UsdUnreal::ExportUtils::GetUniqueFilePathForExport(DesiredPathWithExtension);
}

void UUsdConversionBlueprintLibrary::RemoveAllPrimSpecs(const FString& StageRootLayer, const FString& PrimPath, const FString& TargetLayer)
{
#if USE_USD_SDK
	const bool bUseStageCache = true;
	UE::FUsdStage Stage = UnrealUSDWrapper::OpenStage(*StageRootLayer, EUsdInitialLoadSet::LoadAll, bUseStageCache);
	if (!Stage)
	{
		return;
	}

	UsdUtils::RemoveAllLocalPrimSpecs(Stage.GetPrimAtPath(UE::FSdfPath{*PrimPath}), UE::FSdfLayer::FindOrOpen(*TargetLayer));
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintLibrary::CutPrims(const FString& StageRootLayer, const TArray<FString>& PrimPaths)
{
#if USE_USD_SDK
	const bool bUseStageCache = true;
	UE::FUsdStage Stage = UnrealUSDWrapper::OpenStage(*StageRootLayer, EUsdInitialLoadSet::LoadAll, bUseStageCache);
	if (!Stage)
	{
		return false;
	}

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(PrimPaths.Num());

	for (const FString& PrimPath : PrimPaths)
	{
		Prims.Add(Stage.GetPrimAtPath(UE::FSdfPath{*PrimPath}));
	}

	return UsdUtils::CutPrims(Prims);
#else
	return false;
#endif	  // USE_USD_SDK
}

bool UUsdConversionBlueprintLibrary::CopyPrims(const FString& StageRootLayer, const TArray<FString>& PrimPaths)
{
#if USE_USD_SDK
	const bool bUseStageCache = true;
	UE::FUsdStage Stage = UnrealUSDWrapper::OpenStage(*StageRootLayer, EUsdInitialLoadSet::LoadAll, bUseStageCache);
	if (!Stage)
	{
		return false;
	}

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(PrimPaths.Num());

	for (const FString& PrimPath : PrimPaths)
	{
		Prims.Add(Stage.GetPrimAtPath(UE::FSdfPath{*PrimPath}));
	}

	return UsdUtils::CopyPrims(Prims);
#else
	return false;
#endif	  // USE_USD_SDK
}

TArray<FString> UUsdConversionBlueprintLibrary::PastePrims(const FString& StageRootLayer, const FString& ParentPrimPath)
{
	TArray<FString> Result;

#if USE_USD_SDK
	const bool bUseStageCache = true;
	UE::FUsdStage Stage = UnrealUSDWrapper::OpenStage(*StageRootLayer, EUsdInitialLoadSet::LoadAll, bUseStageCache);
	if (!Stage)
	{
		return Result;
	}

	TArray<UE::FSdfPath> PastedPrims = UsdUtils::PastePrims(Stage.GetPrimAtPath(UE::FSdfPath{*ParentPrimPath}));

	for (const UE::FSdfPath& DuplicatePrim : PastedPrims)
	{
		Result.Add(DuplicatePrim.GetString());
	}
#endif	  // USE_USD_SDK

	return Result;
}

bool UUsdConversionBlueprintLibrary::CanPastePrims()
{
	return UsdUtils::CanPastePrims();
}

void UUsdConversionBlueprintLibrary::ClearPrimClipboard()
{
	UsdUtils::ClearPrimClipboard();
}

TArray<FString> UUsdConversionBlueprintLibrary::DuplicatePrims(
	const FString& StageRootLayer,
	const TArray<FString>& PrimPaths,
	EUsdDuplicateType DuplicateType,
	const FString& TargetLayer
)
{
	TArray<FString> Result;
	Result.SetNum(PrimPaths.Num());

#if USE_USD_SDK
	const bool bUseStageCache = true;
	UE::FUsdStage Stage = UnrealUSDWrapper::OpenStage(*StageRootLayer, EUsdInitialLoadSet::LoadAll, bUseStageCache);
	if (!Stage)
	{
		return Result;
	}

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(PrimPaths.Num());

	for (const FString& PrimPath : PrimPaths)
	{
		Prims.Add(Stage.GetPrimAtPath(UE::FSdfPath{*PrimPath}));
	}

	TArray<UE::FSdfPath> DuplicatedPrims = UsdUtils::DuplicatePrims(Prims, DuplicateType, UE::FSdfLayer::FindOrOpen(*TargetLayer));

	for (int32 Index = 0; Index < DuplicatedPrims.Num(); ++Index)
	{
		const UE::FSdfPath& DuplicatePrim = DuplicatedPrims[Index];
		Result[Index] = DuplicatePrim.GetString();
	}
#endif	  // USE_USD_SDK

	return Result;
}

UUsdAssetUserData* UUsdConversionBlueprintLibrary::GetUsdAssetUserData(UObject* Object)
{
	return UsdUnreal::ObjectUtils::GetAssetUserData(Object);
}

bool UUsdConversionBlueprintLibrary::SetUsdAssetUserData(UObject* Object, UUsdAssetUserData* AssetUserData)
{
	return UsdUnreal::ObjectUtils::SetAssetUserData(Object, AssetUserData);
}

namespace UE::UsdConversionBlueprintLibrary::Private
{
	FEditPropertyChain& GetMetadataPropertyChain()
	{
		static TOptional<FEditPropertyChain> Chain;

		if (!Chain.IsSet())
		{
			FProperty* StageIdentifierProp = UUsdAssetUserData::StaticClass()->FindPropertyByName(
				GET_MEMBER_NAME_CHECKED(UUsdAssetUserData, StageIdentifierToMetadata)
			);
			FProperty* StringifiedValueProp = FUsdMetadataValue::StaticStruct()->FindPropertyByName(
				GET_MEMBER_NAME_CHECKED(FUsdMetadataValue, StringifiedValue)
			);

			Chain.Emplace();
			Chain->AddHead(StageIdentifierProp);
			Chain->AddTail(
				FUsdCombinedPrimMetadata::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUsdCombinedPrimMetadata, PrimPathToMetadata))
			);
			Chain->AddTail(FUsdPrimMetadata::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUsdPrimMetadata, Metadata)));
			Chain->AddTail(StringifiedValueProp);
			Chain->SetActivePropertyNode(StringifiedValueProp);
			Chain->SetActiveMemberPropertyNode(StageIdentifierProp);
		}

		return Chain.GetValue();
	}

	FPropertyChangedChainEvent GetPostChangeChainEvent(UObject* TopLevelObject, EPropertyChangeType::Type ChangeType)
	{
		FPropertyChangedEvent PostChangeEvent{
			FUsdMetadataValue::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUsdMetadataValue, StringifiedValue)),
			ChangeType,
			{TopLevelObject}
		};
		PostChangeEvent.MemberProperty = UUsdAssetUserData::StaticClass()->FindPropertyByName(
			GET_MEMBER_NAME_CHECKED(UUsdAssetUserData, StageIdentifierToMetadata)
		);

		FPropertyChangedChainEvent ChainEvent{GetMetadataPropertyChain(), PostChangeEvent};

		return ChainEvent;
	}
}	 // namespace UE::UsdConversionBlueprintLibrary::Private

bool UUsdConversionBlueprintLibrary::SetMetadataField(
	UUsdAssetUserData* AssetUserData,
	const FString& Key,
	const FString& Value,
	const FString& ValueTypeName,
	const FString& StageIdentifier,
	const FString& PrimPath,
	bool bTriggerPropertyChangeEvents
)
{
	using namespace UE::UsdConversionBlueprintLibrary::Private;

	if (!AssetUserData)
	{
		return false;
	}

	if (StageIdentifier.IsEmpty() && AssetUserData->StageIdentifierToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidStageIdentifier",
				"Failed to set metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the StageIdentifier parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return false;
	}

	const FString& StageIdentifierToUse = !StageIdentifier.IsEmpty() ? StageIdentifier
																	 : AssetUserData->StageIdentifierToMetadata.CreateIterator()->Key;
	FUsdCombinedPrimMetadata& CombinedPrimMetadata = AssetUserData->StageIdentifierToMetadata.FindOrAdd(StageIdentifierToUse);

	if (PrimPath.IsEmpty() && CombinedPrimMetadata.PrimPathToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidPrimPath",
				"Failed to set metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the PrimPath parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return false;
	}

	const FString& PrimPathToUse = !PrimPath.IsEmpty() ? PrimPath : CombinedPrimMetadata.PrimPathToMetadata.CreateConstIterator()->Key;
	FUsdPrimMetadata& PrimMetadata = CombinedPrimMetadata.PrimPathToMetadata.FindOrAdd(PrimPathToUse);

	if (bTriggerPropertyChangeEvents)
	{
		AssetUserData->PreEditChange(GetMetadataPropertyChain());
	}

	FUsdMetadataValue& MetadataValue = PrimMetadata.Metadata.FindOrAdd(Key);
	MetadataValue.StringifiedValue = Value;
	MetadataValue.TypeName = ValueTypeName;

	if (bTriggerPropertyChangeEvents)
	{
		FPropertyChangedChainEvent Event = GetPostChangeChainEvent(AssetUserData, EPropertyChangeType::ValueSet);
		AssetUserData->PostEditChangeChainProperty(Event);
	}

	return true;
}

bool UUsdConversionBlueprintLibrary::ClearMetadataField(
	UUsdAssetUserData* AssetUserData,
	const FString& Key,
	const FString& StageIdentifier,
	const FString& PrimPath,
	bool bTriggerPropertyChangeEvents
)
{
	using namespace UE::UsdConversionBlueprintLibrary::Private;

	if (!AssetUserData)
	{
		return false;
	}

	if (StageIdentifier.IsEmpty() && AssetUserData->StageIdentifierToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidStageIdentifierClear",
				"Failed to clear metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the StageIdentifier parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return false;
	}

	const FString& StageIdentifierToUse = !StageIdentifier.IsEmpty() ? StageIdentifier
																	 : AssetUserData->StageIdentifierToMetadata.CreateIterator()->Key;
	FUsdCombinedPrimMetadata& CombinedPrimMetadata = AssetUserData->StageIdentifierToMetadata.FindOrAdd(StageIdentifierToUse);

	if (PrimPath.IsEmpty() && CombinedPrimMetadata.PrimPathToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidPrimPathClear",
				"Failed to clear metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the PrimPath parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return false;
	}

	const FString& PrimPathToUse = !PrimPath.IsEmpty() ? PrimPath : CombinedPrimMetadata.PrimPathToMetadata.CreateConstIterator()->Key;
	FUsdPrimMetadata& PrimMetadata = CombinedPrimMetadata.PrimPathToMetadata.FindOrAdd(PrimPathToUse);

	if (bTriggerPropertyChangeEvents)
	{
		AssetUserData->PreEditChange(GetMetadataPropertyChain());
	}

	PrimMetadata.Metadata.Remove(Key);

	if (bTriggerPropertyChangeEvents)
	{
		FPropertyChangedChainEvent Event = GetPostChangeChainEvent(AssetUserData, EPropertyChangeType::ArrayRemove);
		AssetUserData->PostEditChangeChainProperty(Event);
	}

	return true;
}

bool UUsdConversionBlueprintLibrary::HasMetadataField(
	UUsdAssetUserData* AssetUserData,
	const FString& Key,
	const FString& StageIdentifier,
	const FString& PrimPath
)
{
	if (!AssetUserData)
	{
		return false;
	}

	if (StageIdentifier.IsEmpty() && AssetUserData->StageIdentifierToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidStageIdentifierCheck",
				"Failed to check for metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the StageIdentifier parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return false;
	}

	const FString& StageIdentifierToUse = !StageIdentifier.IsEmpty() ? StageIdentifier
																	 : AssetUserData->StageIdentifierToMetadata.CreateIterator()->Key;
	const FUsdCombinedPrimMetadata* CombinedPrimMetadata = AssetUserData->StageIdentifierToMetadata.Find(StageIdentifierToUse);
	if (!CombinedPrimMetadata)
	{
		return false;
	}

	if (PrimPath.IsEmpty() && CombinedPrimMetadata->PrimPathToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidPrimPathCheck",
				"Failed to clear metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the PrimPath parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return false;
	}

	const FString& PrimPathToUse = !PrimPath.IsEmpty() ? PrimPath : CombinedPrimMetadata->PrimPathToMetadata.CreateConstIterator()->Key;
	const FUsdPrimMetadata* PrimMetadata = CombinedPrimMetadata->PrimPathToMetadata.Find(PrimPathToUse);
	if (!PrimMetadata)
	{
		return false;
	}

	return PrimMetadata->Metadata.Contains(Key);
}

FUsdMetadataValue UUsdConversionBlueprintLibrary::GetMetadataField(
	UUsdAssetUserData* AssetUserData,
	const FString& Key,
	const FString& StageIdentifier,
	const FString& PrimPath
)
{
	if (!AssetUserData)
	{
		return {};
	}

	if (StageIdentifier.IsEmpty() && AssetUserData->StageIdentifierToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidStageIdentifierGet",
				"Failed to get metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the StageIdentifier parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return {};
	}

	const FString& StageIdentifierToUse = !StageIdentifier.IsEmpty() ? StageIdentifier
																	 : AssetUserData->StageIdentifierToMetadata.CreateIterator()->Key;
	const FUsdCombinedPrimMetadata* CombinedPrimMetadata = AssetUserData->StageIdentifierToMetadata.Find(StageIdentifierToUse);
	if (!CombinedPrimMetadata)
	{
		return {};
	}

	if (PrimPath.IsEmpty() && CombinedPrimMetadata->PrimPathToMetadata.Num() != 1)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"InvalidPrimPathGet",
				"Failed to get metadata field '{0}' on AssetUserData '{1}': Please provide a valid value for the PrimPath parameter"
			),
			FText::FromString(AssetUserData->GetPathName()),
			FText::FromString(Key)
		));
		return {};
	}

	const FString& PrimPathToUse = !PrimPath.IsEmpty() ? PrimPath : CombinedPrimMetadata->PrimPathToMetadata.CreateConstIterator()->Key;
	const FUsdPrimMetadata* PrimMetadata = CombinedPrimMetadata->PrimPathToMetadata.Find(PrimPathToUse);
	if (!PrimMetadata)
	{
		return {};
	}

	if (const FUsdMetadataValue* FoundValue = PrimMetadata->Metadata.Find(Key))
	{
		return *FoundValue;
	}

	USD_LOG_USERWARNING(FText::Format(
		LOCTEXT("FailedToFindMetadata", "Failed to find a metadata entry in '{0}' with stage identifier '{1}', prim path '{2}' and key '{3}'"),
		FText::FromString(AssetUserData->GetPathName()),
		FText::FromString(StageIdentifier),
		FText::FromString(PrimPath),
		FText::FromString(Key)
	));
	return {};
}

FString UUsdConversionBlueprintLibrary::StringifyAsBool(bool Value)
{
	return UsdUtils::StringifyAsBool(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsUChar(uint8 Value)
{
	return UsdUtils::StringifyAsUChar(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt(int32 Value)
{
	return UsdUtils::StringifyAsInt(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsUInt(int32 Value)
{
	// Have to cast as there are no unsigned types higher than uint8 on blueprint...
	return UsdUtils::StringifyAsUInt(static_cast<uint32>(Value));
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt64(int64 Value)
{
	return UsdUtils::StringifyAsInt64(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsUInt64(int64 Value)
{
	// Have to cast as there are no unsigned types higher than uint8 on blueprint...
	return UsdUtils::StringifyAsUInt64(static_cast<uint64>(Value));
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf(float Value)
{
	return UsdUtils::StringifyAsHalf(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat(float Value)
{
	return UsdUtils::StringifyAsFloat(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble(double Value)
{
	return UsdUtils::StringifyAsDouble(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsTimeCode(double Value)
{
	return UsdUtils::StringifyAsTimeCode(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsString(const FString& Value)
{
	return UsdUtils::StringifyAsString(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsToken(const FString& Value)
{
	return UsdUtils::StringifyAsToken(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsAssetPath(const FString& Value)
{
	return UsdUtils::StringifyAsAssetPath(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsMatrix2d(const FMatrix2D& Value)
{
	return UsdUtils::StringifyAsMatrix2d(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsMatrix3d(const FMatrix3D& Value)
{
	return UsdUtils::StringifyAsMatrix3d(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsMatrix4d(const FMatrix& Value)
{
	return UsdUtils::StringifyAsMatrix4d(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsQuatd(const FQuat& Value)
{
	return UsdUtils::StringifyAsQuatd(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsQuatf(const FQuat& Value)
{
	return UsdUtils::StringifyAsQuatf(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsQuath(const FQuat& Value)
{
	return UsdUtils::StringifyAsQuath(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble2(const FVector2D& Value)
{
	return UsdUtils::StringifyAsDouble2(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat2(const FVector2D& Value)
{
	return UsdUtils::StringifyAsFloat2(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf2(const FVector2D& Value)
{
	return UsdUtils::StringifyAsHalf2(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt2(const FIntPoint& Value)
{
	return UsdUtils::StringifyAsInt2(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble3(const FVector& Value)
{
	return UsdUtils::StringifyAsDouble3(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat3(const FVector& Value)
{
	return UsdUtils::StringifyAsFloat3(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf3(const FVector& Value)
{
	return UsdUtils::StringifyAsHalf3(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt3(const FIntVector& Value)
{
	return UsdUtils::StringifyAsInt3(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble4(const FVector4& Value)
{
	return UsdUtils::StringifyAsDouble4(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat4(const FVector4& Value)
{
	return UsdUtils::StringifyAsFloat4(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf4(const FVector4& Value)
{
	return UsdUtils::StringifyAsHalf4(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt4(const FIntVector4& Value)
{
	return UsdUtils::StringifyAsInt4(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsBoolArray(const TArray<bool>& Value)
{
	return UsdUtils::StringifyAsBoolArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsUCharArray(const TArray<uint8>& Value)
{
	return UsdUtils::StringifyAsUCharArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsIntArray(const TArray<int32>& Value)
{
	return UsdUtils::StringifyAsIntArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsUIntArray(const TArray<int32>& Value)
{
	static_assert(sizeof(int32) == sizeof(uint32));

	// Have to cast as there are no unsigned types higher than uint8 on blueprint...
	TArray<uint32> CastValue;
	CastValue.SetNumUninitialized(Value.Num());
	FMemory::Memcpy(CastValue.GetData(), Value.GetData(), Value.Num() * Value.GetTypeSize());

	return UsdUtils::StringifyAsUIntArray(CastValue);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt64Array(const TArray<int64>& Value)
{
	return UsdUtils::StringifyAsInt64Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsUInt64Array(const TArray<int64>& Value)
{
	static_assert(sizeof(int64) == sizeof(uint64));

	// Have to cast as there are no unsigned types higher than uint8 on blueprint...
	TArray<uint64> CastValue;
	CastValue.SetNumUninitialized(Value.Num());
	FMemory::Memcpy(CastValue.GetData(), Value.GetData(), Value.Num() * Value.GetTypeSize());

	return UsdUtils::StringifyAsUInt64Array(CastValue);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalfArray(const TArray<float>& Value)
{
	return UsdUtils::StringifyAsHalfArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloatArray(const TArray<float>& Value)
{
	return UsdUtils::StringifyAsFloatArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDoubleArray(const TArray<double>& Value)
{
	return UsdUtils::StringifyAsDoubleArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsTimeCodeArray(const TArray<double>& Value)
{
	return UsdUtils::StringifyAsTimeCodeArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsStringArray(const TArray<FString>& Value)
{
	return UsdUtils::StringifyAsStringArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsTokenArray(const TArray<FString>& Value)
{
	return UsdUtils::StringifyAsTokenArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsAssetPathArray(const TArray<FString>& Value)
{
	return UsdUtils::StringifyAsAssetPathArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsListOpTokens(const TArray<FString>& Value)
{
	return UsdUtils::StringifyAsListOpTokens(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsMatrix2dArray(const TArray<FMatrix2D>& Value)
{
	return UsdUtils::StringifyAsMatrix2dArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsMatrix3dArray(const TArray<FMatrix3D>& Value)
{
	return UsdUtils::StringifyAsMatrix3dArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsMatrix4dArray(const TArray<FMatrix>& Value)
{
	return UsdUtils::StringifyAsMatrix4dArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsQuatdArray(const TArray<FQuat>& Value)
{
	return UsdUtils::StringifyAsQuatdArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsQuatfArray(const TArray<FQuat>& Value)
{
	return UsdUtils::StringifyAsQuatfArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsQuathArray(const TArray<FQuat>& Value)
{
	return UsdUtils::StringifyAsQuathArray(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble2Array(const TArray<FVector2D>& Value)
{
	return UsdUtils::StringifyAsDouble2Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat2Array(const TArray<FVector2D>& Value)
{
	return UsdUtils::StringifyAsFloat2Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf2Array(const TArray<FVector2D>& Value)
{
	return UsdUtils::StringifyAsHalf2Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt2Array(const TArray<FIntPoint>& Value)
{
	return UsdUtils::StringifyAsInt2Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble3Array(const TArray<FVector>& Value)
{
	return UsdUtils::StringifyAsDouble3Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat3Array(const TArray<FVector>& Value)
{
	return UsdUtils::StringifyAsFloat3Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf3Array(const TArray<FVector>& Value)
{
	return UsdUtils::StringifyAsHalf3Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt3Array(const TArray<FIntVector>& Value)
{
	return UsdUtils::StringifyAsInt3Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsDouble4Array(const TArray<FVector4>& Value)
{
	return UsdUtils::StringifyAsDouble4Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsFloat4Array(const TArray<FVector4>& Value)
{
	return UsdUtils::StringifyAsFloat4Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsHalf4Array(const TArray<FVector4>& Value)
{
	return UsdUtils::StringifyAsHalf4Array(Value);
}

FString UUsdConversionBlueprintLibrary::StringifyAsInt4Array(const TArray<FIntVector4>& Value)
{
	return UsdUtils::StringifyAsInt4Array(Value);
}

namespace UE::UsdConversionLibrary::Private
{
	// It seems the most idiomatic behavior for stuff exposed to scripting is to always return a default value on failure
	// but to emit warnings in case anything went wrong, as dealing with error outputs in Blueprint can be tedious, and having
	// output parameters in Python feels weird. Here we use this adapter to convert the UsdUtils converters into signatures like that
	template<typename T, bool (*Unstringifier)(const FString&, T&)>
	T UnstringifyOrDefault(const FString& String, const FString& TypeName)
	{
		T Output{};

		bool bSuccess = Unstringifier(String, Output);
		if (!bSuccess)
		{
			USD_LOG_WARNING(TEXT("Failed to unstringify '%s' with the type '%s'!"), *String, *TypeName);
		}

		return Output;
	}
}

bool UUsdConversionBlueprintLibrary::UnstringifyAsBool(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<bool, UsdUtils::UnstringifyAsBool>(String, TEXT("bool"));
}

uint8 UUsdConversionBlueprintLibrary::UnstringifyAsUChar(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<uint8, UsdUtils::UnstringifyAsUChar>(String, TEXT("uchar"));
}

int32 UUsdConversionBlueprintLibrary::UnstringifyAsInt(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<int32, UsdUtils::UnstringifyAsInt>(String, TEXT("int"));
}

int32 UUsdConversionBlueprintLibrary::UnstringifyAsUInt(const FString& String)
{
	// Have to cast as there are no unsigned types higher than uint8 on blueprint...
	return static_cast<int32>(UE::UsdConversionLibrary::Private::UnstringifyOrDefault<uint32, UsdUtils::UnstringifyAsUInt>(String, TEXT("uint")));
}

int64 UUsdConversionBlueprintLibrary::UnstringifyAsInt64(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<int64, UsdUtils::UnstringifyAsInt64>(String, TEXT("int64"));
}

int64 UUsdConversionBlueprintLibrary::UnstringifyAsUInt64(const FString& String)
{
	// Have to cast as there are no unsigned types higher than uint8 on blueprint...
	return static_cast<int64>(UE::UsdConversionLibrary::Private::UnstringifyOrDefault<uint64, UsdUtils::UnstringifyAsUInt64>(String, TEXT("uint64")));
}

float UUsdConversionBlueprintLibrary::UnstringifyAsHalf(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<float, UsdUtils::UnstringifyAsHalf>(String, TEXT("half"));
}

float UUsdConversionBlueprintLibrary::UnstringifyAsFloat(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<float, UsdUtils::UnstringifyAsFloat>(String, TEXT("float"));
}

double UUsdConversionBlueprintLibrary::UnstringifyAsDouble(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<double, UsdUtils::UnstringifyAsDouble>(String, TEXT("double"));
}

double UUsdConversionBlueprintLibrary::UnstringifyAsTimeCode(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<double, UsdUtils::UnstringifyAsTimeCode>(String, TEXT("timecode"));
}

FString UUsdConversionBlueprintLibrary::UnstringifyAsString(const FString& String)
{
	return String;
}

FString UUsdConversionBlueprintLibrary::UnstringifyAsToken(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FString, UsdUtils::UnstringifyAsToken>(String, TEXT("token"));
}

FString UUsdConversionBlueprintLibrary::UnstringifyAsAssetPath(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FString, UsdUtils::UnstringifyAsAssetPath>(String, TEXT("asset"));
}

FMatrix2D UUsdConversionBlueprintLibrary::UnstringifyAsMatrix2d(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FMatrix2D, UsdUtils::UnstringifyAsMatrix2d>(String, TEXT("matrix2d"));
}

FMatrix3D UUsdConversionBlueprintLibrary::UnstringifyAsMatrix3d(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FMatrix3D, UsdUtils::UnstringifyAsMatrix3d>(String, TEXT("matrix3d"));
}

FMatrix UUsdConversionBlueprintLibrary::UnstringifyAsMatrix4d(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FMatrix, UsdUtils::UnstringifyAsMatrix4d>(String, TEXT("matrix4d"));
}

FQuat UUsdConversionBlueprintLibrary::UnstringifyAsQuatd(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FQuat, UsdUtils::UnstringifyAsQuatd>(String, TEXT("quatd"));
}

FQuat UUsdConversionBlueprintLibrary::UnstringifyAsQuatf(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FQuat, UsdUtils::UnstringifyAsQuatf>(String, TEXT("quatf"));
}

FQuat UUsdConversionBlueprintLibrary::UnstringifyAsQuath(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FQuat, UsdUtils::UnstringifyAsQuath>(String, TEXT("quath"));
}

FVector2D UUsdConversionBlueprintLibrary::UnstringifyAsDouble2(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector2D, UsdUtils::UnstringifyAsDouble2>(String, TEXT("double2"));
}

FVector2D UUsdConversionBlueprintLibrary::UnstringifyAsFloat2(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector2D, UsdUtils::UnstringifyAsFloat2>(String, TEXT("float2"));
}

FVector2D UUsdConversionBlueprintLibrary::UnstringifyAsHalf2(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector2D, UsdUtils::UnstringifyAsHalf2>(String, TEXT("half2"));
}

FIntPoint UUsdConversionBlueprintLibrary::UnstringifyAsInt2(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FIntPoint, UsdUtils::UnstringifyAsInt2>(String, TEXT("int2"));
}

FVector UUsdConversionBlueprintLibrary::UnstringifyAsDouble3(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector, UsdUtils::UnstringifyAsDouble3>(String, TEXT("double3"));
}

FVector UUsdConversionBlueprintLibrary::UnstringifyAsFloat3(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector, UsdUtils::UnstringifyAsFloat3>(String, TEXT("float3"));
}

FVector UUsdConversionBlueprintLibrary::UnstringifyAsHalf3(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector, UsdUtils::UnstringifyAsHalf3>(String, TEXT("half3"));
}

FIntVector UUsdConversionBlueprintLibrary::UnstringifyAsInt3(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FIntVector, UsdUtils::UnstringifyAsInt3>(String, TEXT("int3"));
}

FVector4 UUsdConversionBlueprintLibrary::UnstringifyAsDouble4(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector4, UsdUtils::UnstringifyAsDouble4>(String, TEXT("double4"));
}

FVector4 UUsdConversionBlueprintLibrary::UnstringifyAsFloat4(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector4, UsdUtils::UnstringifyAsFloat4>(String, TEXT("float4"));
}

FVector4 UUsdConversionBlueprintLibrary::UnstringifyAsHalf4(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FVector4, UsdUtils::UnstringifyAsHalf4>(String, TEXT("half4"));
}

FIntVector4 UUsdConversionBlueprintLibrary::UnstringifyAsInt4(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<FIntVector4, UsdUtils::UnstringifyAsInt4>(String, TEXT("int4"));
}

TArray<bool> UUsdConversionBlueprintLibrary::UnstringifyAsBoolArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<bool>, UsdUtils::UnstringifyAsBoolArray>(String, TEXT("bool[]"));
}

TArray<uint8> UUsdConversionBlueprintLibrary::UnstringifyAsUCharArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<uint8>, UsdUtils::UnstringifyAsUCharArray>(String, TEXT("uchar[]"));
}

TArray<int32> UUsdConversionBlueprintLibrary::UnstringifyAsIntArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<int32>, UsdUtils::UnstringifyAsIntArray>(String, TEXT("int[]"));
}

TArray<int32> UUsdConversionBlueprintLibrary::UnstringifyAsUIntArray(const FString& String)
{
	static_assert(sizeof(uint32) == sizeof(int32));

	TArray<uint32> Converted = UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<uint32>, UsdUtils::UnstringifyAsUIntArray>(
		String,
		TEXT("uint[]")
	);

	TArray<int32> Result;
	Result.SetNumUninitialized(Converted.Num());
	FMemory::Memcpy(Result.GetData(), Converted.GetData(), Converted.Num() * Converted.GetTypeSize());

	return Result;
}

TArray<int64> UUsdConversionBlueprintLibrary::UnstringifyAsInt64Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<int64>, UsdUtils::UnstringifyAsInt64Array>(String, TEXT("int64[]"));
}

TArray<int64> UUsdConversionBlueprintLibrary::UnstringifyAsUInt64Array(const FString& String)
{
	static_assert(sizeof(uint64) == sizeof(int64));

	TArray<uint64> Converted = UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<uint64>, UsdUtils::UnstringifyAsUInt64Array>(
		String,
		TEXT("uint64[]")
	);

	TArray<int64> Result;
	Result.SetNumUninitialized(Converted.Num());
	FMemory::Memcpy(Result.GetData(), Converted.GetData(), Converted.Num() * Converted.GetTypeSize());

	return Result;
}

TArray<float> UUsdConversionBlueprintLibrary::UnstringifyAsHalfArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<float>, UsdUtils::UnstringifyAsHalfArray>(String, TEXT("half[]"));
}

TArray<float> UUsdConversionBlueprintLibrary::UnstringifyAsFloatArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<float>, UsdUtils::UnstringifyAsFloatArray>(String, TEXT("float[]"));
}

TArray<double> UUsdConversionBlueprintLibrary::UnstringifyAsDoubleArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<double>, UsdUtils::UnstringifyAsDoubleArray>(String, TEXT("double[]"));
}

TArray<double> UUsdConversionBlueprintLibrary::UnstringifyAsTimeCodeArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<double>, UsdUtils::UnstringifyAsTimeCodeArray>(String, TEXT("timecode[]"));
}

TArray<FString> UUsdConversionBlueprintLibrary::UnstringifyAsStringArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FString>, UsdUtils::UnstringifyAsStringArray>(String, TEXT("string[]"));
}

TArray<FString> UUsdConversionBlueprintLibrary::UnstringifyAsTokenArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FString>, UsdUtils::UnstringifyAsTokenArray>(String, TEXT("token[]"));
}

TArray<FString> UUsdConversionBlueprintLibrary::UnstringifyAsAssetPathArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FString>, UsdUtils::UnstringifyAsAssetPathArray>(String, TEXT("asset[]"));
}

TArray<FString> UUsdConversionBlueprintLibrary::UnstringifyAsListOpTokens(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FString>, UsdUtils::UnstringifyAsListOpTokens>(
		String,
		TEXT("SdfListOp<Token>")
	);
}

TArray<FMatrix2D> UUsdConversionBlueprintLibrary::UnstringifyAsMatrix2dArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FMatrix2D>, UsdUtils::UnstringifyAsMatrix2dArray>(
		String,
		TEXT("matrix2d[]")
	);
}

TArray<FMatrix3D> UUsdConversionBlueprintLibrary::UnstringifyAsMatrix3dArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FMatrix3D>, UsdUtils::UnstringifyAsMatrix3dArray>(
		String,
		TEXT("matrix3d[]")
	);
}

TArray<FMatrix> UUsdConversionBlueprintLibrary::UnstringifyAsMatrix4dArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FMatrix>, UsdUtils::UnstringifyAsMatrix4dArray>(String, TEXT("matrix4d[]"));
}

TArray<FQuat> UUsdConversionBlueprintLibrary::UnstringifyAsQuatdArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FQuat>, UsdUtils::UnstringifyAsQuatdArray>(String, TEXT("quatd[]"));
}

TArray<FQuat> UUsdConversionBlueprintLibrary::UnstringifyAsQuatfArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FQuat>, UsdUtils::UnstringifyAsQuatfArray>(String, TEXT("quatf[]"));
}

TArray<FQuat> UUsdConversionBlueprintLibrary::UnstringifyAsQuathArray(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FQuat>, UsdUtils::UnstringifyAsQuathArray>(String, TEXT("quath[]"));
}

TArray<FVector2D> UUsdConversionBlueprintLibrary::UnstringifyAsDouble2Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector2D>, UsdUtils::UnstringifyAsDouble2Array>(String, TEXT("double2"));
}

TArray<FVector2D> UUsdConversionBlueprintLibrary::UnstringifyAsFloat2Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector2D>, UsdUtils::UnstringifyAsFloat2Array>(String, TEXT("float2"));
}

TArray<FVector2D> UUsdConversionBlueprintLibrary::UnstringifyAsHalf2Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector2D>, UsdUtils::UnstringifyAsHalf2Array>(String, TEXT("half2"));
}

TArray<FIntPoint> UUsdConversionBlueprintLibrary::UnstringifyAsInt2Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FIntPoint>, UsdUtils::UnstringifyAsInt2Array>(String, TEXT("int2"));
}

TArray<FVector> UUsdConversionBlueprintLibrary::UnstringifyAsDouble3Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector>, UsdUtils::UnstringifyAsDouble3Array>(String, TEXT("double3"));
}

TArray<FVector> UUsdConversionBlueprintLibrary::UnstringifyAsFloat3Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector>, UsdUtils::UnstringifyAsFloat3Array>(String, TEXT("float3"));
}

TArray<FVector> UUsdConversionBlueprintLibrary::UnstringifyAsHalf3Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector>, UsdUtils::UnstringifyAsHalf3Array>(String, TEXT("half3"));
}

TArray<FIntVector> UUsdConversionBlueprintLibrary::UnstringifyAsInt3Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FIntVector>, UsdUtils::UnstringifyAsInt3Array>(String, TEXT("int3"));
}

TArray<FVector4> UUsdConversionBlueprintLibrary::UnstringifyAsDouble4Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector4>, UsdUtils::UnstringifyAsDouble4Array>(String, TEXT("double4"));
}

TArray<FVector4> UUsdConversionBlueprintLibrary::UnstringifyAsFloat4Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector4>, UsdUtils::UnstringifyAsFloat4Array>(String, TEXT("float4"));
}

TArray<FVector4> UUsdConversionBlueprintLibrary::UnstringifyAsHalf4Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FVector4>, UsdUtils::UnstringifyAsHalf4Array>(String, TEXT("half4"));
}

TArray<FIntVector4> UUsdConversionBlueprintLibrary::UnstringifyAsInt4Array(const FString& String)
{
	return UE::UsdConversionLibrary::Private::UnstringifyOrDefault<TArray<FIntVector4>, UsdUtils::UnstringifyAsInt4Array>(String, TEXT("int4"));
}

#undef LOCTEXT_NAMESPACE
