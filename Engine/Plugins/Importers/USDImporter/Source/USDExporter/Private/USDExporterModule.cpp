// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDExporterModule.h"

#include "LevelExporterUSDOptionsCustomization.h"
#include "USDAssetOptions.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"

#include "UsdWrappers/SdfLayer.h"

#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Selection.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "USDExporterModule"

// Small customization to hide the "bExportComponentMetadata" property of shown FUsdMetadataExportOptions
// for the asset exporters, since the inverse of having it hidden by default and only shown for level/sequence exporters
// does not seem possible
class FHideExportComponentMetadataCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FHideExportComponentMetadataCustomization>();
	}
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder) override
	{
		if (TSharedPtr<IPropertyHandle> Property = DetailLayoutBuilder.GetProperty(TEXT("MetadataOptions.bExportComponentMetadata")))
		{
			DetailLayoutBuilder.HideProperty(Property);
		}
	}
};

class FUsdExporterModule : public IUsdExporterModule
{
public:
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Usd);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		// Hide the "bExportComponentMetadata" property if we're just exporting assets
		PropertyModule.RegisterCustomClassLayout(
			TEXT("AnimSequenceExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FHideExportComponentMetadataCustomization::MakeInstance)
		);
		PropertyModule.RegisterCustomClassLayout(
			TEXT("MaterialExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FHideExportComponentMetadataCustomization::MakeInstance)
		);
		PropertyModule.RegisterCustomClassLayout(
			TEXT("SkeletalMeshExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FHideExportComponentMetadataCustomization::MakeInstance)
		);
		PropertyModule.RegisterCustomClassLayout(
			TEXT("StaticMeshExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FHideExportComponentMetadataCustomization::MakeInstance)
		);
		PropertyModule.RegisterCustomClassLayout(
			TEXT("GeometryCacheExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FHideExportComponentMetadataCustomization::MakeInstance)
		);

		// We intentionally use the same customization for both of these
		PropertyModule.RegisterCustomClassLayout(
			TEXT("LevelExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FLevelExporterUSDOptionsCustomization::MakeInstance)
		);
		PropertyModule.RegisterCustomClassLayout(
			TEXT("LevelSequenceExporterUSDOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FLevelExporterUSDOptionsCustomization::MakeInstance)
		);

		// Modify the static mesh LOD range to have the proper define value as the maximum.
		// We have to do this the hard way here because we can't use the define within the meta tag itself
		if (UScriptStruct* ScriptStruct = FUsdMeshAssetOptions::StaticStruct())
		{
			if (FProperty* LowestMeshLODProperty = ScriptStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUsdMeshAssetOptions, LowestMeshLOD)))
			{
				LowestMeshLODProperty->SetMetaData(TEXT("ClampMax"), LexToString(MAX_MESH_LOD_COUNT - 1));
			}

			if (FProperty* HighestMeshLODProperty = ScriptStruct->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FUsdMeshAssetOptions, HighestMeshLOD)))
			{
				HighestMeshLODProperty->SetMetaData(TEXT("ClampMax"), LexToString(MAX_MESH_LOD_COUNT - 1));
			}
		}
	}

	virtual void ShutdownModule() override
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(TEXT("PropertyEditor")))
		{
			PropertyModule->UnregisterCustomClassLayout(TEXT("AnimSequenceExporterUSDOptions"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("MaterialExporterUSDOptions"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("SkeletalMeshExporterUSDOptions"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("StaticMeshExporterUSDOptions"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("GeometryCacheExporterUSDOptions"));

			PropertyModule->UnregisterCustomClassLayout(TEXT("LevelExporterUSDOptions"));
			PropertyModule->UnregisterCustomClassLayout(TEXT("LevelSequenceExporterUSDOptions"));
		}
	}
};

void IUsdExporterModule::HashEditorSelection(FSHA1& HashToUpdate)
{
	if (!GEditor)
	{
		return;
	}

	TSet<FString> SelectedPaths;

	USelection* ComponentSelection = GEditor->GetSelectedComponents();
	TArray<UActorComponent*> Components;
	ComponentSelection->GetSelectedObjects(Components);
	for (const UActorComponent* Component : Components)
	{
		if (Component)
		{
			SelectedPaths.Add(Component->GetFullName());
		}
	}

	USelection* ActorSelection = GEditor->GetSelectedActors();
	TArray<AActor*> Actors;
	ActorSelection->GetSelectedObjects(Actors);
	for (const AActor* Actor : Actors)
	{
		if (Actor)
		{
			SelectedPaths.Add(Actor->GetFullName());
		}
	}

	TArray<FString> SelectedPathsArray = SelectedPaths.Array();
	SelectedPathsArray.Sort();
	for (const FString& Path : SelectedPathsArray)
	{
		HashToUpdate.UpdateWithString(*Path, Path.Len());
	}
}

bool IUsdExporterModule::CanExportToLayer(const FString& TargetFilePath)
{
	for (const UE::FSdfLayerWeak& LoadedLayer : UE::FSdfLayerWeak::GetLoadedLayers())
	{
		if (FPaths::IsSamePath(LoadedLayer.GetRealPath(), TargetFilePath))
		{
			FText ErrorMessage = FText::Format(
				LOCTEXT(
					"FailedExportLayerOpenSubText",
					"Failed to export to the USD layer '{0}' as a layer with this identifier is already open in another USD Stage. Please use a different file path or try closing that other USD stage and exporting again."
				),
				FText::FromString(TargetFilePath)
			);
			USD_LOG_USERERROR(ErrorMessage);

			FNotificationInfo ErrorToast(LOCTEXT("FailedExportLayerOpenText", "USD: Export failure"));
			ErrorToast.ExpireDuration = 10.0f;
			ErrorToast.bFireAndForget = true;
			ErrorToast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
			ErrorToast.SubText = ErrorMessage;
			FSlateNotificationManager::Get().AddNotification(ErrorToast);

			return false;
		}
	}

	return true;
}

IMPLEMENT_MODULE_USD(FUsdExporterModule, USDExporter);

#undef LOCTEXT_NAMESPACE
