// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDMetadataExportOptions.h"

#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDMetadataExportOptions)

void UsdUtils::AddAnalyticsAttributes(const FUsdMetadataExportOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("ExportAssetInfo"), Options.bExportAssetInfo);
	InOutAttributes.Emplace(TEXT("ExportAssetMetadata"), Options.bExportAssetMetadata);
	InOutAttributes.Emplace(TEXT("ExportComponentMetadata"), Options.bExportComponentMetadata);

	FString FilterString = TEXT("[");
	for (const FString& Filter : Options.BlockedPrefixFilters)
	{
		FilterString += Filter + TEXT(", ");
	}
	FilterString.RemoveFromEnd(TEXT(", "));
	FilterString += TEXT("]");
	InOutAttributes.Emplace(TEXT("BlockedPrefixFilters"), FilterString);

	InOutAttributes.Emplace(TEXT("InvertMetadataFilter"), LexToString(Options.bInvertFilters));
}

void UsdUtils::HashForExport(const FUsdMetadataExportOptions& Options, FSHA1& HashToUpdate)
{
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportAssetInfo), sizeof(Options.bExportAssetInfo));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportAssetMetadata), sizeof(Options.bExportAssetMetadata));
	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bExportComponentMetadata), sizeof(Options.bExportComponentMetadata));

	for (const FString& Filter : Options.BlockedPrefixFilters)
	{
		HashToUpdate.UpdateWithString(*Filter, Filter.Len());
	}

	HashToUpdate.Update(reinterpret_cast<const uint8*>(&Options.bInvertFilters), sizeof(Options.bInvertFilters));
}
