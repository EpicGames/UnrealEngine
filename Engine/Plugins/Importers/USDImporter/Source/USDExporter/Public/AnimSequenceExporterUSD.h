// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Exporters/Exporter.h"
#include "UObject/ObjectMacros.h"

#include "AnimSequenceExporterUSD.generated.h"

UCLASS()
class UAnimSequenceExporterUSD : public UExporter
{
	GENERATED_BODY()

public:
	UAnimSequenceExporterUSD();

	//~ Begin UExporter Interface
	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0)
		override;
	//~ End UExporter Interface
};
