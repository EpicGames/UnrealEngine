// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDAssetOptions.h"
#include "USDMetadataExportOptions.h"
#include "USDStageOptions.h"

#include "AnimSequenceExporterUSDOptions.generated.h"

struct FAnalyticsEventAttribute;

/**
 * Options for exporting skeletal mesh animations to USD format.
 */
UCLASS(MinimalAPI, Config = Editor, Blueprintable)
class UAnimSequenceExporterUSDOptions : public UObject
{
	GENERATED_BODY()

public:
	/** Export options to use for the layer where the animation is emitted */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options", meta = (ShowOnlyInnerProperties))
	FUsdStageOptions StageOptions;

	/** Whether to also export the skeletal mesh data of the preview mesh */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options")
	bool bExportPreviewMesh;

	/** Export options to use for the preview mesh, if enabled */
	UPROPERTY(
		EditAnywhere,
		config,
		BlueprintReadWrite,
		Category = "Mesh options",
		meta = (ShowOnlyInnerProperties, EditCondition = bExportPreviewMesh)
	)
	FUsdMeshAssetOptions PreviewMeshOptions;

	/** Options to use when exporting UsdAssetUserData metadata to USD for all assets (including the preview mesh) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Metadata options", meta = (ShowOnlyInnerProperties))
	FUsdMetadataExportOptions MetadataOptions;

	/**
	 * Whether to export any asset (StaticMesh, Material, etc.) even if the existing file already describes the same version of a compatible asset.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Assets"))
	bool bReExportIdenticalAssets = false;
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const UAnimSequenceExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void HashForAnimSequenceExport(const UAnimSequenceExporterUSDOptions& Options, FSHA1& HashToUpdate);
}
