// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "USDAssetOptions.h"
#include "USDMetadataExportOptions.h"
#include "USDStageOptions.h"

#include "AssetExportTask.h"

#include "LevelExporterUSDOptions.generated.h"

#define UE_API USDEXPORTER_API

struct FAnalyticsEventAttribute;

USTRUCT(BlueprintType)
struct FLevelExporterUSDOptionsInner
{
	GENERATED_BODY()

	/** Name of the single top-level prim that is emitted when exporting Levels and LevelSequences to USD */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Export options")
	FString RootPrimName = TEXT("Root");

	/** Whether to export only the selected actors, and assets used by them */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Export options")
	bool bSelectionOnly = false;

	/** Whether to use UE actor folders as empty prims */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Export options")
	bool bExportActorFolders = false;

	/** If true, and if we have a level sequence animating the level during export, it will revert any actor or component to its unanimated state
	 * before writing to USD */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Export options")
	bool bIgnoreSequencerAnimations = false;

	/**
	 * By default foliage instances will be exported to the same layer as the component they were placed on in the editor.
	 * Enable this to instead export the foliage instances to the same layer as the foliage actor they belong to.
	 * This is useful if those foliage instances were placed with the "Place In Current Level" option.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Export options")
	bool bExportFoliageOnActorsLayer = false;

	/** Where to place all the generated asset files */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Asset options")
	FDirectoryPath AssetFolder;

	/**
	 * Export the asset files in a folder structure that mirrors their Unreal project content folder structure.
	 * Warning: This can produce very long file paths that could cause issues depending on the operating system settings.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Asset options", AdvancedDisplay)
	bool bExportToNestedSubFolders = false;

	/** Options to use for all exported assets when appropriate (e.g. static and skeletal meshes, materials, etc.) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Asset options", meta = (ShowOnlyInnerProperties))
	FUsdMeshAssetOptions AssetOptions;

	/** Options to use when exporting UsdAssetUserData metadata to USD for all assets (including the level itself) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Metadata options", meta = (ShowOnlyInnerProperties))
	FUsdMetadataExportOptions MetadataOptions;

	/** Lowest of the LOD indices to export landscapes with (use 0 for full resolution) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Landscape options", meta = (ClampMin = "0"))
	int32 LowestLandscapeLOD = 0;

	/**
	 * Highest of the LOD indices to export landscapes with. Each value above 0 halves resolution.
	 * The max value depends on the number of components and sections per component of each landscape, and may be clamped.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Landscape options", meta = (ClampMin = "0"))
	int32 HighestLandscapeLOD = 0;

	/** Resolution to use when baking landscape materials into textures  */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Landscape options", meta = (ClampMin = "1"))
	FIntPoint LandscapeBakeResolution = FIntPoint(1024, 1024);

	/** If true, will export sub-levels as separate layers (referenced as sublayers). If false, will collapse all sub-levels in a single exported root
	 * layer */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Sublayers", meta = (DisplayName = "Export Sublevels as Layers"))
	bool bExportSublayers = false;

	/** Names of levels that should be ignored when collecting actors to export (e.g. "Persistent Level", "Level1", "MySubLevel", etc.) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Sublayers")
	TSet<FString> LevelsToIgnore;
};

/**
 * Options for exporting levels to USD format.
 */
UCLASS(MinimalAPI, Config = Editor, Blueprintable, HideCategories = Hidden)
class ULevelExporterUSDOptions : public UObject
{
	GENERATED_BODY()

public:
	/** Basic options about the stage to export */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options", meta = (ShowOnlyInnerProperties))
	FUsdStageOptions StageOptions;

	/** StartTimeCode to be used for all exported layers */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options")
	float StartTimeCode = 0.0f;

	/** EndTimeCode to be used for all exported layers */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options")
	float EndTimeCode = 0.0f;

	/** Inner struct that actually contains most of the export options */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Export settings", meta = (ShowOnlyInnerProperties))
	FLevelExporterUSDOptionsInner Inner;

	/**
	 * Whether to export levels and LevelSequences even if the existing files already describe the same versions of compatible assets.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Levels and Sequences"))
	bool bReExportIdenticalLevelsAndSequences = false;

	/**
	 * Whether to export any asset (StaticMesh, Material, etc.) even if the existing file already describes the same version of a compatible asset.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Assets"))
	bool bReExportIdenticalAssets = false;

public:
	// We temporarily stash our export task here as a way of passing our options down to
	// the Python exporter, that does the actual level exporting.
	// This is weak because we often use the CDO of this class directly, and we never want to
	// permanently hold on to a particular export task
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Hidden)
	TWeakObjectPtr<UAssetExportTask> CurrentTask;

private:
	UFUNCTION()
	static UE_API TArray<FString> GetUsdExtensions();
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const FLevelExporterUSDOptionsInner& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void AddAnalyticsAttributes(const ULevelExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void HashForLevelExport(const FLevelExporterUSDOptionsInner& Options, FSHA1& HashToUpdate);

	USDEXPORTER_API void HashForLevelExport(const ULevelExporterUSDOptions& Options, FSHA1& HashToUpdate);
}

#undef UE_API
