// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/Exporter.h"

#include "LevelSequenceExporterUSD.generated.h"

UCLASS()
class ULevelSequenceExporterUsd : public UExporter
{
	GENERATED_BODY()

public:
	ULevelSequenceExporterUsd();

	//~ Begin UExporter Interface
	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0)
		override;
	//~ End UExporter Interface
};
