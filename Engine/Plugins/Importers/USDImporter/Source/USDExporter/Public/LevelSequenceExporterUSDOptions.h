// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelExporterUSDOptions.h"
#include "UnrealUSDWrapper.h"
#include "USDStageOptions.h"

#include "Engine/EngineTypes.h"

#include "LevelSequenceExporterUSDOptions.generated.h"

struct FAnalyticsEventAttribute;

/**
 * Options for level sequences to the USD format.
 */
UCLASS(MinimalAPI, Config = Editor, Blueprintable)
class ULevelSequenceExporterUsdOptions : public UObject
{
	GENERATED_BODY()

public:
	/** Export options to use for the layer where the animation is emitted */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = USDSettings, meta = (ShowOnlyInnerProperties))
	FUsdStageOptions StageOptions;

	/**
	 * Value to bake all generated USD layers with
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence")
	float TimeCodesPerSecond = 24.0f;

	/**
	 * If checked will cause StartFrame and EndFrame to be used as the frame range for the level sequence export.
	 * If unchecked the existing playback range of each level sequence will be used instead.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence")
	bool bOverrideExportRange = false;

	/** Initial frame of the level sequence to bake out to USD (inclusive) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence", meta = (EditCondition = "bOverrideExportRange"))
	int32 StartFrame = 0;

	/** Final frame of the level sequence to bake out to USD (inclusive) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence", meta = (EditCondition = "bOverrideExportRange"))
	int32 EndFrame = 0;

	/**
	 * Whether to export animations exclusively from selected actors or components
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence")
	bool bSelectionOnly = false;

	/**
	 * The effect of subsequences is always included on the main exported layer, but if this option is true we will also
	 * export individual sublayers for each subsequence, so that they can be used by themselves if needed.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence")
	bool bExportSubsequencesAsLayers = true;

	/**
	 * Exports the provided level alongside the layer that represents the exported Level Sequence
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence")
	bool bExportLevel = true;

	/**
	 * If checked this will also add the exported level as a sublayer to the USD files emitted for all exported level sequences
	 */
	UPROPERTY(
		EditAnywhere,
		config,
		BlueprintReadWrite,
		Category = "Level Sequence",
		meta = (EditCondition = "bExportLevel", DisplayName = "Use Exported Level as USD Sublayer")
	)
	bool bUseExportedLevelAsSublayer = true;

	/**
	 * If you have multiple instances of a same LevelSequence subsection with spawnables, setting this to true means each separate
	 * subsection instance will get its own individual copy of the prim exported for the spawnable. Setting this to false means that
	 * only a single prim will be exported for that spawnable, and shared for all subsection instances.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Sequence")
	bool bExportSeparatePrimsPerSpawnableInstance = true;

	/**
	 * Whether to export levels and LevelSequences even if the existing files already describe the same versions of compatible assets.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Levels and Sequences"))
	bool bReExportIdenticalLevelsAndSequences = false;

	/**
	 * Whether to export any asset (StaticMesh, Material, etc.) even if the existing file already describes the same version of a compatible asset.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Assets"))
	bool bReExportIdenticalAssets = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Level Export", meta = (EditCondition = "bExportLevel"))
	TWeakObjectPtr<UWorld> Level = nullptr;

	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Level Export", meta = (EditCondition = "bExportLevel"))
	FLevelExporterUSDOptionsInner LevelExportOptions;
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const ULevelSequenceExporterUsdOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void HashForLevelSequenceExport(const ULevelSequenceExporterUsdOptions& Options, FSHA1& HashToUpdate);
}
