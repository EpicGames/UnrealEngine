// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/Exporter.h"

#include "MaterialExporterUSD.generated.h"

class UMaterialInterface;
struct FUsdMaterialBakingOptions;
struct FUsdMetadataExportOptions;

UCLASS()
class UMaterialExporterUsd : public UExporter
{
	GENERATED_BODY()

public:
	UMaterialExporterUsd();

	//~ Begin UExporter Interface
	USDEXPORTER_API virtual bool ExportBinary(
		UObject* Object,
		const TCHAR* Type,
		FArchive& Ar,
		FFeedbackContext* Warn,
		int32 FileIndex = 0,
		uint32 PortFlags = 0
	) override;
	//~ End UExporter Interface

	/**
	 * Exports a single material with given options to the target filepath as a UsdPreviewSurface USD asset.
	 * @param Material - Material to bake
	 * @param Options - Options to use when baking
	 * @param FilePath - Filepath of the destination file (e.g. "C:/MyFolder/Red.usda")
	 * @param ReplaceIdentical - Whether to overwrite the destination file in case it already exists
	 * @param bReExportIdenticalAssets - Whether to overwrite the destination file even if it already describes an up-to-date version of Material
	 * @param bIsAutomated - Whether the export is being done by a script or not. Just used for analytics
	 * @return Whether the export was successful or not.
	 */
	USDEXPORTER_API static bool ExportMaterial(
		const UMaterialInterface& Material,
		const FUsdMaterialBakingOptions& Options,
		const FUsdMetadataExportOptions& MetadataOptions,
		const FFilePath& FilePath,
		bool bReplaceIdentical = true,
		bool bReExportIdenticalAssets = false,
		bool bIsAutomated = false
	);

	/**
	 * Exports the provided materials next to the stage's root layer on disk, replacing usages of unrealMaterials within `Stage` with references
	 * to the exported UsdPreviewMaterial assets.
	 * @param Materials - Materials to bake
	 * @param Options - Options to use when baking
	 * @param StageRootLayerPath - Path to the stage to replace the unrealMaterials attributes in.
	 *                             Note that this stage must not be opened in case you want this function to be able to
	 *                             replace exported materials with new files.
	 *                             All of it's layers will be traversed.
	 *                             The root layer folder on disk is used as destination folder for the baked materials
	 * @param bIsAssetLayer - True when we're exporting a single mesh/animation asset. False when we're exporting a level. Dictates minor behaviors
	 *                        when authoring the material binding relationships, e.g. whether we author them inside variants or not
	 * @param bUsePayload - Should be True if the Stage was exported using payload files to store the actual Mesh prims. Also dictates minor
	 *                      behaviors when authoring the material binding relationships.
	 * @param ReplaceIdentical - Whether to overwrite the destination files in case they already exist
	 * @param bReExportIdenticalAssets - Whether to overwrite the destination files even if they already describe up-to-date versions of Materials
	 * @param bIsAutomated - Whether the export is being done by a script or not. Just used for analytics
	 * @return Whether the export was successful or not.
	 */
	USDEXPORTER_API static bool ExportMaterialsForStage(
		const TArray<UMaterialInterface*>& Materials,
		const FUsdMaterialBakingOptions& Options,
		const FUsdMetadataExportOptions& MetadataOptions,
		const FString& StageRootLayerPath,
		bool bIsAssetLayer,
		bool bUsePayload,
		bool bReplaceIdentical = true,
		bool bReExportIdenticalAssets = false,
		bool bIsAutomated = false
	);
};
