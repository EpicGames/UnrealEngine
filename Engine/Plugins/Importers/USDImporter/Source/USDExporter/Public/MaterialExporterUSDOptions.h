// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDAssetOptions.h"
#include "USDMetadataExportOptions.h"

#include "MaterialExporterUSDOptions.generated.h"

struct FAnalyticsEventAttribute;

/**
 * Options for exporting materials to USD format.
 * We use a dedicated object instead of reusing the MaterialBaking module as automated export tasks
 * can only have one options object, and we need to also provide the textures directory.
 */
UCLASS(MinimalAPI, Config = Editor, Blueprintable)
class UMaterialExporterUSDOptions : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Material baking options", meta = (ShowOnlyInnerProperties))
	FUsdMaterialBakingOptions MaterialBakingOptions;

	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Metadata options", meta = (ShowOnlyInnerProperties))
	FUsdMetadataExportOptions MetadataOptions;

	/**
	 * Whether to export any asset (StaticMesh, Material, etc.) even if the existing file already describes the same version of a compatible asset.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Assets"))
	bool bReExportIdenticalAssets = false;
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const UMaterialExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void HashForMaterialExport(const UMaterialExporterUSDOptions& Options, FSHA1& HashToUpdate);
}
