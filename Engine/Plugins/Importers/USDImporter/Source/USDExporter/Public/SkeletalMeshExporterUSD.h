// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/Exporter.h"

#include "SkeletalMeshExporterUSD.generated.h"

UCLASS()
class USkeletalMeshExporterUsd : public UExporter
{
	GENERATED_BODY()

public:
	USkeletalMeshExporterUsd();

	//~ Begin UExporter Interface
	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0)
		override;
	//~ End UExporter Interface
};
