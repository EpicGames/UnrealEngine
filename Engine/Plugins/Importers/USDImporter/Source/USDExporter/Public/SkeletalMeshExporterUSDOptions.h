// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDAssetOptions.h"
#include "USDMetadataExportOptions.h"
#include "USDStageOptions.h"

#include "SkeletalMeshExporterUSDOptions.generated.h"

struct FAnalyticsEventAttribute;

/**
 * Options for exporting skeletal meshes to USD format.
 */
UCLASS(MinimalAPI, Config = Editor, Blueprintable)
class USkeletalMeshExporterUSDOptions : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Stage options", meta = (ShowOnlyInnerProperties))
	FUsdStageOptions StageOptions;

	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options", meta = (ShowOnlyInnerProperties))
	FUsdMeshAssetOptions MeshAssetOptions;

	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Metadata options", meta = (ShowOnlyInnerProperties))
	FUsdMetadataExportOptions MetadataOptions;

	/**
	 * Whether to export any asset (StaticMesh, Material, etc.) even if the existing file already describes the same version of a compatible asset.
	 * This is only checked when bReplaceIdentical is set on the asset export task. Otherwise we'll never overwrite files.
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Collision", meta = (DisplayName = "Re-export Identical Assets"))
	bool bReExportIdenticalAssets = false;
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const USkeletalMeshExporterUSDOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void HashForSkeletalMeshExport(const USkeletalMeshExporterUSDOptions& Options, FSHA1& HashToUpdate);
}
