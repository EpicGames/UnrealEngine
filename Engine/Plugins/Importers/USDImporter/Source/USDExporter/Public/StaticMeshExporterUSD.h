// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/Exporter.h"

#include "StaticMeshExporterUSD.generated.h"

UCLASS()
class UStaticMeshExporterUsd : public UExporter
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "USD")
	static bool IsUsdAvailable();

public:
	UStaticMeshExporterUsd();

	//~ Begin UExporter Interface
	virtual bool ExportBinary(UObject* Object, const TCHAR* Type, FArchive& Ar, FFeedbackContext* Warn, int32 FileIndex = 0, uint32 PortFlags = 0)
		override;
	//~ End UExporter Interface
};
