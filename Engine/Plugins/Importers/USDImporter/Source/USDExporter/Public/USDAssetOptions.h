// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialOptions.h"

#include "RHIDefinitions.h"

#include "USDAssetOptions.generated.h"

struct FAnalyticsEventAttribute;

USTRUCT(BlueprintType)
struct FUsdMaterialBakingOptions
{
	GENERATED_BODY()

	/** Properties which are supposed to be baked out for the material */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Material baking options")
	TArray<FPropertyEntry> Properties = {
		MP_BaseColor,
		MP_Metallic,
		MP_Specular,
		MP_Roughness,
		MP_Anisotropy,
		MP_EmissiveColor,
		MP_Opacity,
		MP_OpacityMask,
		MP_Normal,
		MP_Tangent,
		MP_SubsurfaceColor,
		MP_AmbientOcclusion};

	/** Size of the baked texture for all properties that don't have a CustomSize set */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Material baking options", meta = (ClampMin = "1", UIMin = "1"))
	FIntPoint DefaultTextureSize = FIntPoint{128, 128};

	/** When this is true and a baked texture contains a single flat color we will write out that color value directly on the USD layer and skip
	 * generating a texture */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material baking options")
	bool bConstantColorAsSingleValue = true;

	/** Where baked textures are placed. Intentionally not a config as it's heavily dependent on where the stage is */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Material baking options")
	FDirectoryPath TexturesDir;
};

USTRUCT(BlueprintType)
struct FUsdMeshAssetOptions
{
	GENERATED_BODY()

	/** If true, the mesh data is exported to yet another "payload" file, and referenced via a payload composition arc */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options")
	bool bUsePayload = false;

	/** USD format to use for exported payload files */
	UPROPERTY(
		EditAnywhere,
		config,
		BlueprintReadWrite,
		Category = "Mesh options",
		meta = (EditCondition = bUsePayload, GetOptions = "USDExporter.LevelExporterUSDOptions.GetUsdExtensions")
	)
	FString PayloadFormat = TEXT("usdc");

	/** Whether to bake the mesh's assigned material and export these as separate UsdPreviewSurface assets */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Material options")
	bool bBakeMaterials = false;

	/** Whether to remove the 'unrealMaterial' attribute after binding the corresponding baked material */
	UE_DEPRECATED(all, "This option is now deprecated as UE material assignments are only visible in the 'unreal' render context anyway")
	UPROPERTY(config, meta = (EditCondition = bBakeMaterials))
	bool bRemoveUnrealMaterials_DEPRECATED = false;

	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Material options", meta = (EditCondition = bBakeMaterials))
	FUsdMaterialBakingOptions MaterialBakingOptions;

	/**
	 * If true it means we'll try exporting the source data when exporting static meshes.
	 * Not all meshes contain their source data though. If we fail to find any source data, we'll fall back to exporting the render data instead.
	 *
	 * If false it means we'll export the render data instead: This can be closer to what is seen on the viewport, but it can also
	 * be the most processed. For example, if the mesh has Nanite enabled this render data may be a much coarser version of the
	 * source mesh.
	 *
	 * This is only relevant for StaticMeshes, and has no effect on other mesh types (SkeletalMeshes, GeometryCaches, etc.)
	 */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options")
	bool bExportStaticMeshSourceData = true;

	/** Whether to convert skeletal data to non-skeletal, ie. skeletal mesh to static mesh, anim sequence to animated mesh */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options")
	bool bConvertSkeletalToNonSkeletal = false;

	/** Lowest of the LOD indices to export static and skeletal meshes with (use 0 for full detail) */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options", meta = (ClampMin = "0"))
	int32 LowestMeshLOD = 0;

	/** Highest of the LOD indices to export static and skeletal meshes with */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Mesh options", meta = (ClampMin = "0"))
	int32 HighestMeshLOD = MAX_MESH_LOD_COUNT - 1;

	// We need this as clang would otherwise emit some warnings when generating these due to the usages of bRemoveUnrealMaterials
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FUsdMeshAssetOptions() = default;
	FUsdMeshAssetOptions(const FUsdMeshAssetOptions&) = default;
	FUsdMeshAssetOptions& operator=(const FUsdMeshAssetOptions&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const FUsdMaterialBakingOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void AddAnalyticsAttributes(const FUsdMeshAssetOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);

	USDEXPORTER_API void HashForMaterialExport(const FUsdMaterialBakingOptions& Options, FSHA1& HashToUpdate);

	USDEXPORTER_API void HashForMeshExport(const FUsdMeshAssetOptions& Options, FSHA1& HashToUpdate);
}
