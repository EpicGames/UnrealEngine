// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnrealUSDWrapper.h"
#include "USDAssetUserData.h"
#include "USDStageOptions.h"
#include "USDUnrealAssetInfo.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"

#include "Engine/EngineTypes.h"
#include "MaterialOptions.h"

#include "USDConversionBlueprintContext.generated.h"

#define UE_API USDEXPORTER_API

class AInstancedFoliageActor;
class ALandscapeProxy;
class UAudioComponent;
class UCineCameraComponent;
class UDirectionalLightComponent;
class UInstancedStaticMeshComponent;
class UHierarchicalInstancedStaticMeshComponent;
class ULevelExporterUSDOptions;
class ULightComponentBase;
class UMeshComponent;
class UPointLightComponent;
class URectLightComponent;
class USceneComponent;
class USkyLightComponent;
class USpotLightComponent;
class UUsdDrawModeComponent;
struct FUsdCombinedPrimMetadata;
struct FUsdPrimMetadata;

/**
 * Wraps the UnrealToUsd component conversion functions from the USDUtilities module so that they can be used by
 * scripting languages.
 *
 * This is an instanceable object instead of just static functions so that the USDStage to use for the
 * conversions can be provided and cached between function calls, which is helpful because we're forced to provide
 * at most prim and layer file paths (as opposed to direct pxr::UsdPrim objects).
 *
 * We can't provide the pxr::UsdPrim object directly because USD types can't be part of C++ function signatures that
 * are automatically exposed to scripting languages. Lucikly we can use UsdUtils' stage cache to make sure that
 * C++ and e.g. Python are still referencing the same USD Stage in memory, so that we can e.g. use these functions to
 * convert data within stages created via Python.
 */
UCLASS(MinimalAPI, meta = (ScriptName = "UsdConversionContext"))
class UUsdConversionBlueprintContext : public UObject
{
	GENERATED_BODY()

	UE_API virtual ~UUsdConversionBlueprintContext();

private:
	/** Stage to use when converting components */
	UE::FUsdStage Stage;

	/**
	 * Whether we will erase our current stage from the stage cache when we Cleanup().
	 * This is true if we were the ones that put the stage in the cache in the first place.
	 */
	bool bEraseFromStageCache = false;

public:
	/**
	 * Opens or creates a USD stage using `StageRootLayerPath` as root layer, creating the root layer if needed.
	 * All future conversions will fetch prims and get/set USD data to/from this stage.
	 * Note: You must remember to call Cleanup() when done, or else this object will permanently hold a reference to the opened stage!
	 */
	UFUNCTION(BlueprintCallable, Category = "Export context")
	UE_API void SetStageRootLayer(FFilePath StageRootLayerPath);

	/**
	 * Gets the file path of the root layer of our current stage
	 */
	UFUNCTION(BlueprintCallable, Category = "Export context")
	UE_API FFilePath GetStageRootLayer();

	/**
	 * Sets the current edit target of our internal stage. When calling the conversion functions, prims and attributes
	 * will be authored on this edit target only
	 */
	UFUNCTION(BlueprintCallable, Category = "Export context")
	UE_API void SetEditTarget(FFilePath EditTargetLayerPath);

	/**
	 * Gets the filepath of the current edit target layer of our internal stage
	 */
	UFUNCTION(BlueprintCallable, Category = "Export context")
	UE_API FFilePath GetEditTarget();

	/**
	 * Discards the currently opened stage. This is critical when using this class via scripting: The C++ destructor will
	 * not be called when the python object runs out of scope, so we would otherwise keep a strong reference to the stage
	 */
	UFUNCTION(BlueprintCallable, Category = "Export context")
	UE_API void Cleanup();

public:
	// Note: We use FLT_MAX on these functions because Usd.TimeCode.Default().GetValue() is actually a nan, and nan arguments are automatically
	// sanitized to 0.0f. We manually convert the FLT_MAX value into Usd.TimeCode.Default().GetValue() within the functions though, so if you want the
	// Default timecode just omit the argument We are also forced to copypaste the FLT_MAX value (3.402823466e+38F) in here as the default arguments
	// are parsed before the preprocessor replaces the defines

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertLightComponent(const ULightComponentBase* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertDirectionalLightComponent(const UDirectionalLightComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertRectLightComponent(const URectLightComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertPointLightComponent(const UPointLightComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertSkyLightComponent(const USkyLightComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertSpotLightComponent(const USpotLightComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertDrawModeComponent(const UUsdDrawModeComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertAudioComponent(const UAudioComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F, bool bFilePathOnly = false);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertSceneComponent(const USceneComponent* Component, const FString& PrimPath);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertIsmComponent(const UInstancedStaticMeshComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertHismComponent(const UHierarchicalInstancedStaticMeshComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertMeshComponent(const UMeshComponent* Component, const FString& PrimPath);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertCineCameraComponent(const UCineCameraComponent* Component, const FString& PrimPath, float TimeCode = 3.402823466e+38F);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertInstancedFoliageActor(
		const AInstancedFoliageActor* Actor,
		const FString& PrimPath,
		ULevel* InstancesLevel = nullptr,
		float TimeCode = 3.402823466e+38F
	);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertLandscapeProxyActorMesh(
		const ALandscapeProxy* Actor,
		const FString& PrimPath,
		int32 LowestLOD,
		int32 HighestLOD,
		float TimeCode = 3.402823466e+38F
	);

	UFUNCTION(BlueprintCallable, Category = "Component conversion")
	UE_API bool ConvertLandscapeProxyActorMaterial(
		ALandscapeProxy* Actor,
		const FString& PrimPath,
		const TArray<FPropertyEntry>& PropertiesToBake,
		const FIntPoint& DefaultTextureSize,
		const FDirectoryPath& TexturesDir,
		float TimeCode = 3.402823466e+38F
	);

public:
	/**
	 * Sets the material overrides as opinions on MeshPrim, taking care to figure out whether it needs to create LOD or subsection prims, and
	 * which ones given the chosen export LOD range.
	 * MeshAsset should be an UStaticMesh, USkeletalMesh or UGeometryCache.
	 *
	 * If bCheckAssetUserData is enabled, this function which check the asset's USD Asset User Data in order to determine the mapping between
	 * material slot and prim path, which is useful when exporting multiple slots to geom subsets or LOD prims. If this is false, The overrides
	 * will be authored to standard paths used by the USDImporter plugin (i.e. LOD<N> prims when needed, UsdGeomSubsets named "Section<N>")
	 */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils")
	UE_API bool ConvertMaterialOverrides(
		const UObject* MeshAsset,
		const TArray<UMaterialInterface*> MaterialOverrides,
		const FString& PrimPath,
		int32 LowestLOD,
		int32 HighestLOD,
		bool bCheckAssetUserData = true
	);

	/**
	 * Traverses the context's stage and authors material binding attributes for all `unrealMaterials` that were baked into USD material assets.
	 * @param LayerToAuthorIn - File path to the layer where the material binding opinions are authored
	 * @param BakedMaterials - Maps from material path names to file paths where they were baked
	 *                         Example: { "/Game/MyMaterials/Red.Red": "C:/MyFolder/Red.usda" }
	 * @param bIsAssetLayer - True when we're exporting a single mesh/animation asset. False when we're exporting a level. Dictates minor behaviors
	 *                        when authoring the material binding relationships, e.g. whether we author them inside variants or not
	 * @param bUsePayload - Should be True if the Stage was exported using payload files to store the actual Mesh prims. Also dictates minor
	 *                      behaviors when authoring the material binding relationships.
	 * @param bRemoveUnrealMaterials - No longer used.
	 */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils")
	UE_API void ReplaceUnrealMaterialsWithBaked(
		const FFilePath& LayerToAuthorIn,
		const TMap<FString, FString>& BakedMaterials,
		bool bIsAssetLayer,
		bool bUsePayload,
		bool bRemoveUnrealMaterials = false
	);

	/**
	 * Writes UnrealMaterialPathName as a material binding for MeshLikePrimPath, either by reusing an existing UsdShadeMaterial
	 * binding if it already has an 'unreal' render context output and the expected structure, or by creating a new Material prim
	 * that fulfills those requirements and binding to it (optionally with strongerThanDescendants strength in that case).
	 */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils")
	UE_API void AuthorUnrealMaterialBinding(
		const FString& MeshLikePrimPath,
		const FString& UnrealMaterialPathName,
		bool bStrongerThanDescendants = false
	);

	UE_DEPRECATED(all, "No longer used as UE material assignments are only visible in the 'unreal' render context anyway")
	UFUNCTION()
	UE_API bool RemoveUnrealSurfaceOutput(const FString& PrimPath, const FFilePath& LayerToAuthorIn);

	/** Returns how many frames of animation the stage would generate if imported */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils")
	UE_API int32 GetUsdStageNumFrames();

	/** Adds to Prim the assetInfo metadata the values described in Info */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils|Metadata")
	UE_API void SetPrimAssetInfo(const FString& PrimPath, const FUsdUnrealAssetInfo& Info);

	/** Retrieves from Prim the assetInfo metadata values that we use as export metadata, when exporting Unreal assets */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils|Metadata")
	UE_API FUsdUnrealAssetInfo GetPrimAssetInfo(const FString& PrimPath);

	/** Applies Metadata to the prim specified at PrimPath, using the provided filters */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils|Metadata")
	UE_API void SetPrimMetadata(
		const FString& PrimPath,
		const FUsdCombinedPrimMetadata& Metadata,
		const TArray<FString>& BlockedPrefixFilter,
		bool bInvertFilter
	);

	/** Applies all the metadata contained in UserData to the prim specified at PrimPath, using the provided filters */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils|Metadata")
	UE_API void SetPrimMetadataFromUserData(
		const FString& PrimPath,
		const UUsdAssetUserData* UserData,
		const TArray<FString>& BlockedPrefixFilter,
		bool bInvertFilter
	);

	/** Extracts metadata from the prim at PrimPath using the provided filters and returns it */
	UFUNCTION(BlueprintCallable, Category = "Conversion utils|Metadata")
	UE_API FUsdCombinedPrimMetadata GetPrimMetadata(
		const FString& PrimPath,
		const TArray<FString>& BlockedPrefixFilter,
		bool bInvertFilter,
		bool bCollectFromEntireSubtrees
	);
};

#undef UE_API
