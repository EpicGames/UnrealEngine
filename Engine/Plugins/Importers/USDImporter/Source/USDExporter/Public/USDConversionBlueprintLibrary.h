// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDMetadata.h"
#include "USDProjectSettings.h"

#include "AnalyticsBlueprintLibrary.h"
#include "Engine/EngineTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "USDConversionBlueprintLibrary.generated.h"

#define UE_API USDEXPORTER_API

class AInstancedFoliageActor;
class UFoliageType;
class ULevel;
class ULevelExporterUSDOptions;
class UUsdAssetUserData;
struct FMatrix2D;
struct FMatrix3D;

/** Wrapped static conversion functions from the UsdUtilities module, so that they can be used via scripting */
UCLASS(MinimalAPI, meta = (ScriptName = "UsdConversionLibrary"))
class UUsdConversionBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Returns how many total Unreal levels (persistent + all sublevels) will be exported if we consider LevelsToIgnore */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API int32 GetNumLevelsToExport(UWorld* World, const TSet<FString>& LevelsToIgnore);

	/** Fully streams in and displays all levels whose names are not in LevelsToIgnore */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API void StreamInRequiredLevels(UWorld* World, const TSet<FString>& LevelsToIgnore);

	/**
	 * If we have the Sequencer open with a level sequence animating the level before export, this function can revert
	 * any actor or component to its unanimated state
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API void RevertSequencerAnimations();

	/**
	 * If we used `ReverseSequencerAnimations` to undo the effect of an opened sequencer before export, this function
	 * can be used to re-apply the sequencer state back to the level after the export is complete
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API void ReapplySequencerAnimations();

	/**
	 * Returns the path name (e.g. "/Game/Maps/MyLevel") of levels that are loaded on `World`.
	 * We use these to revert the `World` to its initial state after we force-stream levels in for exporting
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API TArray<FString> GetLoadedLevelNames(UWorld* World);

	/**
	 * Returns the path name (e.g. "/Game/Maps/MyLevel") of levels that checked to be visible in the editor within `World`.
	 * We use these to revert the `World` to its initial state after we force-stream levels in for exporting
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API TArray<FString> GetVisibleInEditorLevelNames(UWorld* World);

	/** Streams out/hides sublevels that were streamed in before export */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API void StreamOutLevels(UWorld* OwningWorld, const TArray<FString>& LevelNamesToStreamOut, const TArray<FString>& LevelNamesToHide);

	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API TSet<AActor*> GetActorsToConvert(UWorld* World);

	/**
	 * Generates a unique identifier string that involves ObjectToExport's package's persistent guid, the
	 * corresponding file save date and time, and the number of times the package has been dirtied since last being
	 * saved.
	 * Optionally it can also combine that hash with a hash of the export options being used for the export, if
	 * available.
	 * This can be used to track the version of exported assets and levels, to prevent reexporting of actors and
	 * components.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API FString GenerateObjectVersionString(const UObject* ObjectToExport, UObject* ExportOptions);

	/** Checks whether we can create a USD Layer with "TargetFilePath" as identifier and export to it */
	UFUNCTION(BlueprintCallable, Category = "USD|World utils")
	static UE_API bool CanExportToLayer(const FString& TargetFilePath);

	UFUNCTION(BlueprintCallable, Category = "USD|Layer utils")
	static UE_API FString MakePathRelativeToLayer(const FString& AnchorLayerPath, const FString& PathToMakeRelative);

	UFUNCTION(BlueprintCallable, Category = "USD|Layer utils")
	static UE_API void InsertSubLayer(const FString& ParentLayerPath, const FString& SubLayerPath, int32 Index = -1);

	/**
	 * Adds a reference to the layer at AbsoluteFilePath, optionally specifying a target prim inside that layer.
	 * @param ReferencingStagePath - Stage identifier that contains the referencer prim
	 * @param ReferencingPrimPath - Path to the referencer prim inside the referencer stage
	 * @param TargetStagePath - Path to the layer to reference (can be empty for internal references). e.g. "C:\MyFolder\Cube.usda"
	 * @param TargetPrimPath - Path to a specific prim to reference within the target layer. e.g. "/SomeOtherPrim"
	 * @param TimeCodeOffset - Offset in timeCodes to author on the reference
	 * @param TimeCodeScale - TimeCode scale to author on the reference
	 * @param bUseProjectDefaultTypeHandling - If true it means we will use the ReferencerTypeHandling set on the USD project settings.
	 *                                         If false it means we will use the provided ReferencerSchemaHandling
	 * @param ReferencerSchemaHandling - How to behave when Prim has a different type than the referenced prim.
	 *                                   Passing an unset TOptional will check the project settings for the value instead.
	 *                                   The default value is MatchReferencedType for backwards compatibility.
	 *                                   Note: If this is set to ShowDialog, the call may create an Editor transaction.
	 */
	 UFUNCTION(BlueprintCallable, Category = "USD|Layer utils")
	 static UE_API void AddReference(
		 const FString& ReferencingStagePath,
		 const FString& ReferencingPrimPath,
		 const FString& TargetStagePath,
		 const FString& TargetPrimPath = TEXT(""),
		 double TimeCodeOffset = 0.0,
		 double TimeCodeScale = 1.0,
		 bool bUseProjectDefaultTypeHandling = false,
		 EReferencerTypeHandling ReferencerTypeHandling = EReferencerTypeHandling::MatchReferencedType
		);

	/**
	 * Adds a payload to the layer at AbsoluteFilePath, optionally specifying a target prim inside that layer.
	 * @param ReferencingStagePath - Stage identifier that contains the prim to receive the payload
	 * @param ReferencingPrimPath - Path to the prim inside the referencing stage that will receive the payload
	 * @param TargetStagePath - Path to the layer to add as payload (can be empty for internal payloads). e.g. "C:\MyFolder\Cube.usda"
	 * @param TargetPrimPath - Path to a specific prim to target within the target layer. e.g. "/SomeOtherPrim"
	 * @param TimeCodeOffset - Offset in timeCodes to author on the payload
	 * @param TimeCodeScale - TimeCode scale to author on the payload
	 * @param bUseProjectDefaultTypeHandling - If true it means we will use the ReferencerTypeHandling set on the USD project settings.
	 *                                         If false it means we will use the provided ReferencerSchemaHandling
	 * @param ReferencerSchemaHandling - How to behave when Prim has a different type than the target prim.
	 *                                   Passing an unset TOptional will check the project settings for the value instead.
	 *                                   The default value is MatchReferencedType for backwards compatibility.
	 *                                   Note: If this is set to ShowDialog, the call may create an Editor transaction.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Layer utils")
	static UE_API void AddPayload(
		const FString& ReferencingStagePath,
		const FString& ReferencingPrimPath,
		const FString& TargetStagePath,
		const FString& TargetPrimPath = TEXT(""),
		double TimeCodeOffset = 0.0,
		double TimeCodeScale = 1.0,
		bool bUseProjectDefaultTypeHandling = false,
		EReferencerTypeHandling ReferencerTypeHandling = EReferencerTypeHandling::MatchReferencedType
	);

	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API FString GetPrimPathForObject(
		const UObject* ActorOrComponent,
		const FString& ParentPrimPath = TEXT(""),
		bool bUseActorFolders = false,
		const FString& RootPrimName = TEXT("Root")
	);

	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API FString GetSchemaNameForComponent(const USceneComponent* Component);

	/**
	 * Wraps AInstancedFoliageActor::GetInstancedFoliageActorForLevel, and allows retrieving the current AInstancedFoliageActor
	 * for a level. Will default to the current editor level if Level is left nullptr.
	 * This function is useful because it's difficult to retrieve this actor otherwise, as it will be filtered from
	 * the results of functions like EditorLevelLibrary.get_all_level_actors()
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Foliage Exporter")
	static UE_API AInstancedFoliageActor* GetInstancedFoliageActorForLevel(bool bCreateIfNone = false, ULevel* Level = nullptr);

	/**
	 * Returns all the different types of UFoliageType assets that a particular AInstancedFoliageActor uses.
	 * This function exists because we want to retrieve all instances of all foliage types on an actor, but we
	 * can't return nested containers from UFUNCTIONs, so users of this API should call this, and then GetInstanceTransforms.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "USD|Foliage Exporter")
	static UE_API TArray<UFoliageType*> GetUsedFoliageTypes(AInstancedFoliageActor* Actor);

	/**
	 * Returns the source asset for a UFoliageType.
	 * It can be a UStaticMesh in case we're dealing with a UFoliageType_InstancedStaticMesh, but it can be other types of objects.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "USD|Foliage Exporter")
	static UE_API UObject* GetSource(UFoliageType* FoliageType);

	/**
	 * Returns the transforms of all instances of a particular UFoliageType on a given level. If no level is provided all instances will be returned.
	 * Use GetUsedFoliageTypes() to retrieve all foliage types managed by a particular actor.
	 */
	UFUNCTION(BlueprintCallable, meta = (ScriptMethod), Category = "USD|Foliage Exporter")
	static UE_API TArray<FTransform> GetInstanceTransforms(AInstancedFoliageActor* Actor, UFoliageType* FoliageType, ULevel* InstancesLevel = nullptr);

	/** Retrieves the analytics attributes to send for the provided options object */
	UFUNCTION(BlueprintCallable, Category = "USD|Analytics")
	static UE_API TArray<FAnalyticsEventAttr> GetAnalyticsAttributes(const ULevelExporterUSDOptions* Options);

	/** Defer to the USDClasses module to actually send analytics information */
	UFUNCTION(BlueprintCallable, Category = "USD|Analytics")
	static UE_API void SendAnalytics(
		const TArray<FAnalyticsEventAttr>& Attrs,
		const FString& EventName,
		bool bAutomated,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension
	);

	UFUNCTION(BlueprintCallable, Category = "USD|Analytics")
	static UE_API void BlockAnalyticsEvents();

	UFUNCTION(BlueprintCallable, Category = "USD|Analytics")
	static UE_API void ResumeAnalyticsEvents();

	/**
	 * Begins a UniquePathScope, incrementing the internal scope counter.
	 *
	 * During a UniquePathScope, all paths returned by GetUniqueFilePathForExport will be globally unique (i.e. it will
	 * never return the same path twice).
	 *
	 * Opening a scope while another scope is opened has no effect but to increment the scope counter further.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Exporting utils")
	static UE_API void BeginUniquePathScope();

	/**
	 * Ends a UniquePathScope, decrementing the internal scope counter.
	 *
	 * If the internal scope counter reaches zero (i.e. all previously opened scopes are ended) this also clears the
	 * cache of unique paths.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Exporting utils")
	static UE_API void EndUniquePathScope();

	/**
	 * If we're inside of a UniquePathScope, returns a sanitized (and potentially suffixed) path that is guaranteed
	 * to not collide with any other path returned from this function during the UniquePathScope.
	 *
	 * If we're not inside of a UniquePathScope, returns the sanitized version of DesiredPathWithExtension.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Exporting utils")
	static UE_API FString GetUniqueFilePathForExport(const FString& DesiredPathWithExtension);

	/**
	 * Removes all the prim specs for Prim on the given Layer.
	 *
	 * This function is useful in case the prim is inside a variant set: In that case, just calling FUsdStage::RemovePrim()
	 * will attempt to remove the "/Root/Example/Child", which wouldn't remove the "/Root{Varset=Var}Example/Child" spec,
	 * meaning the prim may still be left on the stage. Note that it's even possible to have both of those specs at the same time:
	 * for example when we have a prim inside a variant set, but outside of it we have overrides to the same prim. This function
	 * will remove both.
	 *
	 * @param StageRootLayer - Path to the root layer of the stage from which we should fetch the Prims
	 * @param PrimPath - Prim to remove
	 * @param Layer - Layer to remove prim specs from. This can be left with the invalid layer (default) in order to remove all
	 *				  specs from the entire stage's local layer stack.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API void RemoveAllPrimSpecs(const FString& StageRootLayer, const FString& PrimPath, const FString& TargetLayer);

	/**
	 * Copies flattened versions of the input prims onto the clipboard stage and removes all the prim specs for Prims from their stages.
	 * These cut prims can then be pasted with PastePrims.
	 *
	 * @param StageRootLayer - Path to the root layer of the stage from which we should fetch the Prims
	 * @param PrimPaths - Prims to cut
	 * @return True if we managed to cut
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API bool CutPrims(const FString& StageRootLayer, const TArray<FString>& PrimPaths);

	/**
	 * Copies flattened versions of the input prims onto the clipboard stage.
	 * These copied prims can then be pasted with PastePrims.
	 *
	 * @param StageRootLayer - Path to the root layer of the stage from which we should fetch the Prims
	 * @param PrimPaths - Prims to copy
	 * @return True if we managed to copy
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API bool CopyPrims(const FString& StageRootLayer, const TArray<FString>& PrimPaths);

	/**
	 * Pastes the prims from the clipboard stage as children of ParentPrim.
	 *
	 * The pasted prims may be renamed in order to have valid names for the target location, which is why this function
	 * returns the pasted prim paths.
	 * This function returns just paths instead of actual prims because USD needs to respond to the notices about
	 * the created prim specs before the prims are fully created, which means we wouldn't be able to return the
	 * created prims yet, in case this function was called from within an SdfChangeBlock.
	 *
	 * @param StageRootLayer - Path to the root layer of the stage from which we should fetch the Prims
	 * @param ParentPrimPath - Prim that will become parent to the pasted prims
	 * @return Paths to the pasted prim specs, after they were added as children of ParentPrim
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API TArray<FString> PastePrims(const FString& StageRootLayer, const FString& ParentPrimPath);

	/** Returns true if we have prims that we can paste within our clipboard stage */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API bool CanPastePrims();

	/** Clears all prims from our clipboard stage */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API void ClearPrimClipboard();

	/**
	 * Duplicates all provided Prims one-by-one, performing the requested DuplicateType.
	 * See the documentation on EUsdDuplicateType for the different operation types.
	 *
	 * The duplicated prims may be renamed in order to have valid names for the target location, which is why this
	 * function returns the pasted prim paths.
	 * This function returns just paths instead of actual prims because USD needs to respond to the notices about
	 * the created prim specs before the prims are fully created, which means we wouldn't be able to return the
	 * created prims yet, in case this function was called from within an SdfChangeBlock.
	 *
	 * @param StageRootLayer - Path to the root layer of the stage from which we should fetch the Prims
	 * @param PrimPaths - Prims to duplicate
	 * @param DuplicateType - Type of prim duplication to perform
	 * @param TargetLayer - Target layer to use when duplicating, if relevant for that duplication type
	 * @return Paths to the duplicated prim specs, after they were added as children of ParentPrim.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Prim utils")
	static UE_API TArray<FString> DuplicatePrims(
		const FString& StageRootLayer,
		const TArray<FString>& PrimPaths,
		EUsdDuplicateType DuplicateType,
		const FString& TargetLayer
	);

	/* Retrieve the first instance of UUsdAssetUserData contained on the Object, if any */
	UFUNCTION(BlueprintCallable, Category = "USD|Metadata utils")
	static UE_API UUsdAssetUserData* GetUsdAssetUserData(UObject* Object);

	/*
	 * Sets AssetUserData as the single UUsdAssetUserData on the Object, overwriting an existing one if encountered.
	 * Returns true if it managed to add AssetUserData to Object.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Metadata utils")
	static UE_API bool SetUsdAssetUserData(UObject* Object, UUsdAssetUserData* AssetUserData);

	/*
	 * Utilities that make it easier to get/set metadata fields without having to manipulate the nested struct instances directly.
	 * It will create the struct entries automatically if needed, and overwrite existing entries with the same key if needed.
	 *
	 * If the AssetUserData contains exactly one entry for StageIdentifier and one entry for PrimPath, you can omit those arguments
	 * and that single entry will be used. If there are more or less than exactly one entry for StageIdentifier or for PrimPath however,
	 * you must specify which one to use, and failing to do so will cause the functions to return false and emit a warning.
	 *
	 * It is possible to get these functions to automatically trigger pre/post property changed events by providing "true" for
	 * bTriggerPropertyChangeEvents, which is useful as it is not trivial to trigger those from Python/Blueprint given how the
	 * metadata is stored inside nested structs and maps. If these AssetUserData belong to generated transient assets when opening
	 * stages, emitting property change events causes those edits to be immediately written out to the opened USD Stage.
	 *
	 * Returns true if it managed to set the new key-value pair.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Metadata utils")
	static UE_API bool SetMetadataField(
		UUsdAssetUserData* AssetUserData,
		const FString& Key,
		const FString& Value,
		const FString& ValueTypeName,
		const FString& StageIdentifier = "",
		const FString& PrimPath = "",
		bool bTriggerPropertyChangeEvents = true
	);
	UFUNCTION(BlueprintCallable, Category = "USD|Metadata utils")
	static UE_API bool ClearMetadataField(
		UUsdAssetUserData* AssetUserData,
		const FString& Key,
		const FString& StageIdentifier = "",
		const FString& PrimPath = "",
		bool bTriggerPropertyChangeEvents = true
	);
	UFUNCTION(BlueprintCallable, Category = "USD|Metadata utils")
	static UE_API bool HasMetadataField(
		UUsdAssetUserData* AssetUserData,
		const FString& Key,
		const FString& StageIdentifier = "",
		const FString& PrimPath = ""
	);
	UFUNCTION(BlueprintCallable, Category = "USD|Metadata utils")
	static UE_API FUsdMetadataValue GetMetadataField(
		UUsdAssetUserData* AssetUserData,
		const FString& Key,
		const FString& StageIdentifier = "",
		const FString& PrimPath = ""
	);

public:	   // Stringify functions
	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsBool(bool Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_uchar"))
	static UE_API FString StringifyAsUChar(uint8 Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt(int32 Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_uint"))
	static UE_API FString StringifyAsUInt(int32 Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt64(int64 Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_uint64"))
	static UE_API FString StringifyAsUInt64(int64 Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf(float Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat(float Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble(double Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_timecode"))
	static UE_API FString StringifyAsTimeCode(double Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsString(const FString& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsToken(const FString& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsAssetPath(const FString& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsMatrix2d(const FMatrix2D& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsMatrix3d(const FMatrix3D& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsMatrix4d(const FMatrix& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsQuatd(const FQuat& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsQuatf(const FQuat& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsQuath(const FQuat& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble2(const FVector2D& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat2(const FVector2D& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf2(const FVector2D& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt2(const FIntPoint& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble3(const FVector& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat3(const FVector& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf3(const FVector& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt3(const FIntVector& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble4(const FVector4& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat4(const FVector4& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf4(const FVector4& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt4(const FIntVector4& Value);

public:	   // Stringify array functions
	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsBoolArray(const TArray<bool>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_uchar_array"))
	static UE_API FString StringifyAsUCharArray(const TArray<uint8>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsIntArray(const TArray<int32>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_uint_array"))
	static UE_API FString StringifyAsUIntArray(const TArray<int32>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt64Array(const TArray<int64>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_uint64_array"))
	static UE_API FString StringifyAsUInt64Array(const TArray<int64>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalfArray(const TArray<float>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloatArray(const TArray<float>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDoubleArray(const TArray<double>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "stringify_as_timecode_array"))
	static UE_API FString StringifyAsTimeCodeArray(const TArray<double>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsStringArray(const TArray<FString>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsTokenArray(const TArray<FString>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsAssetPathArray(const TArray<FString>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsListOpTokens(const TArray<FString>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsMatrix2dArray(const TArray<FMatrix2D>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsMatrix3dArray(const TArray<FMatrix3D>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsMatrix4dArray(const TArray<FMatrix>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsQuatdArray(const TArray<FQuat>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsQuatfArray(const TArray<FQuat>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsQuathArray(const TArray<FQuat>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble2Array(const TArray<FVector2D>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat2Array(const TArray<FVector2D>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf2Array(const TArray<FVector2D>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt2Array(const TArray<FIntPoint>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble3Array(const TArray<FVector>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat3Array(const TArray<FVector>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf3Array(const TArray<FVector>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt3Array(const TArray<FIntVector>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsDouble4Array(const TArray<FVector4>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsFloat4Array(const TArray<FVector4>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsHalf4Array(const TArray<FVector4>& Value);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString StringifyAsInt4Array(const TArray<FIntVector4>& Value);

public:	   // Unstringify functions
	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API bool UnstringifyAsBool(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_uchar"))
	static UE_API uint8 UnstringifyAsUChar(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API int32 UnstringifyAsInt(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_uint"))
	static UE_API int32 UnstringifyAsUInt(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API int64 UnstringifyAsInt64(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_uint64"))
	static UE_API int64 UnstringifyAsUInt64(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API float UnstringifyAsHalf(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API float UnstringifyAsFloat(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API double UnstringifyAsDouble(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_timecode"))
	static UE_API double UnstringifyAsTimeCode(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString UnstringifyAsString(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString UnstringifyAsToken(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FString UnstringifyAsAssetPath(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FMatrix2D UnstringifyAsMatrix2d(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FMatrix3D UnstringifyAsMatrix3d(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FMatrix UnstringifyAsMatrix4d(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FQuat UnstringifyAsQuatd(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FQuat UnstringifyAsQuatf(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FQuat UnstringifyAsQuath(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector2D UnstringifyAsDouble2(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector2D UnstringifyAsFloat2(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector2D UnstringifyAsHalf2(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FIntPoint UnstringifyAsInt2(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector UnstringifyAsDouble3(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector UnstringifyAsFloat3(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector UnstringifyAsHalf3(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FIntVector UnstringifyAsInt3(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector4 UnstringifyAsDouble4(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector4 UnstringifyAsFloat4(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FVector4 UnstringifyAsHalf4(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API FIntVector4 UnstringifyAsInt4(const FString& String);

public:	   // Unstringify array functions
	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<bool> UnstringifyAsBoolArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_uchar_array"))
	static UE_API TArray<uint8> UnstringifyAsUCharArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<int32> UnstringifyAsIntArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_uint_array"))
	static UE_API TArray<int32> UnstringifyAsUIntArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<int64> UnstringifyAsInt64Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_uint64_array"))
	static UE_API TArray<int64> UnstringifyAsUInt64Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<float> UnstringifyAsHalfArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<float> UnstringifyAsFloatArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<double> UnstringifyAsDoubleArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils", Meta = (ScriptName = "unstringify_as_timecode_array"))
	static UE_API TArray<double> UnstringifyAsTimeCodeArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FString> UnstringifyAsStringArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FString> UnstringifyAsTokenArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FString> UnstringifyAsAssetPathArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FString> UnstringifyAsListOpTokens(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FMatrix2D> UnstringifyAsMatrix2dArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FMatrix3D> UnstringifyAsMatrix3dArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FMatrix> UnstringifyAsMatrix4dArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FQuat> UnstringifyAsQuatdArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FQuat> UnstringifyAsQuatfArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FQuat> UnstringifyAsQuathArray(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector2D> UnstringifyAsDouble2Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector2D> UnstringifyAsFloat2Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector2D> UnstringifyAsHalf2Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FIntPoint> UnstringifyAsInt2Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector> UnstringifyAsDouble3Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector> UnstringifyAsFloat3Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector> UnstringifyAsHalf3Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FIntVector> UnstringifyAsInt3Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector4> UnstringifyAsDouble4Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector4> UnstringifyAsFloat4Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FVector4> UnstringifyAsHalf4Array(const FString& String);

	UFUNCTION(BlueprintCallable, Category = "USD|Stringify utils")
	static UE_API TArray<FIntVector4> UnstringifyAsInt4Array(const FString& String);
};

#undef UE_API
