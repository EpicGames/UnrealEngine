// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class IUsdExporterModule : public IModuleInterface
{
public:
	static void HashEditorSelection(FSHA1& HashToUpdate);

	/** Checks whether we can create a USD Layer with "TargetFilePath" as identifier and export to it */
	static bool CanExportToLayer(const FString& TargetFilePath);
};
