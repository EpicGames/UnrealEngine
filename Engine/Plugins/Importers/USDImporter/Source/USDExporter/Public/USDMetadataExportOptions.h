// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDMetadataExportOptions.generated.h"

struct FAnalyticsEventAttribute;

USTRUCT(BlueprintType)
struct FUsdMetadataExportOptions
{
	GENERATED_BODY()

	// Export generic assetInfo when exporting assets (such as export time, engine version, etc.).
	// Note that exporting assetInfo is required in order to avoid the re-export of identical assets to the same file.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config)
	bool bExportAssetInfo = true;

	// Export metadata held on assets' UsdAssetUserData to the output prim.
	// Note that metadata is always exported afterwards assetInfo, which means that if assetInfo entries
	// are specified within the collected metadata, those will override the automatically generated
	// assetInfo vallues
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config)
	bool bExportAssetMetadata = true;

	// Export metadata held on components' UsdAssetUserData to the output prims.
	// This only has an effect for Level/LevelSequence exports.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Metadata", config)
	bool bExportComponentMetadata = false;

	// When exporting metadata, we will ignore all entries that start with these prefixes.
	// Note that you can use the ":" separator for nested dictionaries in USD.
	UPROPERTY(
		EditAnywhere,
		BlueprintReadWrite,
		Category = "Metadata",
		config,
		meta = (EditCondition = "bExportAssetMetadata || bExportComponentMetadata")
	)
	TArray<FString> BlockedPrefixFilters;

	// When this is false (default), the "BlockedPrefixFilters" property describe prefixes to block, and
	// metadata entries that match any of those prefixes are ignored and not exported.
	// When this is true, the "BlockedPrefixFilters" property is inverted, and describes prefixes to *allow*.
	// In that case, entries are only allowed and exported if they match at least one of the provided prefixes.
	UPROPERTY(
		EditAnywhere,
		BlueprintReadWrite,
		Category = "Metadata",
		config,
		meta = (EditCondition = "bExportAssetMetadata || bExportComponentMetadata")
	)
	bool bInvertFilters = false;
};

namespace UsdUtils
{
	USDEXPORTER_API void AddAnalyticsAttributes(const FUsdMetadataExportOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);
	USDEXPORTER_API void HashForExport(const FUsdMetadataExportOptions& Options, FSHA1& HashToUpdate);
}
