// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDExporter : ModuleRules
	{
		public USDExporter(ReadOnlyTargetRules Target) : base(Target)
        {
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"UnrealUSDWrapper",
					"Foliage"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"AnalyticsBlueprintLibrary",
					"CinematicCamera",
					"Constraints",
					"ControlRig", // For detecting rigged skeletal animations when exporting level sequences to USD
					"EditorStyle",
					"GeometryCache",
					"InputCore",
					"JsonUtilities",
					"Landscape",
					"LevelSequence",
					"LevelSequenceEditor", // For LevelSequenceEditorSpawnRegister, which we need to use when exporting level sequences with spawnables
					"MaterialBaking", // So that we can use some of the export option properties
					"MaterialUtilities",
					"MeshDescription",
					"MeshUtilities",
					"MessageLog",
					"MovieScene",
					"MovieSceneTracks",
					"PropertyEditor",
					"PythonScriptPlugin",
					"RawMesh",
					"RenderCore",
					"RHI",
					"Sequencer",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"UniversalObjectLocator",
					"UnrealEd",
					"USDClasses",
					"USDStage",
					"USDStageImporter", // For USDOptionsWindow
					"USDUtilities",
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
