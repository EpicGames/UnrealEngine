// Copyright Epic Games, Inc. All Rights Reserved.

#include "Custom/MaterialXUSDShadeMaterialTranslator.h"
#if USE_USD_SDK
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "Engine/Level.h"
#include "Engine/Texture.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeGenericMaterialPipeline.h"
#include "InterchangeManager.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeTexture2DNode.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MaterialShared.h"
#include "Misc/PackageName.h"
#include "UObject/StrongObjectPtr.h"

#include "USDIncludesStart.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usdShade/material.h"
#include "USDIncludesEnd.h"

#if ENABLE_USD_MATERIALX
#include "MaterialXFormat/Util.h"
#endif // ENABLE_USD_MATERIALX

#define LOCTEXT_NAMESPACE "MaterialXUSDShadeMaterialTranslator"

static bool GUseInterchangeMaterialXTranslator = true;
static FAutoConsoleVariableRef CvarUseInterchangeMaterialXTranslator(
	TEXT("USD.UseInterchangeMaterialXTranslator"),
	GUseInterchangeMaterialXTranslator,
	TEXT(
		"Whether to translate MaterialX materials referenced by USD files with Unreal's own MaterialX importer. If instead this is false, we will try parsing the generated UsdShadeMaterial prims as generated by USD's usdMtlx plugin directly"
	)
);

namespace UE::USDMaterialXTranslator::Private
{
#if ENABLE_USD_MATERIALX
	// Quick shallow parse of MaterialX documents ourselves so that we can find all the referenced MaterialX files and
	// build a robust hash to use for caching our generated assets
	FString HashMaterialXFile(const FString& MaterialXFilePath)
	{
		if (!FPaths::FileExists(MaterialXFilePath))
		{
			return {};
		}

		FSHA1 SHA1;

		TSet<FString> ReferencedMaterialXFiles;
		TFunction<void(const FString&)> CollectReferencedMaterialXFilesRecursive;
		CollectReferencedMaterialXFilesRecursive =
			[&ReferencedMaterialXFiles, &CollectReferencedMaterialXFilesRecursive](const FString& ActiveDocumentFullPath)
		{
			namespace mx = MaterialX;

			mx::DocumentPtr Document = mx::createDocument();
			mx::readFromXmlFile(Document, TCHAR_TO_UTF8(*ActiveDocumentFullPath));

			for (const std::string& ReferencedURI : Document->getReferencedSourceUris())
			{
				FString UEReferencedURI = UTF8_TO_TCHAR(ReferencedURI.c_str());

				// This could be relative to the referencer .mtlx file
				UEReferencedURI = FPaths::ConvertRelativePathToFull(FPaths::GetPath(ActiveDocumentFullPath), UEReferencedURI);

				if (ReferencedMaterialXFiles.Contains(UEReferencedURI))
				{
					continue;
				}

				if (FPaths::FileExists(UEReferencedURI))
				{
					ReferencedMaterialXFiles.Add(UEReferencedURI);
					CollectReferencedMaterialXFilesRecursive(UEReferencedURI);
				}
			}
		};

		// Note we don't have to add MaterialXFilePath ourselves because getReferencedSourceUris
		// always includes at least the active document itself anyway
		CollectReferencedMaterialXFilesRecursive(MaterialXFilePath);

		for (const FString& ReferencedMaterialXFile : ReferencedMaterialXFiles)
		{
			if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*ReferencedMaterialXFile)})
			{
				TArray<uint8> LocalScratch;
				LocalScratch.SetNumUninitialized(1024 * 64);

				const int64 Size = Ar->TotalSize();
				int64 Position = 0;

				// Read in BufferSize chunks
				while (Position < Size)
				{
					const int64 ReadNum = FMath::Min(Size - Position, (int64)LocalScratch.Num());
					Ar->Serialize(LocalScratch.GetData(), ReadNum);
					SHA1.Update(LocalScratch.GetData(), ReadNum);

					Position += ReadNum;
				}
			}
		}

		FSHAHash Hash;
		SHA1.Final();
		SHA1.GetHash(&Hash.Hash[0]);
		return Hash.ToString();
	}

	bool TranslateMaterialXFile(
		const FString& MaterialXFilePath,
		const FString& FileHash,
		const pxr::UsdPrim& MaterialXReferencerPrim,
		UUsdAssetCache3& AssetCache,
		EObjectFlags ObjectFlags,
		bool bShareAssetsForIdenticalPrims
	)
	{
		if (!FPaths::FileExists(MaterialXFilePath))
		{
			return false;
		}

		UInterchangeManager& Manager = UInterchangeManager::GetInterchangeManager();
		UInterchangeSourceData* SourceData = Manager.CreateSourceData(MaterialXFilePath);

		FString ReferencerPrimPath = *UsdToUnreal::ConvertPath(MaterialXReferencerPrim.GetPrimPath());

		FImportAssetParameters InterchangeParameters;
		InterchangeParameters.bIsAutomated = true;

		const FString HashPrefix = UsdUtils::GetAssetHashPrefix(MaterialXReferencerPrim, bShareAssetsForIdenticalPrims);

		// Annoyingly we have to make a new target folder for the interchange import, and then rename all the assets over.
		// This because for direct imports we'll be dealing with an asset cache that is pointing at the transient package.
		// If we naively give the transient package path to Interchange as the target import location, it will create packages
		// inside of the transient package, which sounds like very bad news.
		//
		// Additionally, importing to a guaranteed empty folder and then renaming the assets into their target packages
		// means that we avoid another annoyance: If Interchange finds an asset with the target name at the target location
		// and of the target class, it will just silently stop importing that factory node and use that asset instead. There
		// is no way for us to tell when it did this, so from our end it just looks like sometimes some assets will be missing
		// from the import for no reason... when that happens we'll end up falling back to translating the Material prim with
		// UsdShadeMaterialTranslator and get a broken black plastic material instead, and a warning on the output log.
		// By importing into an empty folder we avoid this issue.
		int32 PackageSuffix = FMath::RandRange(0, TNumericLimits<int32>::Max() - 1);
		FString TempPackagePath = FString::Printf(TEXT("/Engine/USDImporter/Transient/%d"), PackageSuffix++);

		// Make sure that content folder doesn't have any packages already in it
		{
			TArray<FString> Filenames;
			while (FPackageName::FindPackagesInDirectory(Filenames, TempPackagePath))
			{
				TempPackagePath = FString::Printf(TEXT("/Engine/USDImporter/Transient/%d"), PackageSuffix++);
			}
		}

		InterchangeParameters.OnAssetsImportDoneNative.BindLambda(
			[&AssetCache, &FileHash, &ReferencerPrimPath, &TempPackagePath, &MaterialXFilePath, &HashPrefix, ObjectFlags](
				const TArray<UObject*>& ImportedObjects
			)
			{
				// Move all our assets to the transient package
				for (UObject* ImportedObject : ImportedObjects)
				{
					FString PrefixedAssetHash;
					if (UMaterialInterface* Material = Cast<UMaterialInterface>(ImportedObject))
					{
						// MaterialX names are unique, and can only have alphanumeric and the "_" character, so we should
						// always have a solid enough mapping to assume UAsset name == Prim name == MaterialX name
						PrefixedAssetHash = HashPrefix + FileHash + TEXT("/") + Material->GetFName().GetPlainNameString();
					}
					else if (UTexture* Texture = Cast<UTexture>(ImportedObject))
					{
						const FString FilePath = Texture->AssetImportData ? Texture->AssetImportData->GetFirstFilename() : TEXT("");
						PrefixedAssetHash = HashPrefix
											+ UsdUtils::GetTextureHash(
												FilePath,
												Texture->SRGB,
												Texture->CompressionSettings,
												Texture->GetTextureAddressX(),
												Texture->GetTextureAddressY()
											);
					}
					else
					{
						ensureMsgf(false, TEXT("Asset type unsupported!"));
						continue;
					}

					// We generate all assets from the MaterialX file once, but it's possible we're just updating a
					// single material prim here. If we were to cache all assets here, we'd potentially be trying
					// to overwrite the existing assets that are being used by other prims and wouldn't otherwise
					// be discarded, so make sure we don't do that
					const bool bAssetAlreadyCached = AssetCache.IsAssetTrackedByCache(AssetCache.GetCachedAssetPath(PrefixedAssetHash));
					if (bAssetAlreadyCached)
					{
						continue;
					}

					// Cache the asset for the first time
					bool bCreatedAsset = false;
					UClass* ImportedObjectClass = ImportedObject->GetClass();
					UObject* NewCachedAsset = AssetCache.GetOrCreateCustomCachedAsset(
						PrefixedAssetHash,
						ImportedObjectClass,
						ImportedObject->GetName(),
						ObjectFlags,
						[ImportedObject](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse) -> UObject*
						{
							UPackage* InterchangePackage = ImportedObject->GetOutermost();

							// Rename the UMaterialInterface into the target UPackage the asset cache created for us.
							// SanitizedName will already match it.
							const bool bRenamed = ImportedObject
													  ->Rename(*SanitizedName.ToString(), Outer, REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
							ensure(bRenamed);

							ImportedObject->ClearFlags(ImportedObject->GetFlags());
							ImportedObject->SetFlags(FlagsToUse);

							// Get rid of the original package interchange made for this asset
							if (InterchangePackage && InterchangePackage != GetTransientPackage())
							{
								InterchangePackage->MarkAsGarbage();
								InterchangePackage->ClearFlags(RF_Public | RF_Standalone);
							}

							return ImportedObject;
						},
						&bCreatedAsset
					);
					ensure(bCreatedAsset && NewCachedAsset);
				}

				// Now that the asset cache took everything we imported we shouldn't have anything else
				TArray<FString> Filenames;
				if (FPackageName::FindPackagesInDirectory(Filenames, TempPackagePath))
				{
					ensureMsgf(false, TEXT("We should not have any leftover assets from MaterialX translation!"));
				}
			}
		);

		return Manager.ImportAsset(TempPackagePath, SourceData, InterchangeParameters);
	}
#endif // ENABLE_USD_MATERIALX
}	 // namespace UE::USDMaterialXTranslator::Private

void FMaterialXUsdShadeMaterialTranslator::CreateAssets()
{
#if ENABLE_USD_MATERIALX
	// We handle MaterialX materials here by leveraging USD's own usdMtlx plugin. What it does is translate references
	// to .mtlx files into hierarchies of Material/Shader/NodeGraph prims as the stage is opened. To actually use
	// the materials contained in the .mtlx files, other prims from the stage will create material bindings to those
	// generated Material prims. For example, a stage that wishes to use MaterialX materials may have this:
	//
	// def Mesh "SomeMesh"
	// {
	//     ...
	//     rel material:binding = </MaterialX/Materials/TextureMaterial>
	// }
	//
	// def Scope "MaterialX" (
	//     references = [
	//         @./textureTest.mtlx@</MaterialX>,
	//     ]
	// )
	// {
	// }
	//
	// Note how SomeMesh references the "TextureMaterial" material: That's the name of one of the surfacematerials
	// inside the MaterialX file. This because when the stage is opened, the usdMtlx plugin will generate and
	// compose a child "Materials" prim and additional child prims something like the below, which is what the external
	// prims will be referencing:
	//
	// def Scope "MaterialX"
	// {
	//     def "Materials"
	//     {
	//         def Material "TextureMaterial"
	//         {
	//             float inputs:base = 1
	//             color3f inputs:base_color
	//             float inputs:coat
	//             float inputs:coat_affect_color
	//             ...
	//             token outputs:mtlx:surface.connect = </MaterialX/Materials/TextureMaterial/ND_standard_surface_surfaceshader.outputs:surface>
	//
	//             def Shader "ND_standard_surface_surfaceshader"
	//             {
	//                 uniform token info:id = "ND_standard_surface_surfaceshader"
	//                 float inputs:base.connect = </MaterialX/Materials/TextureMaterial.inputs:base>
	//                 color3f inputs:base_color.connect = </MaterialX/Materials/TextureMaterial/NG_imagetex1.outputs:out_color_0>
	//                 float inputs:coat.connect = </MaterialX/Materials/TextureMaterial.inputs:coat>
	//                 float inputs:coat_affect_color.connect = </MaterialX/Materials/TextureMaterial.inputs:coat_affect_color>
	//                 ...
	//             }
	//         }
	//     }
	// }
	//
	// Our goal in here is to translate those generated Materials (like "TextureMaterial" above). This because we want
	// to generate UMaterialInterfaces via Unreal's MaterialX plugin instead, and link them to those generated Material
	// prims within our info cache. This way, the rest or our USDImporter plugin doesn't need to care or know where
	// this material came from: It will find an UMaterialInterface linked to that Material prim and it will happily use
	// that as any other material.

	if (Context->RenderContext != UnrealIdentifiers::MaterialXRenderContext || !GUseInterchangeMaterialXTranslator)
	{
		Super::CreateAssets();
		return;
	}

	if (!Context->UsdAssetCache || !Context->UsdInfoCache)
	{
		return;
	}

	if (Context->bTranslateOnlyUsedMaterials && !Context->UsdInfoCache->IsMaterialUsed(PrimPath))
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim Prim = GetPrim();
	pxr::UsdShadeMaterial ShadeMaterial(Prim);
	if (!ShadeMaterial)
	{
		return;
	}

	// We check for the mtlx surface output directly, because ComputeSurfaceSource will return a valid SurfaceShader
	// in case the material just has a regular universal render context output.
	// This is just for checking though: We will defer back to USD to let it ComputeSurfaceSource with whatever logic it has
	const static pxr::TfToken RenderContextToken = UnrealToUsd::ConvertToken(*UnrealIdentifiers::MaterialXRenderContext.ToString()).Get();
	pxr::UsdShadeOutput MtlxOutput = ShadeMaterial.GetSurfaceOutput(RenderContextToken);
	if (!MtlxOutput)
	{
		Super::CreateAssets();
		return;
	}
	pxr::UsdShadeShader SurfaceShader = ShadeMaterial.ComputeSurfaceSource(RenderContextToken);
	if (!SurfaceShader)
	{
		// This really shouldn't ever happen if we have an actual 'mtlx' output, but anyway
		Super::CreateAssets();
		return;
	}

	// This material prim has the mtlx render context, so maybe it is one of the ones generated by usdMtlx.
	// Let's traverse upwards and try finding a .mtlx file reference in one of our parents.

	TArray<FString> MaterialXFilePaths = UsdUtils::GetMaterialXFilePaths(Prim);
	pxr::UsdPrim MaterialXReferencerPrim = Prim;
	if (MaterialXFilePaths.Num() == 0)
	{
		// We know the usdMtlx plugin always generates a "Materials" schemaless prim to contain all the generated
		// Materials, so let's use that too.
		pxr::UsdPrim ParentPrim = Prim.GetParent();
		while (ParentPrim && ParentPrim.GetName() != "Materials")
		{
			ParentPrim = ParentPrim.GetParent();
		}

		if (ParentPrim)
		{
			// This prim likely holds the reference to the MaterialX file, but let's search upwards too
			pxr::UsdPrim MaterialXReferencerCandidate = ParentPrim;
			while (MaterialXReferencerCandidate)
			{
				MaterialXFilePaths = UsdUtils::GetMaterialXFilePaths(MaterialXReferencerCandidate);
				if (MaterialXFilePaths.Num() > 0)
				{
					break;
				}

				MaterialXReferencerCandidate = MaterialXReferencerCandidate.GetParent();
			}
			MaterialXReferencerPrim = MaterialXReferencerCandidate;
		}
	}

	if (MaterialXFilePaths.IsEmpty() || !MaterialXReferencerPrim)
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"NoReferencedMtlxFile",
				"Recognized potential MaterialX materials on prim '{0}', but failed to find a valid referenced MaterialX file. Reverting to parsing the generated Material prims instead."
			),
			FText::FromString(PrimPath.GetString())
		));
		Super::CreateAssets();
		return;
	}

	FString TargetHashSuffix = TEXT("/") + UsdToUnreal::ConvertString(Prim.GetName());
	FString TargetHashPrefix = UsdUtils::GetAssetHashPrefix(Prim, Context->bShareAssetsForIdenticalPrims);

	FString FoundMaterialAssetHash;
	UMaterialInterface* ParsedMaterial = nullptr;

	// Try to find the parsed material already in the asset cache assuming it came from any one of the MaterialX file paths
	TArray<FString> MaterialXFileHashes;
	MaterialXFileHashes.Reserve(MaterialXFilePaths.Num());
	for (const FString& MaterialXFilePath : MaterialXFilePaths)
	{
		FString& MaterialXHash = MaterialXFileHashes.Emplace_GetRef(UE::USDMaterialXTranslator::Private::HashMaterialXFile(MaterialXFilePath));
		if (MaterialXHash.IsEmpty())
		{
			continue;
		}

		FString MaterialAssetHashForThisFile = TargetHashPrefix + MaterialXHash + TargetHashSuffix;
		ParsedMaterial = Cast<UMaterialInterface>(Context->UsdAssetCache->GetCachedAsset(MaterialAssetHashForThisFile));
		if (ParsedMaterial)
		{
			FoundMaterialAssetHash = MaterialAssetHashForThisFile;
			break;
		}
	}

	if (!ParsedMaterial)
	{
		// Translate all MaterialX files
		ensure(MaterialXFilePaths.Num() == MaterialXFileHashes.Num());
		for (int32 Index = 0; Index < MaterialXFilePaths.Num(); ++Index)
		{
			const FString& MaterialXFilePath = MaterialXFilePaths[Index];
			const FString& MaterialXFileHash = MaterialXFileHashes[Index];

			const bool bSuccess = UE::USDMaterialXTranslator::Private::TranslateMaterialXFile(
				MaterialXFilePath,
				MaterialXFileHash,
				MaterialXReferencerPrim,
				*Context->UsdAssetCache,
				Context->ObjectFlags,
				Context->bShareAssetsForIdenticalPrims
			);

			if (!bSuccess)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"ParsingReferencedFailed",
						"Recognized potential MaterialX materials on prim '{0}', but MaterialX parsing of file '{1}' failed."
					),
					FText::FromString(PrimPath.GetString()),
					FText::FromString(MaterialXFilePath)
				));
			}

			// Check if we found our target material when parsing this file
			if (!ParsedMaterial)
			{
				FString MaterialAssetHashForThisFile = TargetHashPrefix + MaterialXFileHash + TargetHashSuffix;
				ParsedMaterial = Cast<UMaterialInterface>(Context->UsdAssetCache->GetCachedAsset(MaterialAssetHashForThisFile));
				if (ParsedMaterial)
				{
					FoundMaterialAssetHash = MaterialAssetHashForThisFile;

					// Notice we don't break here: Let's parse all referenced MaterialX files anyway
				}
			}
		}
	}

	if (ParsedMaterial)
	{
		ensure(!FoundMaterialAssetHash.IsEmpty());
		PostImportMaterial(FoundMaterialAssetHash, ParsedMaterial);
	}
	else
	{
		FString FilePaths;
		for (const FString& MaterialXFilePath : MaterialXFilePaths)
		{
			FilePaths += MaterialXFilePath + TEXT(", ");
		}
		FilePaths.RemoveFromEnd(TEXT(", "));

		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT(
				"MaterialNotFoundInMtlxFile",
				"Failed to find target Material '{0}' after parsing all Material X files [{1}]. Reverting back to parsing the USD Material prim generated by usdMtlx directly."
			),
			FText::FromString(PrimPath.GetString()),
			FText::FromString(FilePaths)
		));
		Super::CreateAssets();
	}
#else
	Super::CreateAssets();
#endif // ENABLE_USD_MATERIALX
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
