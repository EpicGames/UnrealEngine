// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshTranslationImpl.h"

#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "Objects/USDSchemaTranslator.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDProjectSettings.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"

#include "Components/MeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "MaterialShared.h"
#include "MeshDescription.h"
#include "UObject/Package.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "MeshTranslationImpl"

static_assert(
	USD_PREVIEW_SURFACE_MAX_UV_SETS <= MAX_MESH_TEXTURE_COORDS_MD,
	"UsdPreviewSurface materials can only have up to as many UV sets as MeshDescription supports!"
);

namespace UE::MeshTranslationImplInternal::Private
{
	UMaterialInterface* GetOrCreateTwoSidedVersionOfMaterial(
		UMaterialInterface* OneSidedMat,
		const FString& PrefixedTwoSidedHash,
		UUsdAssetCache3& AssetCache
	)
	{
		if (!OneSidedMat)
		{
			return nullptr;
		}

		UMaterialInterface* TwoSidedMat = nullptr;

		UMaterialInstance* OneSidedMaterialInstance = Cast<UMaterialInstance>(OneSidedMat);

		// Important to use Parent.Get() and not GetBaseMaterial() here because if our parent is the translucent we'll
		// get the reference UsdPreviewSurface instead, as that is also *its* reference
		UMaterialInterface* ReferenceMaterial = OneSidedMaterialInstance ? OneSidedMaterialInstance->Parent.Get() : nullptr;
		UMaterialInterface* ReferenceMaterialTwoSided = nullptr;
		if (ReferenceMaterial && UsdUnreal::MaterialUtils::IsReferencePreviewSurfaceMaterial(ReferenceMaterial))
		{
			FSoftObjectPath TwoSidedPath = UsdUnreal::MaterialUtils::GetTwoSidedVersionOfReferencePreviewSurfaceMaterial(ReferenceMaterial);
			ReferenceMaterialTwoSided = Cast<UMaterialInterface>(TwoSidedPath.TryLoad());
		}

		const FString& DesiredMaterialName = OneSidedMat->GetName() + UnrealIdentifiers::TwoSidedMaterialSuffix;
		const EObjectFlags DesiredFlags = OneSidedMat->GetFlags();

		UMaterialInstance* MI = Cast<UMaterialInstance>(OneSidedMat);
#if WITH_EDITOR
		UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(OneSidedMat);

		// One-sided material is an instance of one of our USD reference materials.
		// Just create an instance of the TwoSided version of the same reference material and copy parameter values.
		if (GIsEditor && MIC && ReferenceMaterialTwoSided)
		{
			bool bCreatedAsset = false;
			UMaterialInstanceConstant* TwoSidedMIC = AssetCache.GetOrCreateCachedAsset<UMaterialInstanceConstant>(
				PrefixedTwoSidedHash,
				DesiredMaterialName,
				DesiredFlags,
				&bCreatedAsset
			);

			if (bCreatedAsset && TwoSidedMIC)
			{
				TwoSidedMIC->SetParentEditorOnly(ReferenceMaterialTwoSided);
				TwoSidedMIC->CopyMaterialUniformParametersEditorOnly(OneSidedMat);
			}

			TwoSidedMat = TwoSidedMIC;
		}

		// One-sided material is some other material (e.g. MaterialX/MDL-generated material).
		// Create a new material instance of it and set the override to two-sided.
		else if (GIsEditor)
		{
			bool bCreatedAsset = false;
			UMaterialInstanceConstant* TwoSidedMIC = AssetCache.GetOrCreateCachedAsset<UMaterialInstanceConstant>(
				PrefixedTwoSidedHash,
				DesiredMaterialName,
				DesiredFlags,
				&bCreatedAsset
			);

			if (bCreatedAsset && TwoSidedMIC)
			{
				TwoSidedMIC->SetParentEditorOnly(OneSidedMat);
				TwoSidedMIC->BasePropertyOverrides.bOverride_TwoSided = true;
				TwoSidedMIC->BasePropertyOverrides.TwoSided = true;

				FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::Default, GMaxRHIShaderPlatform);
				UpdateContext.AddMaterialInstance(TwoSidedMIC);
			}

			TwoSidedMat = TwoSidedMIC;
		}

		else
#endif	  // WITH_EDITOR

			// At runtime all we can do is create another instance of our two-sided reference materials, we cannot set
			// another override
			if (MI && ReferenceMaterialTwoSided)
			{
				// Note how we're requesting just a UMaterialInstance here, instead of spelling out the MID. This because
				// if we're a runtime we may have a cooked MIC assigned to this hash, and in that case we want to use it
				// instead of overwriting it with a MID. Our creation func will ensure we create a MID as a fallback anyway
				bool bCreatedAsset = false;
				UMaterialInstance* TwoSidedMI = AssetCache.GetOrCreateCustomCachedAsset<UMaterialInstance>(
					PrefixedTwoSidedHash,
					DesiredMaterialName,
					DesiredFlags | RF_Transient,	// We never want MIDs to become assets in the content browser
					[ReferenceMaterialTwoSided](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
					{
						UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(ReferenceMaterialTwoSided, Outer, SanitizedName);
						NewMID->ClearFlags(NewMID->GetFlags());
						NewMID->SetFlags(FlagsToUse);
						return NewMID;
					},
					&bCreatedAsset
				);

				if (UMaterialInstanceDynamic* TwoSidedMID = Cast<UMaterialInstanceDynamic>(TwoSidedMI))
				{
					TwoSidedMID->CopyParameterOverrides(MI);
				}

				TwoSidedMat = TwoSidedMI;
			}

		return TwoSidedMat;
	}

	/**
	 * Returns Material in case it is already compatible with the provided MeshPrimvarToUVIndex, otherwise creates a
	 * new material instance of it, and sets the UVIndex material parameters to match a UVIndex setup that is compatible
	 * with the mesh.
	 * This function will also already cache and link the generated material.
	 */
	UMaterialInterface* CreatePrimvarCompatibleVersionOfMaterial(
		UMaterialInterface& Material,
		const TMap<FString, int32>& MeshPrimvarToUVIndex,
		UUsdAssetCache3* AssetCache,
		FUsdPrimLinkCache* PrimLinkCache,
		const FString& MaterialHashPrefix,
		bool bShareAssetsForIdenticalPrims
	)
	{
		UUsdMaterialAssetUserData* MaterialAssetUserData = Material.GetAssetUserData<UUsdMaterialAssetUserData>();
		if (!ensureMsgf(
				MaterialAssetUserData,
				TEXT("Expected material '%s' to have an UUsdMaterialAssetUserData at this point!"),
				*Material.GetPathName()
			))
		{
			return nullptr;
		}

		TArray<TSet<FString>> UVIndexToMeshPrimvars;
		UVIndexToMeshPrimvars.SetNum(USD_PREVIEW_SURFACE_MAX_UV_SETS);
		for (const TPair<FString, int32>& MeshPair : MeshPrimvarToUVIndex)
		{
			if (UVIndexToMeshPrimvars.IsValidIndex(MeshPair.Value))
			{
				UVIndexToMeshPrimvars[MeshPair.Value].Add(MeshPair.Key);
			}
		}

		// Check if mesh and material are compatible. Note that it's perfectly valid for the material to try reading
		// an UVIndex the mesh doesn't provide at all, or trying to read a primvar that doesn't exist on the mesh.
		bool bCompatible = true;
		for (const TPair<FString, int32>& Pair : MaterialAssetUserData->PrimvarToUVIndex)
		{
			const FString& MaterialPrimvar = Pair.Key;
			int32 MaterialUVIndex = Pair.Value;

			// If the mesh has the same primvar the material wants, it should be at the same UVIndex the material
			// will read from
			if (const int32* MeshUVIndex = MeshPrimvarToUVIndex.Find(MaterialPrimvar))
			{
				if (*MeshUVIndex != MaterialUVIndex)
				{
					bCompatible = false;
					// Don't break here so we can show all warnings
				}
			}
			else
			{
				USD_LOG_INFO(
					TEXT("Failed to find primvar '%s' needed by material '%s' on its assigned mesh. Available primvars and UV indices: %s"),
					*MaterialPrimvar,
					*Material.GetPathName(),
					*UsdUtils::StringifyMap(MeshPrimvarToUVIndex)
				);
			}

			// If the material is going to read from a given UVIndex that exists on the mesh, that UV set should
			// contain the primvar data that the material expects to read
			if (UVIndexToMeshPrimvars.IsValidIndex(MaterialUVIndex))
			{
				const TSet<FString>& CompatiblePrimvars = UVIndexToMeshPrimvars[MaterialUVIndex];
				if (!CompatiblePrimvars.Contains(MaterialPrimvar))
				{
					bCompatible = false;
					// Don't break here so we can show all warnings
				}
			}
		}
		if (bCompatible)
		{
			return &Material;
		}

		// We need to find or create another compatible material instead
		UMaterialInterface* CompatibleMaterial = nullptr;

		// First, let's create the target primvar UVIndex assignment that is compatible with this mesh.
		// We use an array of TPairs here so that we can sort these into a deterministic order for hashing later.
		TArray<TPair<FString, int32>> CompatiblePrimvarAndUVIndexPairs;
		CompatiblePrimvarAndUVIndexPairs.Reserve(MaterialAssetUserData->PrimvarToUVIndex.Num());
		for (const TPair<FString, int32>& Pair : MaterialAssetUserData->PrimvarToUVIndex)
		{
			const FString& MaterialPrimvar = Pair.Key;

			bool bFoundUVIndex = false;

			// Mesh has this primvar available at some UV index, point to it
			if (const int32* FoundMeshUVIndex = MeshPrimvarToUVIndex.Find(MaterialPrimvar))
			{
				int32 MeshUVIndex = *FoundMeshUVIndex;
				if (MeshUVIndex >= 0 && MeshUVIndex < USD_PREVIEW_SURFACE_MAX_UV_SETS)
				{
					CompatiblePrimvarAndUVIndexPairs.Add(TPair<FString, int32>{MaterialPrimvar, MeshUVIndex});
					bFoundUVIndex = true;
				}
			}

			if (!bFoundUVIndex)
			{
				// Point this primvar to read an unused UV index instead, since our mesh doesn't have this primvar
				CompatiblePrimvarAndUVIndexPairs.Add(TPair<FString, int32>{MaterialPrimvar, UNUSED_UV_INDEX});
			}
		}

		FString ExistingHash = AssetCache->GetHashForAsset(&Material);
		const bool bMaterialTrackedByAssetCache = !ExistingHash.IsEmpty();
		if (!bMaterialTrackedByAssetCache)
		{
			return nullptr;
		}

		// Generate a deterministic hash based on the original material hash and this primvar UVIndex assignment
		CompatiblePrimvarAndUVIndexPairs.Sort(
			[](const TPair<FString, int32>& LHS, const TPair<FString, int32>& RHS)
			{
				if (LHS.Key == RHS.Key)
				{
					return LHS.Value < RHS.Value;
				}
				else
				{
					return LHS.Key < RHS.Key;
				}
			}
		);
		FSHAHash Hash;
		FSHA1 SHA1;
		SHA1.UpdateWithString(*ExistingHash, ExistingHash.Len());
		for (const TPair<FString, int32>& Pair : CompatiblePrimvarAndUVIndexPairs)
		{
			SHA1.UpdateWithString(*Pair.Key, Pair.Key.Len());
			SHA1.Update((const uint8*)&Pair.Value, sizeof(Pair.Value));
		}
		SHA1.Final();
		SHA1.GetHash(&Hash.Hash[0]);

		// In theory we don't even need to add the prefix here because our ExistingHash will already have the same prefix...
		// However for consistency it's probably for the best to have both assets have the same prefix, so you can tell
		// from the hash that they originated from the same prim
		const FString PrefixedCompatibleHash = MaterialHashPrefix + Hash.ToString();

		bool bCreatedNew = false;
#if WITH_EDITOR
		if (GIsEditor)
		{
			UMaterialInstanceConstant* CompatibleMIC = AssetCache->GetOrCreateCachedAsset<UMaterialInstanceConstant>(
				PrefixedCompatibleHash,
				Material.GetName(),
				Material.GetFlags(),
				&bCreatedNew
			);

			if (bCreatedNew)
			{
				CompatibleMIC->SetParentEditorOnly(&Material);
			}

			CompatibleMaterial = CompatibleMIC;
		}
		else
#endif	  // WITH_EDITOR
		{
			// Note how we're requesting just a UMaterialInstance here, instead of spelling out the MID. This because
			// if we're a runtime we may have a cooked MIC assigned to this hash, and in that case we want to use it
			// instead of overwriting it with a MID. Our creation func will ensure we create a MID as a fallback anyway
			UMaterialInstance* CompatibleMI = AssetCache->GetOrCreateCustomCachedAsset<UMaterialInstance>(
				PrefixedCompatibleHash,
				Material.GetName(),
				Material.GetFlags() | RF_Transient,	   // We never want MIDs to become assets in the content browser
				[&Material](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
				{
					UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(&Material, Outer, SanitizedName);
					NewMID->ClearFlags(NewMID->GetFlags());
					NewMID->SetFlags(FlagsToUse);
					return NewMID;
				},
				&bCreatedNew
			);

			CompatibleMaterial = CompatibleMI;
		}

		TMap<FString, int32> CompatiblePrimvarToUVIndex;
		CompatiblePrimvarToUVIndex.Reserve(CompatiblePrimvarAndUVIndexPairs.Num());
		for (const TPair<FString, int32>& Pair : CompatiblePrimvarAndUVIndexPairs)
		{
			CompatiblePrimvarToUVIndex.Add(Pair);
		}

		// Update the AssetUserData whether we created a new material instance or reused one from the asset cache.
		// The compatible AssetUserData should always match the original except for the different PrimvarToUVIndex
		UUsdMaterialAssetUserData* CompatibleUserData = nullptr;
		if (CompatibleMaterial)
		{
			CompatibleUserData = DuplicateObject(MaterialAssetUserData, CompatibleMaterial);
			CompatibleUserData->PrimvarToUVIndex = CompatiblePrimvarToUVIndex;

			UsdUnreal::ObjectUtils::SetAssetUserData(CompatibleMaterial, CompatibleUserData);
		}

		// Now that the AssetUserData is done, actually set the UV index material parameters with the target indices
		UMaterialInstance* CompatibleInstance = Cast<UMaterialInstance>(CompatibleMaterial);
		if (bCreatedNew && CompatibleInstance && CompatibleUserData)
		{
			for (const TPair<FString, FString>& ParameterPair : CompatibleUserData->ParameterToPrimvar)
			{
				const FString& Parameter = ParameterPair.Key;
				const FString& Primvar = ParameterPair.Value;

				if (int32* UVIndex = CompatibleUserData->PrimvarToUVIndex.Find(Primvar))
				{
					// Force-disable using the texture at all if the mesh doesn't provide the primvar that should be
					// used to sample it with
					if (*UVIndex == UNUSED_UV_INDEX)
					{
						UsdUtils::SetScalarParameterValue(*CompatibleInstance, *FString::Printf(TEXT("Use%sTexture"), *Parameter), 0.0f);
					}
					else
					{
						UsdUtils::SetScalarParameterValue(
							*CompatibleInstance,
							*FString::Printf(TEXT("%sUVIndex"), *Parameter),
							static_cast<float>(*UVIndex)
						);
					}
				}
			}

#if WITH_EDITOR
			CompatibleInstance->PostEditChange();
#endif	  // WITH_EDITOR
		}

		if (CompatibleMaterial && CompatibleMaterial != &Material && PrimLinkCache)
		{
			for (const UE::FSdfPath& Prim : PrimLinkCache->GetPrimsForAsset(&Material))
			{
				PrimLinkCache->LinkAssetToPrim(Prim, CompatibleMaterial);
			}
		}

		return CompatibleMaterial;
	}
};	  // namespace UE::MeshTranslationImplInternal::Private

TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> MeshTranslationImpl::ResolveMaterialAssignmentInfo(
	const pxr::UsdPrim& UsdPrim,
	const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& AssignmentInfo,
	UUsdAssetCache3& AssetCache,
	FUsdPrimLinkCache& PrimLinkCache,
	EObjectFlags Flags,
	bool bShareAssetsForIdenticalPrims
)
{
	using namespace UsdUnreal::MaterialUtils;

	FScopedUnrealAllocs Allocs;

	TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolvedMaterials;
	if (AssignmentInfo.Num() == 0)
	{
		return ResolvedMaterials;
	}

	// Generating compatible materials is somewhat elaborate, so we'll cache the generated ones in this call in case we
	// have multiple material slots using the same material. The MeshPrimvarToUVIndex would always be the same for those
	// anyway, so we know we can reuse these compatible materials
	TMap<UMaterialInterface*, UMaterialInterface*> MaterialToCompatibleMaterial;
	const TMap<FString, int32>& MeshPrimvarToUVIndex = AssignmentInfo[0].PrimvarToUVIndex;

	uint32 GlobalResolvedMaterialIndex = 0;
	for (int32 InfoIndex = 0; InfoIndex < AssignmentInfo.Num(); ++InfoIndex)
	{
		const TArray<UsdUtils::FUsdPrimMaterialSlot>& Slots = AssignmentInfo[InfoIndex].Slots;

		for (int32 SlotIndex = 0; SlotIndex < Slots.Num(); ++SlotIndex, ++GlobalResolvedMaterialIndex)
		{
			const UsdUtils::FUsdPrimMaterialSlot& Slot = Slots[SlotIndex];
			UMaterialInterface* Material = nullptr;

			switch (Slot.AssignmentType)
			{
				case UsdUtils::EPrimAssignmentType::DisplayColor:
				{
					TOptional<FDisplayColorMaterial> DisplayColorDesc = FDisplayColorMaterial::FromString(Slot.MaterialSource);

					if (!DisplayColorDesc.IsSet())
					{
						continue;
					}

					const FSoftObjectPath* ReferencePath = GetReferenceMaterialPath(DisplayColorDesc.GetValue());
					if (!ReferencePath)
					{
						continue;
					}

					UMaterialInterface* ParentMaterial = Cast<UMaterialInterface>(ReferencePath->TryLoad());
					if (!ParentMaterial)
					{
						continue;
					}

					FString DisplayColorHash;
					{
						FSHAHash Hash;
						FSHA1 SHA1;
						SHA1.UpdateWithString(*Slot.MaterialSource, Slot.MaterialSource.Len());

						FString ReferencePathString = ReferencePath->ToString();
						SHA1.UpdateWithString(*ReferencePathString, ReferencePathString.Len());

						SHA1.Final();
						SHA1.GetHash(&Hash.Hash[0]);
						DisplayColorHash = Hash.ToString();
					}
					const FString PrefixedHash = UsdUtils::GetAssetHashPrefix(UsdPrim, bShareAssetsForIdenticalPrims) + DisplayColorHash;

					FString DisplayColorName = FString::Printf(
						TEXT("DisplayColor%s%s"),
						DisplayColorDesc->bHasOpacity ? TEXT("_Translucent") : TEXT(""),
						DisplayColorDesc->bIsDoubleSided ? TEXT("_TwoSided") : TEXT("")
					);

					bool bCreatedNew = false;
#if WITH_EDITOR
					if (GIsEditor)
					{
						UMaterialInstanceConstant* MaterialInstance = AssetCache.GetOrCreateCachedAsset<UMaterialInstanceConstant>(
							PrefixedHash,
							DisplayColorName,
							Flags,
							&bCreatedNew
						);

						if (bCreatedNew)
						{
							MaterialInstance->SetParentEditorOnly(ParentMaterial);
						}

						Material = MaterialInstance;
					}
					else
#endif	  // WITH_EDITOR
					{
						// Note how we're requesting just a UMaterialInstance here, instead of spelling out the MID. This because
						// if we're a runtime we may have a cooked MIC assigned to this hash, and in that case we want to use it
						// instead of overwriting it with a MID. Our creation func will ensure we create a MID as a fallback anyway
						UMaterialInstance* MaterialInstance = AssetCache.GetOrCreateCustomCachedAsset<UMaterialInstance>(
							PrefixedHash,
							DisplayColorName,
							Flags | RF_Transient,	 // We never want MIDs to become assets in the content browser
							[Material](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
							{
								UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(Material, Outer, SanitizedName);
								NewMID->ClearFlags(NewMID->GetFlags());
								NewMID->SetFlags(FlagsToUse);
								return NewMID;
							},
							&bCreatedNew
						);

						Material = MaterialInstance;
					}

					UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Material);
					break;
				}
				case UsdUtils::EPrimAssignmentType::MaterialPrim:
				{
					UE::FSdfPath MaterialPrimPath{*Slot.MaterialSource};

					bool bMaterialIsDirectReference = false;
					bool bUsingTwoSidedPreviewSurfaceAsFallback = false;

					// Here we have to pick the "best" material to use as reference, in case we need compatible/TwoSided versions.
					// They are returned from PrimLinkCache.GetAssetsForPrim in the most recent to least recent order, so
					// in general we want to pick the first ones we find that match our criteria (as the older assets may be leftover from
					// before we resynced something)
					TArray<UMaterialInterface*> ExistingMaterials = PrimLinkCache.GetAssetsForPrim<UMaterialInterface>(MaterialPrimPath);
					for (UMaterialInterface* ExistingMaterial : ExistingMaterials)
					{
						const bool bExistingIsTwoSided = ExistingMaterial->IsTwoSided();
						const bool bSidednessMatches = Slot.bMeshIsDoubleSided == bExistingIsTwoSided;

						// Prefer sticking with a material instance that has as parent one of our reference materials.
						// The idea here being that we have two approaches when making TwoSided and compatible
						// materials: A) Make the material compatible first, and then a TwoSided version of the
						// compatible; B) Make the material TwoSided first, and then a compatible version of the
						// TwoSided; We're going to chose B), for the reason that at runtime we can only make a material
						// TwoSided if it is an instance of our reference materials (as we can't manually change the
						// material base property overrides at runtime)
						//
						// Note that we may end up with MaterialX/MDL materials in here, so not being a direct reference
						// doesn't mean it's just another one of our material instances... It could be that an e.g.
						// non-UsdPreviewSurface MDL UMaterial is the best reference we can find
						bool bExistingIsDirectReference = false;
						if (UMaterialInstance* ExistingInstance = Cast<UMaterialInstance>(ExistingMaterial))
						{
							bExistingIsDirectReference = UsdUnreal::MaterialUtils::IsReferencePreviewSurfaceMaterial(ExistingInstance->Parent);
						}

						if (bSidednessMatches)
						{
							Material = ExistingMaterial;
							bUsingTwoSidedPreviewSurfaceAsFallback = false;

							if (bExistingIsDirectReference)
							{
								// This is a perfect match, we don't need to keep looking
								break;
							}
						}
						else if (Slot.bMeshIsDoubleSided && !bExistingIsTwoSided)
						{
							// Keep track of this one-sided material to turn it into TwoSided later

							// Prefer the one that is a direct preview surface reference if we have one already
							if (!Material || (!bMaterialIsDirectReference && bExistingIsDirectReference))
							{
								Material = ExistingMaterial;
								bMaterialIsDirectReference = bExistingIsDirectReference;
								bUsingTwoSidedPreviewSurfaceAsFallback = false;
							}
						}
						else
						{
							// If we're searching for a one sided material and just ran into an existing two-sided one we
							// should keep it just in case, but continue iterating: If a two-sided material is within
							// ExistingMaterials, its one-sided reference material should also be in there, so we'll find
							// something eventually. The only exception is in case the user manually set a two-sided material
							// as a one-sided preview surface parent material, which we'll keep track of and warn about
							if (!Material)
							{
								Material = ExistingMaterial;
							}
							if (!Slot.bMeshIsDoubleSided && ExistingMaterial && bExistingIsDirectReference && ExistingMaterial->IsTwoSided())
							{
								bUsingTwoSidedPreviewSurfaceAsFallback = true;
							}
						}
					}

					if (bUsingTwoSidedPreviewSurfaceAsFallback)
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"TwoSidedAsOneSidedParent",
								"Using two-sided material '{0}' for a one-sided mesh prim '{1}' as there is no better alternative. Make sure the ReferencePreviewSurfaceMaterial material set on the project settings is one-sided. You can assign a two-sided material on ReferencePreviewSurfaceTwoSidedMaterial instead."
							),
							FText::FromString(Material->GetPathName()),
							FText::FromString(UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath()))
						));
					}

					FString PrefixedMaterialHash = Material ? AssetCache.GetHashForAsset(Material) : FString{};
					FString HashPrefix = UsdUtils::GetAssetHashPrefix(
						UsdPrim.GetStage()->GetPrimAtPath(MaterialPrimPath),
						bShareAssetsForIdenticalPrims
					);

					// Need to create a two-sided material on-demand, *before* we make it compatible:
					// This because at runtime we can't just set the base property overrides, and just instead create a new
					// MID based on the TwoSided reference material, and the compatible material should be a MID of that MID
					if (Slot.bMeshIsDoubleSided && Material && !Material->IsTwoSided())
					{
						const FString PrefixedOneSidedHash = AssetCache.GetHashForAsset(Material);
						const FString PrefixedTwoSidedHash = PrefixedOneSidedHash + UnrealIdentifiers::TwoSidedMaterialSuffix;

						UMaterialInterface* TwoSidedMat = UE::MeshTranslationImplInternal::Private::GetOrCreateTwoSidedVersionOfMaterial(
							Material,
							PrefixedTwoSidedHash,
							AssetCache
						);

						if (TwoSidedMat)
						{
							// Update AssetUserData whether we generated a new material or reused one from the asset cache
							{
								UUsdMaterialAssetUserData* OneSidedUserData = UsdUnreal::ObjectUtils::GetAssetUserData<UUsdMaterialAssetUserData>(
									Material
								);
								ensure(OneSidedUserData);

								UUsdMaterialAssetUserData*
									TwoSidedUserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdMaterialAssetUserData>(TwoSidedMat);

								// Copy stuff from OneSidedMat when it makes sense, as it may have been regenerated
								if (OneSidedUserData && TwoSidedUserData)
								{
									TwoSidedUserData->ParameterToPrimvar = OneSidedUserData->ParameterToPrimvar;
									TwoSidedUserData->PrimvarToUVIndex = OneSidedUserData->PrimvarToUVIndex;
									TwoSidedUserData->StageIdentifierToMetadata = OneSidedUserData->StageIdentifierToMetadata;
									TwoSidedUserData->PrimPaths = OneSidedUserData->PrimPaths;
								}
							}

							Material = TwoSidedMat;
							PrefixedMaterialHash = PrefixedTwoSidedHash;
						}
						else
						{
							USD_LOG_WARNING(
								TEXT(
									"Failed to generate a two-sided material from the material prim at path '%s'. Falling back to using the single-sided material '%s' instead."
								),
								*Slot.MaterialSource,
								*Material->GetPathName()
							);
							PrefixedMaterialHash = PrefixedOneSidedHash;
						}
					}

					if (Material)
					{
						// Mark that we used this Material. We don't have to worry about our one-sided material because
						// if we have one, it will be the two-sided's reference material, and we collect reference materials
						// when collecting asset dependencies for import (which is the only mechanism that uses this
						// TouchAsset/ActiveAssets stuff)
						AssetCache.TouchAssetPath(Material);

						PrimLinkCache.LinkAssetToPrim(UE::FSdfPath{*Slot.MaterialSource}, Material);

						// Finally, try to make our generated material primvar-compatible. We do this last because this will
						// create another instance with the non-compatible material as reference material, which means we also
						// need that reference to be cached and linked for the asset cache to be able to handle dependencies
						// properly
						if (UMaterialInterface* AlreadyHandledMaterial = MaterialToCompatibleMaterial.FindRef(Material))
						{
							Material = AlreadyHandledMaterial;

							AssetCache.TouchAssetPath(Material);
							PrimLinkCache.LinkAssetToPrim(UE::FSdfPath{*Slot.MaterialSource}, Material);
						}
						else
						{
							UMaterialInterface*
								CompatibleMaterial = UE::MeshTranslationImplInternal::Private::CreatePrimvarCompatibleVersionOfMaterial(
									*Material,
									MeshPrimvarToUVIndex,
									&AssetCache,
									&PrimLinkCache,
									HashPrefix,
									bShareAssetsForIdenticalPrims
								);

							if (CompatibleMaterial)
							{
								MaterialToCompatibleMaterial.Add(Material, CompatibleMaterial);
								Material = CompatibleMaterial;
							}
						}
					}

					break;
				}
				case UsdUtils::EPrimAssignmentType::UnrealMaterial:
				{
					UObject* Object = FSoftObjectPath(Slot.MaterialSource).TryLoad();
					Material = Cast<UMaterialInterface>(Object);
					if (!Object)
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT("FailToLoadMaterial", "UE material '{0}' for prim '{1}' could not be loaded or was not found."),
							FText::FromString(Slot.MaterialSource),
							FText::FromString(UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath()))
						));
					}
					else if (!Material)
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"NotAMaterial",
								"Object '{0}' assigned as an Unreal Material for prim '{1}' is not actually a material (but instead a '{2}') and will not be used"
							),
							FText::FromString(Slot.MaterialSource),
							FText::FromString(UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath())),
							FText::FromString(Object->GetClass()->GetName())
						));
					}
					else if (!Material->IsTwoSided() && Slot.bMeshIsDoubleSided)
					{
						USD_LOG_WARNING(
							TEXT("Using one-sided UE material '%s' for doubleSided prim '%s'"),
							*Slot.MaterialSource,
							*UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath())
						);
					}

					break;
				}
				case UsdUtils::EPrimAssignmentType::None:
				default:
				{
					ensure(false);
					break;
				}
			}

			ResolvedMaterials.Add(&Slot, Material);
		}
	}

	return ResolvedMaterials;
}

void MeshTranslationImpl::SetMaterialOverrides(
	const pxr::UsdPrim& Prim,
	const TArray<UMaterialInterface*>& ExistingAssignments,
	UMeshComponent& MeshComponent,
	FUsdSchemaTranslationContext& Context
)
{
	FScopedUsdAllocs Allocs;

	pxr::SdfPath PrimPath = Prim.GetPath();
	pxr::UsdStageRefPtr Stage = Prim.GetStage();

	pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
	if (!Context.RenderContext.IsNone())
	{
		RenderContextToken = UnrealToUsd::ConvertToken(*Context.RenderContext.ToString()).Get();
	}

	pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
	if (!Context.MaterialPurpose.IsNone())
	{
		MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context.MaterialPurpose.ToString()).Get();
	}

	TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToAssignments;
	const bool bProvideMaterialIndices = false;	   // We have no use for material indices and it can be slow to retrieve, as it will iterate all faces

	// Extract material assignment info from prim if it is a LOD mesh
	bool bInterpretedLODs = false;
	if (Context.bAllowInterpretingLODs && UsdUtils::IsGeomMeshALOD(Prim))
	{
		TMap<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToMaterialInfoMap;
		TMap<int32, TSet<UsdUtils::FUsdPrimMaterialSlot>> CombinedSlotsForLODIndex;

		TFunction<bool(const pxr::UsdGeomMesh&, int32)> IterateLODs = [&Context,
																	   &RenderContextToken,
																	   &MaterialPurposeToken,
																	   &LODIndexToMaterialInfoMap,
																	   &CombinedSlotsForLODIndex](const pxr::UsdGeomMesh& LODMesh, int32 LODIndex)
		{
			// In here we need to parse the assignments again and can't rely on the cache because the info cache
			// only has info about the default variant selection state of the stage: It won't have info about the
			// LOD variant set setups as that involves actively toggling variants.
			// TODO: Make the cache rebuild collect this info. Right now is not a good time for this as that would
			// break the parallel-for setup that that function has
			UsdUtils::FUsdPrimMaterialAssignmentInfo LocalInfo = UsdUtils::GetPrimMaterialAssignments(
				LODMesh.GetPrim(),
				pxr::UsdTimeCode(Context.Time),
				bProvideMaterialIndices,
				RenderContextToken,
				MaterialPurposeToken
			);

			// When merging slots, we share the same material info across all LODs
			int32 LODIndexToUse = Context.bMergeIdenticalMaterialSlots ? 0 : LODIndex;
			TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToMaterialInfoMap.FindOrAdd(LODIndexToUse).Slots;
			TSet<UsdUtils::FUsdPrimMaterialSlot>& CombinedLODSlotsSet = CombinedSlotsForLODIndex.FindOrAdd(LODIndexToUse);

			for (UsdUtils::FUsdPrimMaterialSlot& LocalSlot : LocalInfo.Slots)
			{
				if (Context.bMergeIdenticalMaterialSlots)
				{
					if (!CombinedLODSlotsSet.Contains(LocalSlot))
					{
						LODSlots.Add(LocalSlot);
						CombinedLODSlotsSet.Add(LocalSlot);
					}
				}
				else
				{
					LODSlots.Add(LocalSlot);
				}
			}
			return true;
		};

		pxr::UsdPrim ParentPrim = Prim.GetParent();
		bInterpretedLODs = UsdUtils::IterateLODMeshes(ParentPrim, IterateLODs);
		if (bInterpretedLODs)
		{
			LODIndexToMaterialInfoMap.KeySort(TLess<int32>());
			for (TPair<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo>& Entry : LODIndexToMaterialInfoMap)
			{
				LODIndexToAssignments.Add(MoveTemp(Entry.Value));
			}
		}
	}

	// Refresh reference to Prim because variant switching potentially invalidated it
	pxr::UsdPrim ValidPrim = Stage->GetPrimAtPath(PrimPath);

	// Extract material assignment info from prim if its *not* a LOD mesh, or if we failed to parse LODs
	if (!bInterpretedLODs)
	{
		// Try to pull the material slot info from the info cache first, which is useful if ValidPrim is a collapsed
		// prim subtree: Querying it's material assignments directly is likely not what we want, as ValidPrim is
		// likely just some root Xform prim.
		// Note: This only works because we'll rebuild the cache when our material purpose/render context changes,
		// and because in USD relationships (and so material bindings) can't vary with time
		TOptional<TArray<UsdUtils::FUsdPrimMaterialSlot>> SubtreeSlots = Context.UsdInfoCache->GetSubtreeMaterialSlots(UE::FSdfPath{PrimPath});
		if (SubtreeSlots.IsSet())
		{
			UsdUtils::FUsdPrimMaterialAssignmentInfo& NewInfo = LODIndexToAssignments.Emplace_GetRef();
			NewInfo.Slots = MoveTemp(SubtreeSlots.GetValue());
		}
		else
		{
			LODIndexToAssignments = {UsdUtils::GetPrimMaterialAssignments(
				ValidPrim,
				pxr::UsdTimeCode(Context.Time),
				bProvideMaterialIndices,
				RenderContextToken,
				MaterialPurposeToken
			)};
		}
	}

	TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolvedMaterials;

	UUsdMeshAssetUserData* UserData = nullptr;
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(&MeshComponent))
	{
		if (UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh())
		{
			UserData = Mesh->GetAssetUserData<UUsdMeshAssetUserData>();
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(&MeshComponent))
	{
		if (USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			UserData = Mesh->GetAssetUserData<UUsdMeshAssetUserData>();
		}
	}
	else if (UGeometryCacheComponent* GeometryCacheComponent = Cast<UGeometryCacheComponent>(&MeshComponent))
	{
		if (UGeometryCache* Mesh = GeometryCacheComponent->GetGeometryCache())
		{
			UserData = Mesh->GetAssetUserData<UUsdMeshAssetUserData>();
		}
	}
	else
	{
		ensureMsgf(
			false,
			TEXT("Unexpected component class '%s' encountered when setting material overrides for prim '%s'!"),
			*MeshComponent.GetClass()->GetName(),
			*UsdToUnreal::ConvertPath(Prim.GetPrimPath())
		);
	}

	ensureMsgf(
		UserData,
		TEXT("Mesh assigned to component '%s' generated for prim '%s' should have an UUsdMeshAssetUserData at this point!"),
		*MeshComponent.GetPathName(),
		*UsdToUnreal::ConvertPath(Prim.GetPrimPath())
	);

	if (UserData && LODIndexToAssignments.Num() > 0)
	{
		// Stash our PrimvarToUVIndex in here, as that's where ResolveMaterialAssignmentInfo will look for it
		LODIndexToAssignments[0].PrimvarToUVIndex = UserData->PrimvarToUVIndex;

		ResolvedMaterials = MeshTranslationImpl::ResolveMaterialAssignmentInfo(
			ValidPrim,
			LODIndexToAssignments,
			*Context.UsdAssetCache,
			*Context.PrimLinkCache,
			Context.ObjectFlags,
			Context.bShareAssetsForIdenticalPrims
		);
	}

	// Compare resolved materials with existing assignments, and create overrides if we need to
	uint32 StaticMeshSlotIndex = 0;
	for (int32 LODIndex = 0; LODIndex < LODIndexToAssignments.Num(); ++LODIndex)
	{
		const TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToAssignments[LODIndex].Slots;
		for (int32 LODSlotIndex = 0; LODSlotIndex < LODSlots.Num(); ++LODSlotIndex, ++StaticMeshSlotIndex)
		{
			// If we don't even have as many existing assignments as we have overrides just stop here.
			// This should happen often now because we'll always at least attempt at setting overrides on every
			// component (but only ever set anything if we really need to).
			// Previously we only attempted setting overrides in case the component didn't "own" the mesh prim,
			// but now it is not feasible to do that given the global asset cache and how assets may have come
			// from an entirely new stage/session.
			if (!ExistingAssignments.IsValidIndex(StaticMeshSlotIndex))
			{
				break;
			}

			const UsdUtils::FUsdPrimMaterialSlot& Slot = LODSlots[LODSlotIndex];

			UMaterialInterface* Material = nullptr;
			if (UMaterialInterface** FoundMaterial = ResolvedMaterials.Find(&Slot))
			{
				Material = *FoundMaterial;
			}
			else
			{
				USD_LOG_ERROR(
					TEXT("Lost track of resolved material for slot '%d' of LOD '%d' for mesh '%s'"),
					LODSlotIndex,
					LODIndex,
					*UsdToUnreal::ConvertPath(Prim.GetPath())
				);
				continue;
			}

			UMaterialInterface* ExistingMaterial = ExistingAssignments[StaticMeshSlotIndex];
			if (ExistingMaterial == Material)
			{
				continue;
			}
			else
			{
				MeshComponent.SetMaterial(StaticMeshSlotIndex, Material);
			}
		}
	}
}

void MeshTranslationImpl::RecordSourcePrimsForMaterialSlots(
	const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& LODIndexToMaterialInfo,
	UUsdMeshAssetUserData* UserData
)
{
	if (!UserData)
	{
		return;
	}

	uint32 SlotIndex = 0;
	for (int32 LODIndex = 0; LODIndex < LODIndexToMaterialInfo.Num(); ++LODIndex)
	{
		const TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToMaterialInfo[LODIndex].Slots;

		for (int32 LODSlotIndex = 0; LODSlotIndex < LODSlots.Num(); ++LODSlotIndex, ++SlotIndex)
		{
			TArray<FString>& PrimPaths = UserData->MaterialSlotToPrimPaths.FindOrAdd(SlotIndex).PrimPaths;

			const UsdUtils::FUsdPrimMaterialSlot& Slot = LODSlots[LODSlotIndex];

			PrimPaths.Reserve(PrimPaths.Num() + Slot.PrimPaths.Num());
			for (const FString& SlotPrimPath : Slot.PrimPaths)
			{
				PrimPaths.AddUnique(SlotPrimPath);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
