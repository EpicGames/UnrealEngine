// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
PXR_NAMESPACE_CLOSE_SCOPE

class FUsdInfoCache;
class FUsdPrimLinkCache;
class UMaterialInterface;
class UMeshComponent;
class UUsdAssetCache3;
class UUsdMeshAssetUserData;
struct FUsdSchemaTranslationContext;
namespace UsdUtils
{
	struct FUsdPrimMaterialSlot;
	struct FUsdPrimMaterialAssignmentInfo;
}

/** Implementation that can be shared between the Skeleton translator and GeomMesh translators */
namespace MeshTranslationImpl
{
	/** Resolves the material assignments in AssignmentInfo, returning an UMaterialInterface for each material slot */
	TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolveMaterialAssignmentInfo(
		const pxr::UsdPrim& UsdPrim,
		const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& AssignmentInfo,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache,
		EObjectFlags Flags,
		bool bShareAssetsForIdenticalPrims
	);

	/**
	 * Sets the material overrides on MeshComponent according to the material assignments of the UsdGeomMesh Prim.
	 * Warning: This function will temporarily switch the active LOD variant if one exists, so it's *not* thread safe!
	 */
	void SetMaterialOverrides(
		const pxr::UsdPrim& Prim,
		const TArray<UMaterialInterface*>& ExistingAssignments,
		UMeshComponent& MeshComponent,
		FUsdSchemaTranslationContext& Context
	);

	void RecordSourcePrimsForMaterialSlots(
		const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& LODIndexToMaterialInfo,
		UUsdMeshAssetUserData* UserData
	);
}	 // namespace MeshTranslationImpl

#endif	  // #if USE_USD_SDK
