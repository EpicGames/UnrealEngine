// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGeomCameraTranslator.h"

#include "USDDrawModeComponent.h"
#include "USDMemory.h"
#include "USDPrimConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/UsdPrim.h"

#include "CineCameraActor.h"
#include "CineCameraComponent.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"

USceneComponent* FUsdGeomCameraTranslator::CreateComponents()
{
	USceneComponent* Component = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		Component = CreateAlternativeDrawModeComponents(DrawMode);
	}

	// Check if this prim actually originated from a CineCameraComponent that was the main camera component
	// of an ACineCameraActor. If so, then the USDGeomXformableTranslator that ran on our parent prim
	// likely already created an ACineCameraActor for us, and we can just take its already created
	// main camera component instead
	if (!Component)
	{
		FScopedUsdAllocs UsdAllocs;

		if (pxr::UsdPrim UsdPrim{GetPrim()})
		{
			pxr::UsdPrim UsdParentPrim = UsdPrim.GetParent();

			// Check if we're a child of the pseudoroot here because if we are USD will emit a warning if we try getting attributes from it...
			if (UsdParentPrim && !UsdParentPrim.IsPseudoRoot())
			{
				bool bIsCineCameraActorMainComponent = false;
				if (pxr::UsdAttribute Attr = UsdParentPrim.GetAttribute(UnrealToUsd::ConvertToken(TEXT("unrealCameraPrimName")).Get()))
				{
					pxr::TfToken CameraComponentPrimName;
					if (Attr.Get<pxr::TfToken>(&CameraComponentPrimName))
					{
						if (CameraComponentPrimName == UsdPrim.GetName())
						{
							bIsCineCameraActorMainComponent = true;
						}
					}
				}

				if (Context->ParentComponent && bIsCineCameraActorMainComponent)
				{
					ACineCameraActor* ParentOwnerActor = Cast<ACineCameraActor>(Context->ParentComponent->GetOwner());
					if (ParentOwnerActor && ParentOwnerActor->GetRootComponent() == Context->ParentComponent)
					{
						Component = ParentOwnerActor->GetCineCameraComponent();
					}
				}
			}
		}
	}

	if (!Component)
	{
		const bool bNeedsActor = true;
		Component = CreateComponentsEx({UCineCameraComponent::StaticClass()}, bNeedsActor);
	}

	// We pulled UpdateComponents outside CreateComponentsEx as in some cases we don't want to do it
	// right away (like on FUsdGeomPointInstancerTranslator::CreateComponents)
	UpdateComponents(Component);

	return Component;
}

void FUsdGeomCameraTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	if (!SceneComponent)
	{
		return;
	}

	FUsdGeomXformableTranslator::UpdateComponents(SceneComponent);

	UCineCameraComponent* CameraComponent = Cast<UCineCameraComponent>(SceneComponent);

	// If could be that we're a random Camera prim (and not an exported CineCameraComponent of an ACineCameraActor),
	// and so we'll have spawned a brand new ACineCameraActor for this prim alone and SceneComponent will be pointing
	// at its root component (which is just an USceneComponent)
	if (!CameraComponent)
	{
		if (ACineCameraActor* CameraActor = Cast<ACineCameraActor>(SceneComponent->GetOwner()))
		{
			if (SceneComponent == CameraActor->GetRootComponent())
			{
				CameraComponent = CameraActor->GetCineCameraComponent();
			}
		}
	}

	if (CameraComponent)
	{
		CameraComponent->Modify();

		FScopedUsdAllocs UsdAllocs;
		UsdToUnreal::ConvertGeomCamera(GetPrim(), *CameraComponent, Context->Time);
	}
}

bool FUsdGeomCameraTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	return false;
}

bool FUsdGeomCameraTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return false;
}

#endif	  // #if USE_USD_SDK
