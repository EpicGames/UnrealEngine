// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGeomMeshTranslator.h"

#if USE_USD_SDK

#include "MeshTranslationImpl.h"
#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDTranslatorUtils.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "CompGeom/FitKDOP3.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Misc/App.h"
#include "Misc/SecureHash.h"
#include "Modules/ModuleManager.h"
#include "NaniteDefinitions.h"
#include "PhysicsEngine/BodySetup.h"
#include "Rendering/NaniteResources.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "StaticMeshResources.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "ConvexDecompTool.h"
#include "IMeshBuilderModule.h"
#include "MeshBudgetProjectSettings.h"
#endif	  // WITH_EDITOR

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usd/typed.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/subset.h"
#include "pxr/usd/usdPhysics/collisionAPI.h"
#include "pxr/usd/usdPhysics/meshCollisionAPI.h"
#include "pxr/usd/usdPhysics/tokens.h"
#include "pxr/usd/usdSkel/bindingAPI.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDGeomMeshTranslator"

static bool GBuildReversedIndexBuffer = false;
static FAutoConsoleVariableRef CVarBuildReversedIndexBuffer(
	TEXT("USD.StaticMesh.BuildSettings.BuildReversedIndexBuffer"),
	GBuildReversedIndexBuffer,
	TEXT("Enable to optimize mesh in mirrored transform. Double index buffer size.")
);

static bool GUseFullPrecisionUVs = false;
static FAutoConsoleVariableRef CVarUseFullPrecisionUVs(
	TEXT("USD.StaticMesh.BuildSettings.UseFullPrecisionUVs"),
	GUseFullPrecisionUVs,
	TEXT("If true, UVs will be stored at full floating point precision.")
);

static bool GUseHighPrecisionTangentBasis = false;
static FAutoConsoleVariableRef CVarUseHighPrecisionTangentBasis(
	TEXT("USD.StaticMesh.BuildSettings.UseHighPrecisionTangentBasis"),
	GUseHighPrecisionTangentBasis,
	TEXT("If true, Tangents will be stored at 16 bit vs 8 bit precision.")
);

static int32 GNaniteSettingsNormalPrecision = -1;
static FAutoConsoleVariableRef CVarNaniteSettingsNormalPrecision(
	TEXT("USD.Nanite.Settings.NormalPrecision"),
	GNaniteSettingsNormalPrecision,
	TEXT("Normal Precision in bits. -1 is auto.")
);

static int32 GNaniteSettingsTangentPrecision = -1;
static FAutoConsoleVariableRef CVarNaniteSettingsTangentPrecision(
	TEXT("USD.Nanite.Settings.TangentPrecision"),
	GNaniteSettingsTangentPrecision,
	TEXT("Tangent Precision in bits. -1 is auto.")
);

namespace UsdGeomMeshTranslatorImpl
{
	bool ShouldEnableNanite(
		const TArray<FMeshDescription>& LODIndexToMeshDescription,
		const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& LODIndexToMaterialInfo,
		const FUsdSchemaTranslationContext& Context,
		const UE::FUsdPrim& Prim
	)
	{
		if (LODIndexToMeshDescription.Num() < 1)
		{
			return false;
		}

		FScopedUsdAllocs UsdAllocs;

		UE::FSdfPath PrimPath = Prim.GetPrimPath();

		pxr::UsdPrim UsdPrim{Prim};
		if (!UsdPrim)
		{
			return false;
		}

		bool bHasNaniteOverrideEnabled = false;

		// We want Nanite because of an override on Prim
		if (pxr::UsdAttribute NaniteOverride = UsdPrim.GetAttribute(UnrealIdentifiers::UnrealNaniteOverride))
		{
			pxr::TfToken OverrideValue;
			if (NaniteOverride.Get(&OverrideValue))
			{
				if (OverrideValue == UnrealIdentifiers::UnrealNaniteOverrideEnable)
				{
					bHasNaniteOverrideEnabled = true;
					USD_LOG_INFO(
						TEXT("Trying to enable Nanite for mesh generated for prim '%s' as the '%s' attribute is set to '%s'"),
						*PrimPath.GetString(),
						*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealNaniteOverride),
						*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealNaniteOverrideEnable)
					);
				}
				else if (OverrideValue == UnrealIdentifiers::UnrealNaniteOverrideDisable)
				{
					USD_LOG_INFO(
						TEXT("Not enabling Nanite for mesh generated for prim '%s' as the '%s' attribute is set to '%s'"),
						*PrimPath.GetString(),
						*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealNaniteOverride),
						*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealNaniteOverrideDisable)
					);
					return false;
				}
			}
		}

		// We want Nanite because the mesh is large enough for the threshold, which is set to something valid
		if (!bHasNaniteOverrideEnabled)
		{
			const int32 NumTriangles = LODIndexToMeshDescription[0].Triangles().Num();
			if (NumTriangles < Context.NaniteTriangleThreshold)
			{
				return false;
			}
		}

		if (Context.UsdInfoCache)
		{
			TOptional<uint64> SubtreeSectionCount = Context.UsdInfoCache->GetSubtreeMaterialSlotCount(PrimPath);

			if (!SubtreeSectionCount.IsSet() || SubtreeSectionCount.GetValue() > NANITE_MAX_CLUSTER_MATERIALS)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"NaniteSlotWarning",
						"Not enabling Nanite for mesh generated for prim '{0}' as LOD0 has '{1}' material slots, which is above the Nanite limit of '{2}'"
					),
					FText::FromString(PrimPath.GetString()),
					SubtreeSectionCount.GetValue(),
					NANITE_MAX_CLUSTER_MATERIALS
				));
				return false;
			}
		}

#if !WITH_EDITOR
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT("NaniteRuntimeWarning", "Not enabling Nanite for mesh generated for prim '{0}' as we can't setup Nanite during runtime"),
			FText::FromString(PrimPath.GetString())
		));
		return false;
#else
		return true;
#endif
	}

	/** Returns true if material infos have changed on the StaticMesh */
	bool ProcessStaticMeshMaterials(
		const pxr::UsdPrim& UsdPrim,
		const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& LODIndexToMaterialInfo,
		UStaticMesh& StaticMesh,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache,
		float Time,
		EObjectFlags Flags,
		bool bShareAssetsForIdenticalPrims
	)
	{
		bool bMaterialAssignementsHaveChanged = false;

		TArray<UMaterialInterface*> ExistingAssignments;
		for (const FStaticMaterial& StaticMaterial : StaticMesh.GetStaticMaterials())
		{
			ExistingAssignments.Add(StaticMaterial.MaterialInterface);
		}

		TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolvedMaterials = MeshTranslationImpl::ResolveMaterialAssignmentInfo(
			UsdPrim,
			LODIndexToMaterialInfo,
			AssetCache,
			PrimLinkCache,
			Flags,
			bShareAssetsForIdenticalPrims
		);

		uint32 StaticMeshSlotIndex = 0;
		for (int32 LODIndex = 0; LODIndex < LODIndexToMaterialInfo.Num(); ++LODIndex)
		{
			const TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToMaterialInfo[LODIndex].Slots;

			for (int32 LODSlotIndex = 0; LODSlotIndex < LODSlots.Num(); ++LODSlotIndex, ++StaticMeshSlotIndex)
			{
				const UsdUtils::FUsdPrimMaterialSlot& Slot = LODSlots[LODSlotIndex];

				UMaterialInterface* Material = UMaterial::GetDefaultMaterial(MD_Surface);
				if (UMaterialInterface** FoundMaterial = ResolvedMaterials.Find(&Slot))
				{
					Material = *FoundMaterial;
				}
				else
				{
					USD_LOG_ERROR(
						TEXT("Failed to resolve material '%s' for slot '%d' of LOD '%d' for mesh '%s'"),
						*Slot.MaterialSource,
						LODSlotIndex,
						LODIndex,
						*UsdToUnreal::ConvertPath(UsdPrim.GetPath())
					);
					continue;
				}

				// Create and set the static material
				FStaticMaterial StaticMaterial(Material, *LexToString(StaticMeshSlotIndex));
				if (!StaticMesh.GetStaticMaterials().IsValidIndex(StaticMeshSlotIndex))
				{
					StaticMesh.GetStaticMaterials().Add(MoveTemp(StaticMaterial));
					bMaterialAssignementsHaveChanged = true;
				}
				else if (StaticMesh.GetStaticMaterials()[StaticMeshSlotIndex] != StaticMaterial)
				{
					StaticMesh.GetStaticMaterials()[StaticMeshSlotIndex] = MoveTemp(StaticMaterial);
					bMaterialAssignementsHaveChanged = true;
				}

#if WITH_EDITOR
				// Setup the section map so that our LOD material index is properly mapped to the static mesh material index
				// At runtime we don't ever parse these variants as LODs so we don't need this
				if (StaticMesh.GetSectionInfoMap().IsValidSection(LODIndex, LODSlotIndex))
				{
					FMeshSectionInfo MeshSectionInfo = StaticMesh.GetSectionInfoMap().Get(LODIndex, LODSlotIndex);

					if (MeshSectionInfo.MaterialIndex != StaticMeshSlotIndex)
					{
						MeshSectionInfo.MaterialIndex = StaticMeshSlotIndex;
						StaticMesh.GetSectionInfoMap().Set(LODIndex, LODSlotIndex, MeshSectionInfo);

						bMaterialAssignementsHaveChanged = true;
					}
				}
				else
				{
					FMeshSectionInfo MeshSectionInfo;
					MeshSectionInfo.MaterialIndex = StaticMeshSlotIndex;

					StaticMesh.GetSectionInfoMap().Set(LODIndex, LODSlotIndex, MeshSectionInfo);

					bMaterialAssignementsHaveChanged = true;
				}
#endif	  // WITH_EDITOR
			}
		}

#if WITH_EDITOR
		StaticMesh.GetOriginalSectionInfoMap().CopyFrom(StaticMesh.GetSectionInfoMap());
#endif	  // WITH_EDITOR

		return bMaterialAssignementsHaveChanged;
	}

	// If UsdMesh is a LOD, will parse it and all of the other LODs, and and place them in OutLODIndexToMeshDescription and OutLODIndexToMaterialInfo.
	// Note that these other LODs will be hidden in other variants, and won't show up on traversal unless we actively switch the variants (which we do
	// here). We use a separate function for this because there is a very specific set of conditions where we successfully can do this, and we want to
	// fall back to just parsing UsdMesh as a simple single-LOD mesh if we fail.
	bool TryLoadingMultipleLODs(
		const UE::FUsdPrim& MeshPrim,
		TArray<FMeshDescription>& OutLODIndexToMeshDescription,
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& OutLODIndexToMaterialInfo,
		FUsdCombinedPrimMetadata& OutLODMetadata,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const FUsdMetadataImportOptions& MetadataOptions
	)
	{
		if (!MeshPrim)
		{
			return false;
		}

		UE::FUsdPrim ParentPrim = MeshPrim.GetParent();

		TMap<int32, FMeshDescription> LODIndexToMeshDescriptionMap;
		TMap<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToMaterialInfoMap;

		UsdToUnreal::FUsdMeshConversionOptions OptionsCopy = Options;

		TSet<FString> AllPrimvars;
		TSet<FString> PreferredPrimvars;

		// Here we are choosing to preemptively traverse all LODs to fetch the combined primvars that we'll use for
		// each UV index for the static mesh as a whole
		TFunction<bool(const pxr::UsdGeomMesh&, int32)> CombinePrimvars =
			[&AllPrimvars, &PreferredPrimvars](const pxr::UsdGeomMesh& LODMesh, int32 LODIndex)
		{
			TArray<TUsdStore<pxr::UsdGeomPrimvar>> MeshPrimvars = UsdUtils::GetUVSetPrimvars(LODMesh.GetPrim(), TNumericLimits<int32>::Max());

			for (const TUsdStore<pxr::UsdGeomPrimvar>& MeshPrimvar : MeshPrimvars)
			{
				FString PrimvarName = UsdToUnreal::ConvertToken(MeshPrimvar.Get().GetName());
				PrimvarName.RemoveFromStart(TEXT("primvars:"));

				AllPrimvars.Add(PrimvarName);

				// Keep track of which primvars are texCoord2f as we always want to prefer these over other float2s
				if (MeshPrimvar.Get().GetTypeName().GetRole() == pxr::SdfValueTypeNames->TexCoord2f.GetRole())
				{
					PreferredPrimvars.Add(PrimvarName);
				}
			}

			return true;
		};
		if (!UsdUtils::IterateLODMeshes(ParentPrim, CombinePrimvars))
		{
			return false;
		}
		TMap<FString, int32> PrimvarToUVIndex = UsdUtils::CombinePrimvarsIntoUVSets(AllPrimvars, PreferredPrimvars);

		TFunction<bool(const pxr::UsdGeomMesh&, int32)> ConvertLOD =
			[&OptionsCopy, &Options, &LODIndexToMeshDescriptionMap, &LODIndexToMaterialInfoMap, &OutLODMetadata, &MetadataOptions, &PrimvarToUVIndex](
				const pxr::UsdGeomMesh& LODMesh,
				int32 LODIndex
			)
		{
			FMeshDescription TempMeshDescription;

			// When merging slots we combine all our assignments on a single MaterialAssignmentInfo
			UsdUtils::FUsdPrimMaterialAssignmentInfo& TempMaterialInfo = LODIndexToMaterialInfoMap.FindOrAdd(
				Options.bMergeIdenticalMaterialSlots ? 0 : LODIndex
			);
			TempMaterialInfo.PrimvarToUVIndex = PrimvarToUVIndex;

			FStaticMeshAttributes StaticMeshAttributes(TempMeshDescription);
			StaticMeshAttributes.Register();

			bool bSuccess = true;

			{
				FScopedUsdAllocs Allocs;
				// The user can't manually hide/unhide a particular LOD in the engine after it is imported, so we should probably bake
				// the particular LOD visibility into the combined mesh. Note how we don't use computed visibility here, as we only really
				// care if this mesh in particular has been marked as invisible
				pxr::TfToken Visibility;
				pxr::UsdAttribute VisibilityAttr = LODMesh.GetVisibilityAttr();
				if (VisibilityAttr && VisibilityAttr.Get(&Visibility, Options.TimeCode) && Visibility == pxr::UsdGeomTokens->inherited)
				{
					// If we're interpreting LODs we must bake the transform from each LOD Mesh into the vertices, because there's no guarantee
					// all LODs have the same transform, so we can't just put the transforms directly on the component. If we are not interpreting
					// LODs we can do that though
					// TODO: Handle resetXformOp here
					bool bResetXformStack = false;
					FTransform MeshTransform = FTransform::Identity;
					bSuccess &= UsdToUnreal::ConvertXformable(
						LODMesh.GetPrim().GetStage(),
						LODMesh,
						MeshTransform,
						Options.TimeCode.GetValue(),
						&bResetXformStack
					);

					if (bSuccess)
					{
						OptionsCopy.AdditionalTransform = MeshTransform * Options.AdditionalTransform;

						bSuccess &= UsdToUnreal::ConvertGeomMesh(LODMesh, TempMeshDescription, TempMaterialInfo, OptionsCopy);
					}
				}
			}

			if (bSuccess)
			{
				LODIndexToMeshDescriptionMap.Add(LODIndex, MoveTemp(TempMeshDescription));

				if (MetadataOptions.bCollectMetadata)
				{
					UsdToUnreal::ConvertMetadata(
						LODMesh.GetPrim(),
						OutLODMetadata,
						MetadataOptions.BlockedPrefixFilters,
						MetadataOptions.bInvertFilters,
						MetadataOptions.bCollectFromEntireSubtrees
					);
				}
			}

			return true;
		};
		if (!UsdUtils::IterateLODMeshes(ParentPrim, ConvertLOD))
		{
			return false;
		}

		// Place them in order as we can't have e.g. LOD0 and LOD2 without LOD1, and there's no reason downstream code needs to care about this
		{
			LODIndexToMeshDescriptionMap.KeySort(TLess<int32>());
			OutLODIndexToMeshDescription.Reset(LODIndexToMeshDescriptionMap.Num());
			for (TPair<int32, FMeshDescription>& Entry : LODIndexToMeshDescriptionMap)
			{
				const int32 OldLODIndex = Entry.Key;
				OutLODIndexToMeshDescription.Add(MoveTemp(Entry.Value));
			}

			LODIndexToMaterialInfoMap.KeySort(TLess<int32>());
			OutLODIndexToMaterialInfo.Reset(LODIndexToMaterialInfoMap.Num());
			for (TPair<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo>& Entry : LODIndexToMaterialInfoMap)
			{
				const int32 OldLODIndex = Entry.Key;
				OutLODIndexToMaterialInfo.Add(MoveTemp(LODIndexToMaterialInfoMap[OldLODIndex]));
			}
		}

		return true;
	}

	void LoadMeshDescriptions(
		UE::FUsdPrim MeshPrim,
		TArray<FMeshDescription>& OutLODIndexToMeshDescription,
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& OutLODIndexToMaterialInfo,
		FUsdCombinedPrimMetadata& OutLODMetadata,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const FUsdMetadataImportOptions& MetadataOptions,
		bool bInterpretLODs
	)
	{
		if (!MeshPrim)
		{
			return;
		}

		// Ignore meshes from disabled purposes
		if (!EnumHasAllFlags(Options.PurposesToLoad, IUsdPrim::GetPurpose(MeshPrim)))
		{
			return;
		}

		UE::FUsdStage Stage = MeshPrim.GetStage();
		UE::FSdfPath Path = MeshPrim.GetPrimPath();

		bool bInterpretedLODs = false;
		if (bInterpretLODs)
		{
			bInterpretedLODs = TryLoadingMultipleLODs(
				MeshPrim,
				OutLODIndexToMeshDescription,
				OutLODIndexToMaterialInfo,
				OutLODMetadata,
				Options,
				MetadataOptions
			);

			// Have to be very careful here as flipping through LODs invalidates prim references, so we need to
			// re-acquire them
			MeshPrim = Stage.GetPrimAtPath(Path);
		}

		// If we've managed to interpret LODs, we won't place our mesh transform on the static mesh component itself
		// (c.f. FUsdGeomXformableTranslator::UpdateComponents), and will instead expect it to be baked into the mesh.
		// So here we do that
		bool bSuccess = true;
		FTransform MeshTransform = FTransform::Identity;
		if (bInterpretedLODs && OutLODIndexToMeshDescription.Num() > 1)
		{
			// TODO: Handle resetXformOp here
			bool bResetXformStack = false;
			bSuccess &= UsdToUnreal::ConvertXformable(Stage, UE::FUsdTyped{MeshPrim}, MeshTransform, Options.TimeCode.GetValue(), &bResetXformStack);
		}

		if (!bInterpretedLODs)
		{
			FMeshDescription TempMeshDescription;
			UsdUtils::FUsdPrimMaterialAssignmentInfo TempMaterialInfo;

			FStaticMeshAttributes StaticMeshAttributes(TempMeshDescription);
			StaticMeshAttributes.Register();

			if (bSuccess)
			{
				UsdToUnreal::FUsdMeshConversionOptions OptionsCopy = Options;
				OptionsCopy.AdditionalTransform = MeshTransform * Options.AdditionalTransform;
				OptionsCopy.bMergeIdenticalMaterialSlots = false;	 // We only merge for collapsed meshes

				FScopedUsdAllocs Allocs;
				pxr::UsdGeomMesh UsdMesh{MeshPrim};

				bSuccess &= UsdToUnreal::ConvertGeomMesh(UsdMesh, TempMeshDescription, TempMaterialInfo, OptionsCopy);
			}

			if (bSuccess)
			{
				OutLODIndexToMeshDescription = {MoveTemp(TempMeshDescription)};
				OutLODIndexToMaterialInfo = {MoveTemp(TempMaterialInfo)};

				if (MetadataOptions.bCollectMetadata)
				{
					UsdToUnreal::ConvertMetadata(
						MeshPrim,
						OutLODMetadata,
						MetadataOptions.BlockedPrefixFilters,
						MetadataOptions.bInvertFilters,
						MetadataOptions.bCollectFromEntireSubtrees
					);
				}
			}
		}
	}

	void GetCollisionSettings(
		const pxr::UsdPrim& InUsdPrim,
		EUsdCollisionType InFallbackCollisionType,
		bool& OutEnableCollision,
		EUsdCollisionType& OutApproximationType
	)
	{
		bool bPrimEnablesCollision = false;
		pxr::UsdPhysicsCollisionAPI CollisionAPI{InUsdPrim};
		if (CollisionAPI)
		{
			if (pxr::UsdAttribute CollisionAttr = CollisionAPI.GetCollisionEnabledAttr())
			{
				CollisionAttr.Get(&bPrimEnablesCollision);
			}
		}

		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.EnableCollision"));
		const bool bCvarEnablesCollision = CVar && CVar->GetBool();

		OutEnableCollision = bCvarEnablesCollision && (bPrimEnablesCollision || InFallbackCollisionType != EUsdCollisionType::None);

		if (!OutEnableCollision)
		{
			OutApproximationType = EUsdCollisionType::None;	   // It doesn't matter, so let's always return None for consistency (helps with hashing)
			return;
		}

		OutApproximationType = CollisionAPI ? UsdUtils::GetCollisionApproximationType(InUsdPrim) : InFallbackCollisionType;
	}

	UStaticMesh* CreateStaticMesh(
		const UE::FUsdPrim& Prim,
		TArray<FMeshDescription>& LODIndexToMeshDescription,
		FUsdSchemaTranslationContext& Context,
		const FString& MeshName,
		const bool bShouldEnableNanite,
		bool& bOutIsNew
	)
	{
		UStaticMesh* StaticMesh = nullptr;

		bool bHasValidMeshDescription = false;

		FSHAHash AllLODHash;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CreateStaticMesh::HashingMeshDescription);

			FSHA1 SHA1;

			for (const FMeshDescription& MeshDescription : LODIndexToMeshDescription)
			{
				const bool bSkipTransientAttributes = true;
				FSHAHash LODHash = FStaticMeshOperations::ComputeSHAHash(MeshDescription, bSkipTransientAttributes);
				SHA1.Update(&LODHash.Hash[0], sizeof(LODHash.Hash));

				if (!MeshDescription.IsEmpty())
				{
					bHasValidMeshDescription = true;
				}
			}

			// Put whether we want Nanite or not within the hash, so that the user could have one instance of the mesh without Nanite and another
			// with Nanite if they want to (using the override parameters). This also nicely handles a couple of edge cases:
			//	- What if we change a mesh from having Nanite disabled to enabled, or vice-versa (e.g. by changing the threshold)? We'd reuse the mesh
			// from the asset cache
			//    in that case, so we'd need to rebuild it;
			//  - What if multiple meshes on the scene hash the same, but only one of them has a Nanite override attribute?
			// If we always enabled Nanite when either the mesh from the asset cache or the new prim wanted it, we wouldn't be able to turn Nanite off
			// from a single mesh that once had it enabled: It would always find the old Nanite-enabled mesh on the cache and leave it enabled. If we
			// always set Nanite to whatever the current prim wants, we could handle a single mesh turning Nanite on/off alright, but then we can't
			// handle the case where multiple meshes on the scene hash the same and only one of them has the override: The last prim would win, and
			// they'd all be randomly either enabled or disabled. Note that we could also fix these problems by trying to check if a mesh is reused
			// due to being in the cache from an old instance of the stage, or due to being used by another prim, but that doesn't seem like a good
			// path to go down Additionally, hashing this bool also prevents us from having to force-rebuild a mesh to switch its Nanite flag, which
			// could be tricky to do since some of these build steps are async/thread-pool based.
			SHA1.Update(reinterpret_cast<const uint8*>(&bShouldEnableNanite), sizeof(bShouldEnableNanite));

			// Hash these so that if we update them and reload we'll regenerate static meshes
			static IConsoleVariable* NormalThresholdCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.MeshNormalRepairThreshold"));
			static IConsoleVariable* SkipTangentComputationCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.SkipMeshTangentComputation"));
			const float MeshNormalRepairThreshold = NormalThresholdCvar ? NormalThresholdCvar->GetFloat() : 1.0f;
			const bool SkipTangentComputation = SkipTangentComputationCvar && SkipTangentComputationCvar->GetBool();
			SHA1.Update(reinterpret_cast<const uint8*>(&MeshNormalRepairThreshold), sizeof(MeshNormalRepairThreshold));
			SHA1.Update(reinterpret_cast<const uint8*>(&SkipTangentComputation), sizeof(SkipTangentComputation));

			// We don't want to reuse static meshes if they need different collision shapes
			{
				bool bIsCollisionEnabled = false;
				EUsdCollisionType Approximation = EUsdCollisionType::None;
				GetCollisionSettings(Prim, Context.FallbackCollisionType, bIsCollisionEnabled, Approximation);

				SHA1.Update(reinterpret_cast<const uint8*>(&bIsCollisionEnabled), sizeof(bIsCollisionEnabled));
				SHA1.Update(reinterpret_cast<const uint8*>(&Approximation), sizeof(Approximation));
			}

			SHA1.Final();
			SHA1.GetHash(&AllLODHash.Hash[0]);
		}

		FString PrefixedAssetHash = UsdUtils::GetAssetHashPrefix(Prim, Context.bShareAssetsForIdenticalPrims) + AllLODHash.ToString();

		FString DesiredName = FPaths::GetBaseFilename(MeshName);

		// If we're just opening the stage we'll ignore the LOD variant sets and just parse the first LOD mesh as a regular Mesh prim,
		// which would have produced a static mesh just named something like "LOD0" or "LOD1". This should provide a more descriptive name,
		// like "Cube_LOD0" instead
		if (!Context.bAllowInterpretingLODs && UsdUtils::IsGeomMeshALOD(Prim))
		{
			DesiredName = FPaths::GetBaseFilename(FPaths::GetPath(MeshName)) + TEXT("_") + DesiredName;
		}

		if (Context.UsdAssetCache)
		{
			StaticMesh = Context.UsdAssetCache->GetOrCreateCachedAsset<UStaticMesh>(PrefixedAssetHash, DesiredName, Context.ObjectFlags, &bOutIsNew);
		}

		if (StaticMesh && bHasValidMeshDescription && bOutIsNew)
		{
#if WITH_EDITOR
			for (int32 LODIndex = 0; LODIndex < LODIndexToMeshDescription.Num(); ++LODIndex)
			{
				FMeshDescription& MeshDescription = LODIndexToMeshDescription[LODIndex];

				UsdUtils::RepairNormalsAndTangents(MeshName, MeshDescription);

				FStaticMeshSourceModel& SourceModel = StaticMesh->AddSourceModel();
				SourceModel.BuildSettings.bGenerateLightmapUVs = false;
				SourceModel.BuildSettings.bRecomputeNormals = false;
				SourceModel.BuildSettings.bRecomputeTangents = false;
				SourceModel.BuildSettings.bBuildReversedIndexBuffer = GBuildReversedIndexBuffer;
				SourceModel.BuildSettings.bUseFullPrecisionUVs = GUseFullPrecisionUVs;
				SourceModel.BuildSettings.bUseHighPrecisionTangentBasis = GUseHighPrecisionTangentBasis;
				SourceModel.BuildSettings.bRemoveDegenerates = true;	// Note: This may get rid of the entire mesh if it is all invalid

				FMeshDescription* StaticMeshDescription = StaticMesh->CreateMeshDescription(LODIndex);
				check(StaticMeshDescription);
				*StaticMeshDescription = MoveTemp(MeshDescription);
			}

			FMeshBudgetProjectSettingsUtils::SetLodGroupForStaticMesh(StaticMesh);

#endif	  // WITH_EDITOR

			StaticMesh->SetLightingGuid();
		}

		return StaticMesh;
	}

	void PreBuildStaticMesh(UStaticMesh& StaticMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdGeomMeshTranslatorImpl::PreBuildStaticMesh);

		if (StaticMesh.GetRenderData())
		{
			StaticMesh.ReleaseResources();
			StaticMesh.ReleaseResourcesFence.Wait();
		}

		StaticMesh.SetRenderData(MakeUnique<FStaticMeshRenderData>());
		StaticMesh.CreateBodySetup();
		StaticMesh.MarkAsNotHavingNavigationData();	   // Needed or else it will warn if we try cooking with body setup

#if WITH_EDITOR
		// If it's a Nanite Assembly, cache references
		StaticMesh.RecacheNaniteAssemblyReferences();
#endif
	}

	bool BuildStaticMesh(UStaticMesh& StaticMesh, const FStaticFeatureLevel& FeatureLevel, TArray<FMeshDescription>& LODIndexToMeshDescription)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdGeomMeshTranslatorImpl::BuildStaticMesh);

#if WITH_EDITOR
		if (LODIndexToMeshDescription.Num() == 0 && !StaticMesh.IsNaniteAssembly())
		{
			return false;
		}

		ITargetPlatformManagerModule& TargetPlatformManager = GetTargetPlatformManagerRef();
		ITargetPlatform* RunningPlatform = TargetPlatformManager.GetRunningTargetPlatform();
		check(RunningPlatform);

		const FStaticMeshLODSettings& LODSettings = RunningPlatform->GetStaticMeshLODSettings();
		StaticMesh.GetRenderData()->Cache(RunningPlatform, &StaticMesh, LODSettings);
#else
		if (LODIndexToMeshDescription.Num() == 0)
		{
			return false;
		}

		StaticMesh.GetRenderData()->AllocateLODResources(LODIndexToMeshDescription.Num());

		// Build render data from each mesh description
		for (int32 LODIndex = 0; LODIndex < LODIndexToMeshDescription.Num(); ++LODIndex)
		{
			FStaticMeshLODResources& LODResources = StaticMesh.GetRenderData()->LODResources[LODIndex];

			FMeshDescription& MeshDescription = LODIndexToMeshDescription[LODIndex];
			TVertexInstanceAttributesConstRef<FVector4f>
				MeshDescriptionColors = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);

			// Compute normals here if necessary because they're not going to be computed via the regular static mesh build pipeline at runtime
			// (i.e. StaticMeshBuilder is not available at runtime)
			// We need polygon info because ComputeTangentsAndNormals uses it to repair the invalid vertex normals/tangents
			// Can't calculate just the required polygons as ComputeTangentsAndNormals is parallel and we can't guarantee thread-safe access patterns
			FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription);
			FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, EComputeNTBsFlags::UseMikkTSpace);

			// Manually set this as it seems the UStaticMesh only sets this whenever the mesh is serialized, which we won't do
			LODResources.bHasColorVertexData = MeshDescriptionColors.GetNumElements() > 0;

			StaticMesh.BuildFromMeshDescription(MeshDescription, LODResources);
		}

#if RHI_RAYTRACING
		if (IsRayTracingAllowed() && StaticMesh.bSupportRayTracing)
		{
			StaticMesh.GetRenderData()->InitializeRayTracingRepresentationFromRenderingLODs();
		}
#endif	  // RHI_RAYTRACING
#endif	  // WITH_EDITOR

		return true;
	}

	void PostBuildStaticMesh(UStaticMesh& StaticMesh, const TArray<FMeshDescription>& LODIndexToMeshDescription)
	{
		// For runtime builds, the analogue for this stuff is already done from within BuildFromMeshDescriptions
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdGeomMeshTranslatorImpl::PostBuildStaticMesh);

		if (FApp::CanEverRender() || !FPlatformProperties::RequiresCookedData())
		{
			StaticMesh.InitResources();

			FBoxSphereBounds::Builder MeshBoundsBuilder;
			FStaticMeshRenderData& RenderData = *StaticMesh.GetRenderData();
#if WITH_EDITOR
			// Fetch the MeshDescription from the StaticMesh because we'll have moved it away from LODIndexToMeshDescription CreateStaticMesh
			if (const FMeshDescription* MeshDescription = StaticMesh.GetMeshDescription(0))
			{
				MeshBoundsBuilder += MeshDescription->GetBounds();
			}
#else
			// Fetch the MeshDescription from the imported LODIndexToMeshDescription as StaticMesh.GetMeshDescription is editor-only
			MeshBoundsBuilder += LODIndexToMeshDescription[0].GetBounds();
#endif	  // WITH_EDITOR

			// Include the Nanite mesh bounds as well. This is important for Nanite Assemblies
			if (RenderData.HasValidNaniteData())
			{
				const FBoxSphereBounds3f NaniteBounds3f = RenderData.NaniteResourcesPtr->MeshBounds;
				FBoxSphereBounds NaniteBounds;
				NaniteBounds.Origin = FVector(NaniteBounds3f.Origin);
				NaniteBounds.BoxExtent = FVector(NaniteBounds3f.BoxExtent);
				NaniteBounds.SphereRadius = NaniteBounds3f.SphereRadius;
				
				MeshBoundsBuilder += NaniteBounds;
			}
			RenderData.Bounds = MeshBoundsBuilder;

			StaticMesh.CalculateExtendedBounds();
#if WITH_EDITOR
			StaticMesh.ClearMeshDescriptions();	   // Clear mesh descriptions to reduce memory usage, they are kept only in bulk data form
#endif
		}
	}
}	 // namespace UsdGeomMeshTranslatorImpl

namespace UE::UsdCollision::Private
{

	class IMeshData
	{
	public:
		virtual int32 GetNumVertices() const = 0;
		virtual int32 GetNumIndices() const = 0;
		virtual FVector3f GetVertexPosition(int32 Index) const = 0;
		virtual int32 GetVertexIndex(int32 Index) const = 0;
		virtual FBox GetBoundingBox() const = 0;
		virtual bool IsValid() const = 0;
		virtual ~IMeshData()
		{
		}
	};

	class FMeshDescriptionWrapper : public IMeshData
	{
	public:
		FMeshDescriptionWrapper(const FMeshDescription& InMeshDescription)
			: MeshDescription(InMeshDescription)
		{
		}

		int32 GetNumVertices() const override
		{
			return MeshDescription.Vertices().Num();
		}
		int32 GetNumIndices() const override
		{
			return MeshDescription.VertexInstances().Num();
		}
		FVector3f GetVertexPosition(int32 Index) const override
		{
			return MeshDescription.GetVertexPositions()[Index];
		}
		int32 GetVertexIndex(int32 Index) const override
		{
			return MeshDescription.GetVertexInstanceVertex(Index);
		}
		FBox GetBoundingBox() const override
		{
			return MeshDescription.ComputeBoundingBox();
		}
		bool IsValid() const override
		{
			return MeshDescription.Vertices().Num() > 0;
		};

	private:
		const FMeshDescription& MeshDescription;
	};

	class FRenderDataWrapper : public IMeshData
	{
	public:
		FRenderDataWrapper(const FStaticMeshLODResources& InRenderData)
			: RenderData(InRenderData)
		{
		}

		int32 GetNumVertices() const override
		{
			return RenderData.GetNumVertices();
		}
		int32 GetNumIndices() const override
		{
			return RenderData.IndexBuffer.GetNumIndices();
		}
		FVector3f GetVertexPosition(int32 Index) const override
		{
			return RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(Index);
		}
		int32 GetVertexIndex(int32 Index) const override
		{
			return RenderData.IndexBuffer.GetIndex(Index);
		}
		bool IsValid() const override
		{
			return RenderData.GetNumVertices() > 0;
		};
		FBox GetBoundingBox() const override
		{
			if (!BoundingBox.IsValid)
			{
				for (int32 Index = 0; Index < RenderData.GetNumVertices(); ++Index)
				{
					BoundingBox += static_cast<FVector>(RenderData.VertexBuffers.PositionVertexBuffer.VertexPosition(Index));
				}
			}
			return BoundingBox;
		}

	private:
		const FStaticMeshLODResources& RenderData;
		mutable FBox BoundingBox;
	};

	class FMeshData : public IMeshData
	{
	public:
		FMeshData(const UStaticMesh& StaticMesh)
			: MeshData(nullptr)
		{
#if WITH_EDITORONLY_DATA
			FMeshDescription* MeshDescription = StaticMesh.GetMeshDescription(0);
			if (!MeshDescription || MeshDescription->Vertices().Num() == 0)
			{
				return;
			}

			MeshData = new FMeshDescriptionWrapper(*MeshDescription);
#else
			if (StaticMesh.GetRenderData()->LODResources.Num() > 0)
			{
				MeshData = new FRenderDataWrapper(StaticMesh.GetRenderData()->LODResources[0]);
			}
#endif
		}

		FMeshData(const FMeshDescription& MeshDescription)
			: MeshData(nullptr)
		{
			if (MeshDescription.Vertices().Num() > 0)
			{
				MeshData = new FMeshDescriptionWrapper(MeshDescription);
			}
		}

		virtual ~FMeshData()
		{
			delete MeshData;
		}

		int32 GetNumVertices() const override
		{
			return IsValid() ? MeshData->GetNumVertices() : 0;
		}
		int32 GetNumIndices() const override
		{
			return IsValid() ? MeshData->GetNumIndices() : 0;
		}
		FVector3f GetVertexPosition(int32 Index) const override
		{
			return IsValid() ? MeshData->GetVertexPosition(Index) : FVector3f::ZeroVector;
		}
		int32 GetVertexIndex(int32 Index) const override
		{
			return IsValid() ? MeshData->GetVertexIndex(Index) : 0;
		}
		FBox GetBoundingBox() const override
		{
			return IsValid() ? MeshData->GetBoundingBox() : FBox();
		}
		bool IsValid() const override
		{
			return MeshData ? MeshData->IsValid() : false;
		}

	private:
		IMeshData* MeshData;
	};

	// Based on GeomFitUtils.cpp

	// k-DOP (k-Discrete Oriented Polytopes) Direction Vectors
	constexpr float RCP_SQRT2 = 0.70710678118654752440084436210485f;

	TArray<FVector> KDopDir18 = {
		FVector(1.f, 0.f, 0.f),
		FVector(-1.f, 0.f, 0.f),
		FVector(0.f, 1.f, 0.f),
		FVector(0.f, -1.f, 0.f),
		FVector(0.f, 0.f, 1.f),
		FVector(0.f, 0.f, -1.f),
		FVector(0.f, RCP_SQRT2, RCP_SQRT2),
		FVector(0.f, -RCP_SQRT2, -RCP_SQRT2),
		FVector(0.f, RCP_SQRT2, -RCP_SQRT2),
		FVector(0.f, -RCP_SQRT2, RCP_SQRT2),
		FVector(RCP_SQRT2, 0.f, RCP_SQRT2),
		FVector(-RCP_SQRT2, 0.f, -RCP_SQRT2),
		FVector(RCP_SQRT2, 0.f, -RCP_SQRT2),
		FVector(-RCP_SQRT2, 0.f, RCP_SQRT2),
		FVector(RCP_SQRT2, RCP_SQRT2, 0.f),
		FVector(-RCP_SQRT2, -RCP_SQRT2, 0.f),
		FVector(RCP_SQRT2, -RCP_SQRT2, 0.f),
		FVector(-RCP_SQRT2, RCP_SQRT2, 0.f)
	};

	void GenerateKDopAsSimpleCollision(const FMeshData& MeshData, const TArray<FVector>& Dirs, FKAggregateGeom& CollisionShapes)
	{
		TArray<FVector> HullVertices;
		UE::Geometry::FitKDOPVertices3<double>(
			Dirs,
			MeshData.GetNumVertices(),
			[&MeshData](int32 VertexId)
			{
				return static_cast<FVector>(MeshData.GetVertexPosition(VertexId));
			},
			HullVertices
		);

		FKConvexElem ConvexElem;
		ConvexElem.VertexData = HullVertices;
		// Note: UpdateElemBox also computes the convex hull indices
		ConvexElem.UpdateElemBox();

		CollisionShapes.ConvexElems.Add(ConvexElem);
	}

	void CalcBoundingSphere(const FMeshData& MeshData, FSphere& Sphere)
	{
		FBox Box;
		FVector MinIx[3];
		FVector MaxIx[3];

		bool bFirstVertex = true;
		for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
		{
			FVector Point = static_cast<FVector>(MeshData.GetVertexPosition(VertexID));
			if (bFirstVertex)
			{
				// First, find AABB, remembering furthest points in each dir.
				Box.Min = Point;
				Box.Max = Box.Min;

				MinIx[0] = Point;
				MinIx[1] = Point;
				MinIx[2] = Point;

				MaxIx[0] = Point;
				MaxIx[1] = Point;
				MaxIx[2] = Point;
				bFirstVertex = false;
				continue;
			}

			// X //
			if (Point.X < Box.Min.X)
			{
				Box.Min.X = Point.X;
				MinIx[0] = Point;
			}
			else if (Point.X > Box.Max.X)
			{
				Box.Max.X = Point.X;
				MaxIx[0] = Point;
			}

			// Y //
			if (Point.Y < Box.Min.Y)
			{
				Box.Min.Y = Point.Y;
				MinIx[1] = Point;
			}
			else if (Point.Y > Box.Max.Y)
			{
				Box.Max.Y = Point.Y;
				MaxIx[1] = Point;
			}

			// Z //
			if (Point.Z < Box.Min.Z)
			{
				Box.Min.Z = Point.Z;
				MinIx[2] = Point;
			}
			else if (Point.Z > Box.Max.Z)
			{
				Box.Max.Z = Point.Z;
				MaxIx[2] = Point;
			}
		}

		const FVector Extremes[3] = {(MaxIx[0] - MinIx[0]), (MaxIx[1] - MinIx[1]), (MaxIx[2] - MinIx[2])};

		// Now find extreme points furthest apart, and initial center and radius of sphere.
		float MaxDist2 = 0.f;
		for (int32 i = 0; i < 3; ++i)
		{
			const float TmpDist2 = Extremes[i].SizeSquared();
			if (TmpDist2 > MaxDist2)
			{
				MaxDist2 = TmpDist2;
				Sphere.Center = (MinIx[i] + (0.5f * Extremes[i]));
				Sphere.W = 0.f;
			}
		}

		const FVector Extents = FVector(Extremes[0].X, Extremes[1].Y, Extremes[2].Z);

		// radius and radius squared
		float Radius = 0.5f * Extents.GetMax();
		float Radius2 = FMath::Square(Radius);

		// Now check each point lies within this sphere. If not - expand it a bit.
		for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
		{
			const FVector CenterToPoint = static_cast<FVector>(MeshData.GetVertexPosition(VertexID)) - Sphere.Center;
			const float CenterToPoint2 = CenterToPoint.SizeSquared();

			// If this point is outside our current bounding sphere's radius
			if (CenterToPoint2 > Radius2)
			{
				// ..expand radius just enough to include this point.
				const float PointRadius = FMath::Sqrt(CenterToPoint2);
				Radius = 0.5f * (Radius + PointRadius);
				Radius2 = FMath::Square(Radius);

				Sphere.Center += ((PointRadius - Radius) / PointRadius * CenterToPoint);
			}
		}

		Sphere.W = Radius;
	}

	// This is the one thats already used by unreal.
	// Seems to do better with more symmetric input...
	void CalcBoundingSphere2(const FMeshData& MeshData, FSphere& Sphere)
	{
		FVector Center = MeshData.GetBoundingBox().GetCenter();

		Sphere.Center = Center;
		Sphere.W = 0.0f;

		for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
		{
			float Dist2 = FVector::DistSquared(static_cast<FVector>(MeshData.GetVertexPosition(VertexID)), Sphere.Center);
			if (Dist2 > Sphere.W)
				Sphere.W = Dist2;
		}
		Sphere.W = FMath::Sqrt(Sphere.W);
	}

	void GenerateSphereAsSimpleCollision(const FMeshData& MeshData, FKAggregateGeom& CollisionShapes)
	{
		FSphere Sphere, Sphere2, BestSphere;

		// Calculate bounding sphere.
		CalcBoundingSphere(MeshData, Sphere);
		CalcBoundingSphere2(MeshData, Sphere2);

		if (Sphere.W < Sphere2.W)
			BestSphere = Sphere;
		else
			BestSphere = Sphere2;

		// Don't use if radius is zero.
		if (BestSphere.W <= 0.f)
		{
			return;
		}

		FKSphereElem SphereElem;
		SphereElem.Center = BestSphere.Center;
		SphereElem.Radius = BestSphere.W;
		CollisionShapes.SphereElems.Add(SphereElem);
	}

	void GenerateBoxAsSimpleCollision(const FMeshData& MeshData, FKAggregateGeom& CollisionShapes)
	{
		// Calculate bounding Box.
		FVector Center, Extents;
		MeshData.GetBoundingBox().GetCenterAndExtents(Center, Extents);

		FKBoxElem BoxElem;
		BoxElem.Center = Center;
		BoxElem.X = Extents.X * 2.0f;
		BoxElem.Y = Extents.Y * 2.0f;
		BoxElem.Z = Extents.Z * 2.0f;
		CollisionShapes.BoxElems.Add(BoxElem);
	}

	void CalcBoundingSphyl(const FMeshData& MeshData, FSphere& Sphere, float& Length, FRotator& Rotation)
	{
		FVector Center, Extents;
		MeshData.GetBoundingBox().GetCenterAndExtents(Center, Extents);

		Sphere.Center = Center;

		// Work out best axis aligned orientation (longest side)
		double Extent = Extents.GetMax();
		if (Extent == Extents.X)
		{
			Rotation = FRotator(90.f, 0.f, 0.f);
			Extents.X = 0.0f;
		}
		else if (Extent == Extents.Y)
		{
			Rotation = FRotator(0.f, 0.f, 90.f);
			Extents.Y = 0.0f;
		}
		else
		{
			Rotation = FRotator(0.f, 0.f, 0.f);
			Extents.Z = 0.0f;
		}

		// Cleared the largest axis above, remaining determines the radius
		float Radius = Extents.GetMax();
		float Radius2 = FMath::Square(Radius);

		// Now check each point lies within this the radius. If not - expand it a bit.
		for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
		{
			FVector CenterToPoint = static_cast<FVector>(MeshData.GetVertexPosition(VertexID)) - Sphere.Center;
			CenterToPoint = Rotation.UnrotateVector(CenterToPoint);

			const float PointRadius2 = CenterToPoint.SizeSquared2D();	 // Ignore Z here...

			// If this point is outside our current bounding sphere's radius
			if (PointRadius2 > Radius2)
			{
				// ..expand radius just enough to include this point.
				const float PointRadius = FMath::Sqrt(PointRadius2);
				Radius = 0.5f * (Radius + PointRadius);
				Radius2 = FMath::Square(Radius);
			}
		}

		// The length is the longest side minus the radius.
		float HalfLength = FMath::Max(0.0f, Extent - Radius);

		// Now check each point lies within the length. If not - expand it a bit.
		for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
		{
			FVector CenterToPoint = static_cast<FVector>(MeshData.GetVertexPosition(VertexID)) - Sphere.Center;
			CenterToPoint = Rotation.UnrotateVector(CenterToPoint);

			// If this point is outside our current bounding sphyl's length
			if (FMath::Abs(CenterToPoint.Z) > HalfLength)
			{
				const bool bFlip = (CenterToPoint.Z < 0.f ? true : false);
				const FVector Origin(0.f, 0.f, (bFlip ? -HalfLength : HalfLength));

				const float PointRadius2 = (Origin - CenterToPoint).SizeSquared();

				// If this point is outside our current bounding sphyl's radius
				if (PointRadius2 > Radius2)
				{
					FVector ClosestPoint;
					FMath::SphereDistToLine(Origin, Radius, CenterToPoint, (bFlip ? FVector(0.f, 0.f, 1.f) : FVector(0.f, 0.f, -1.f)), ClosestPoint);

					// Don't accept zero as a valid diff when we know it's outside the sphere (saves needless retest on further iterations of like
					// points)
					HalfLength += FMath::Max<float>(FMath::Abs(CenterToPoint.Z - ClosestPoint.Z), 1.e-6f);
				}
			}
		}

		Sphere.W = Radius;
		Length = HalfLength * 2.0f;
	}

	void GenerateSphylAsSimpleCollision(const FMeshData& MeshData, FKAggregateGeom& CollisionShapes)
	{
		FSphere Sphere;
		float Length;
		FRotator Rotation;

		// Calculate bounding box.
		CalcBoundingSphyl(MeshData, Sphere, Length, Rotation);

		// Don't use if radius is zero.
		if (Sphere.W <= 0.f)
		{
			return;
		}

		// If height is zero, then a sphere would be better (should we just create one instead?)
		if (Length <= 0.f)
		{
			Length = SMALL_NUMBER;
		}

		FKSphylElem SphylElem;
		SphylElem.Center = Sphere.Center;
		SphylElem.Rotation = Rotation;
		SphylElem.Radius = Sphere.W;
		SphylElem.Length = Length;
		CollisionShapes.SphylElems.Add(SphylElem);
	}

	void ConvertMeshToSimpleCollision(const FMeshData& MeshData, FKAggregateGeom& CollisionShapes)
	{
		FKConvexElem ConvexElem;
		ConvexElem.VertexData.Reserve(MeshData.GetNumVertices());

		for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
		{
			ConvexElem.VertexData.Add(static_cast<FVector>(MeshData.GetVertexPosition(VertexID)));
		}

		// Note: UpdateElemBox also computes the convex hull indices
		ConvexElem.UpdateElemBox();
		CollisionShapes.ConvexElems.Add(ConvexElem);
	}

	void CollectCustomCollisionShapes(const pxr::UsdPrim& UsdPrim, FKAggregateGeom& CollisionShapes)
	{
		using namespace UsdUtils;
		UsdToUnreal::FUsdMeshConversionOptions Options;
		Options.PurposesToLoad = EUsdPurpose::Guide;	// custom collision mesh must have guide purpose
		Options.bMergeIdenticalMaterialSlots = false;

		FTransform ParentTransform;
		UsdToUnreal::ConvertXformable(UsdPrim.GetStage(), pxr::UsdGeomMesh{UsdPrim}, ParentTransform, Options.TimeCode.GetValue());

		for (const pxr::UsdPrim& ChildPrim : UsdPrim.GetParent().GetChildren())
		{
			if (UsdUtils::IsCollisionMesh(ChildPrim))
			{
				FMeshDescription MeshDescription;
				UsdUtils::FUsdPrimMaterialAssignmentInfo MaterialAssignments;

				FStaticMeshAttributes MeshAttributes(MeshDescription);
				MeshAttributes.Register();

				pxr::UsdGeomMesh UsdMesh{ChildPrim};

				// Since the custom collision shapes are "attached" to the static mesh, its transform must be
				// removed when their own transforms are baked into the collision shapes
				UsdToUnreal::ConvertXformable(ChildPrim.GetStage(), UsdMesh, Options.AdditionalTransform, Options.TimeCode.GetValue());
				Options.AdditionalTransform = Options.AdditionalTransform * ParentTransform.Inverse();

				if (UsdToUnreal::ConvertGeomMesh(UsdMesh, MeshDescription, MaterialAssignments, Options))
				{
					// By default, use the custom mesh as convex hull, which takes care of the UCX_ prefix
					EUsdCollisionType Approximation{EUsdCollisionType::ConvexHull};

					// Use the FBX custom collision name convention as a hint
					FString PrimName(UsdToUnreal::ConvertToken(ChildPrim.GetName()));
					if (PrimName.StartsWith(TEXT("UBX_")))
					{
						Approximation = EUsdCollisionType::Cube;
					}
					else if (PrimName.StartsWith(TEXT("UCP_")))
					{
						Approximation = EUsdCollisionType::Capsule;
					}
					else if (PrimName.StartsWith(TEXT("USP_")))
					{
						Approximation = EUsdCollisionType::Sphere;
					}

					switch (Approximation)
					{
						case EUsdCollisionType::ConvexHull:
						{
							GenerateKDopAsSimpleCollision(MeshDescription, KDopDir18, CollisionShapes);
							break;
						}
						case EUsdCollisionType::Cube:
						{
							GenerateBoxAsSimpleCollision(MeshDescription, CollisionShapes);
							break;
						}
						case EUsdCollisionType::Capsule:
						{
							GenerateSphylAsSimpleCollision(MeshDescription, CollisionShapes);
							break;
						}
						case EUsdCollisionType::Sphere:
						{
							GenerateSphereAsSimpleCollision(MeshDescription, CollisionShapes);
							break;
						}
					}
				}
			}
		}
	}

	void SetupSimpleCollision(const pxr::UsdPrim& UsdPrim, UStaticMesh& StaticMesh, EUsdCollisionType FallbackCollisionType)
	{
		using namespace UsdUtils;

		UBodySetup* BodySetup = StaticMesh.GetBodySetup();
		if (!BodySetup)
		{
			return;
		}

		FMeshData MeshData(StaticMesh);
		if (!MeshData.IsValid())
		{
			return;
		}

		bool bIsCollisionEnabled = false;
		EUsdCollisionType Approximation = EUsdCollisionType::None;
		UsdGeomMeshTranslatorImpl::GetCollisionSettings(UsdPrim, FallbackCollisionType, bIsCollisionEnabled, Approximation);

		if (!bIsCollisionEnabled)
		{
			BodySetup->Modify();
			BodySetup->RemoveSimpleCollision();
			BodySetup->bNeverNeedsCookedCollisionData = true;	 // this will prevent the complex collision mesh from being generated
			BodySetup->DefaultInstance.SetCollisionEnabled(ECollisionEnabled::NoCollision);
			BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

			return;
		}

		// Note we don't set a CollisionProfileName here to let the default value of the StaticMesh be used
		// (The CreateBodySetup() inside PreBuildStaticMesh() should set it to BlockAll)
		BodySetup->Modify();
		BodySetup->RemoveSimpleCollision();
		BodySetup->bNeverNeedsCookedCollisionData = false;
		BodySetup->DefaultInstance.SetCollisionEnabled(ECollisionEnabled::QueryAndPhysics);
		BodySetup->CollisionTraceFlag = CTF_UseDefault;

		switch (Approximation)
		{
			case EUsdCollisionType::None:
			{
				// No approximation but allow for custom collision shapes
				// If there are no custom collision shapes, then use complex as simple
				FKAggregateGeom CollisionShapes;
				CollectCustomCollisionShapes(UsdPrim, CollisionShapes);
				if (CollisionShapes.GetElementCount())
				{
					BodySetup->AddCollisionFrom(CollisionShapes);
				}
				else
				{
					BodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;
				}
				break;
			}
#if WITH_EDITOR
			case EUsdCollisionType::ConvexDecomposition:
			case EUsdCollisionType::MeshSimplification:
			{
				// Could use IMeshReduction for MeshSimplification, but it will get converted to a single convex hull
				// so the result would be not as good as convex decomposition anyway

				TArray<FVector3f> Vertices;
				Vertices.Reserve(MeshData.GetNumVertices());
				for (int32 VertexID = 0; VertexID < MeshData.GetNumVertices(); ++VertexID)
				{
					Vertices.Add(MeshData.GetVertexPosition(VertexID));
				}

				TArray<uint32> Indices;
				Indices.Reserve(MeshData.GetNumIndices());
				for (int32 InstanceID = 0; InstanceID < MeshData.GetNumIndices(); ++InstanceID)
				{
					Indices.Add(MeshData.GetVertexIndex(InstanceID));
				}

				// Do not perform any action if we have invalid input
				if (Vertices.Num() >= 3 && Indices.Num() >= 3)
				{
					const uint32 HullCount = 32;
					const int32 MaxHullVertices = 32;
					DecomposeMeshToHulls(BodySetup, Vertices, Indices, HullCount, MaxHullVertices);
				}
				break;
			}
#endif	  // WITH_EDITOR
			case EUsdCollisionType::ConvexHull:
			{
				GenerateKDopAsSimpleCollision(MeshData, KDopDir18, BodySetup->AggGeom);
				break;
			}
			case EUsdCollisionType::Sphere:
			{
				GenerateSphereAsSimpleCollision(MeshData, BodySetup->AggGeom);
				break;
			}
			case EUsdCollisionType::Cube:
			{
				GenerateBoxAsSimpleCollision(MeshData, BodySetup->AggGeom);
				break;
			}
			case EUsdCollisionType::Capsule:
			{
				GenerateSphylAsSimpleCollision(MeshData, BodySetup->AggGeom);
				break;
			}
			case EUsdCollisionType::CustomMesh:
			{
				ConvertMeshToSimpleCollision(MeshData, BodySetup->AggGeom);
				break;
			}
			default:
				break;
		}

		BodySetup->CreatePhysicsMeshes();

#if WITH_EDITORONLY_DATA
		StaticMesh.SetCustomizedCollision(true);
#endif

		const bool bIsUpdate = true;
		StaticMesh.CreateNavCollision(bIsUpdate);
	}
}	 // namespace UE::UsdCollision::Private

FBaseBuildStaticMeshTaskChain::FBaseBuildStaticMeshTaskChain(
	const TSharedRef<FUsdSchemaTranslationContext>& InContext,
	const UE::FSdfPath& InPrimPath
)
	: PrimPath(InPrimPath)
	, Context(InContext)
{
}

void FBaseBuildStaticMeshTaskChain::SetupTasks()
{
#if WITH_EDITOR
	// Commit mesh description (Async)
	Do(
		ESchemaTranslationLaunchPolicy::Async,
		[this]()
		{
			UStaticMesh::FCommitMeshDescriptionParams Params;
			Params.bMarkPackageDirty = false;
			Params.bUseHashAsGuid = true;

			int32 NumMeshDescriptions = LODIndexToMeshDescription.Num();
			if (StaticMesh->IsNaniteAssembly() && NumMeshDescriptions == 0)
			{
				NumMeshDescriptions = 1;
			}

			for (int32 LODIndex = 0; LODIndex < NumMeshDescriptions; ++LODIndex)
			{
				StaticMesh->CommitMeshDescription(LODIndex, Params);
			}

			return true;
		}
	);
#endif	  // WITH_EDITOR

	// PreBuild static mesh (Main thread)
	Do(
		ESchemaTranslationLaunchPolicy::Sync,
		[this]()
		{
			RecreateRenderStateContextPtr = MakeShared<FStaticMeshComponentRecreateRenderStateContext>(StaticMesh, true, true);

			UsdGeomMeshTranslatorImpl::PreBuildStaticMesh(*StaticMesh);

			return true;
		}
	);

	// Build static mesh (Async)
	Then(
		ESchemaTranslationLaunchPolicy::Async,
		[this]() mutable
		{
			FStaticFeatureLevel FeatureLevel = GMaxRHIFeatureLevel;

			UWorld* World = Context->Level ? Context->Level->GetWorld() : nullptr;
			if (!World)
			{
				World = IUsdClassesModule::GetCurrentWorld();
			}
			if (World)
			{
				FeatureLevel = World->GetFeatureLevel();
			}

			bool bSuccess = UsdGeomMeshTranslatorImpl::BuildStaticMesh(*StaticMesh, FeatureLevel, LODIndexToMeshDescription);
			if (bSuccess)
			{
				FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();

				// We want to discard our StaticMesh if it generated no valid RenderData (e.g. was compeletely degenerate).
				// They are not really going to be useful, and this is also handy as in some cases the StaticMeshComponents that use them
				// may get confused as to why they have a StaticMesh with no valid RenderData at points where they expected to have them
				// (e.g. see UE-201946)
				bSuccess = RenderData && !RenderData->Bounds.ContainsNaN() && RenderData->GetCurrentFirstLODIdx(0) != INDEX_NONE;
			}

			// Build failed, abandon mesh
			if (!bSuccess)
			{
				USD_LOG_WARNING(
					TEXT(
						"Discarding StaticMesh generated for prim '%s' as it didn't produce any valid RenderData (likely all triangles were degenerate)"
					),
					*PrimPath.GetString()
				);
				UsdUnreal::TranslatorUtils::AbandonFailedAsset(StaticMesh, Context->UsdAssetCache.Get(), Context->PrimLinkCache);
				return false;
			}

			UE::UsdCollision::Private::SetupSimpleCollision(GetPrim(), *StaticMesh, Context->FallbackCollisionType);

			return true;
		}
	);

	// PostBuild static mesh (Main thread)
	Then(
		ESchemaTranslationLaunchPolicy::Sync,
		[this]()
		{
			UsdGeomMeshTranslatorImpl::PostBuildStaticMesh(*StaticMesh, LODIndexToMeshDescription);

			RecreateRenderStateContextPtr.Reset();

			return true;
		}
	);
}

FBuildStaticMeshTaskChain::FBuildStaticMeshTaskChain(
	const TSharedRef<FUsdSchemaTranslationContext>& InContext,
	const UE::FSdfPath& InPrimPath,
	const TOptional<UE::FSdfPath>& InAlternativePrimToLinkAssetsTo
)

	: FBaseBuildStaticMeshTaskChain(InContext, InPrimPath)
	, AlternativePrimToLinkAssetsTo(InAlternativePrimToLinkAssetsTo)
{
}

void FBuildStaticMeshTaskChain::SetupTasks()
{
	// Create static mesh (Main thread)
	Do(ESchemaTranslationLaunchPolicy::Sync,
	   [this]()
	   {
	// Force load MeshBuilderModule so that it's ready for the async tasks
#if WITH_EDITOR
		   FModuleManager::LoadModuleChecked<IMeshBuilderModule>(TEXT("MeshBuilder"));
#endif	  // WITH_EDITOR

		   const FString PrimPathString = PrimPath.GetString();

		   const bool bParsedLODs = Context->bAllowInterpretingLODs && UsdUtils::IsGeomMeshALOD(GetPrim());

		   // It's useful to have the LOD Mesh prims be named "LOD0", "LOD1", etc. within the LOD variants so that we
		   // can easily tell which Mesh is actually meant to be the LOD mesh (in case there are more Meshes in each
		   // variant or other Meshes outside of the variant), but it's not ideal to have all the generated assets end
		   // up imported as "SM_LOD0_22", "SM_LOD0_23", etc. So here we fetch the parent prim name in case we're a LOD
		   FString MeshName;
		   if (bParsedLODs)
		   {
			   MeshName = PrimPath.GetParentPath().GetString();
		   }
		   else
		   {
			   MeshName = PrimPathString;
		   }

		   bool bIsNew = true;
		   const bool bShouldEnableNanite = UsdGeomMeshTranslatorImpl::ShouldEnableNanite(
			   LODIndexToMeshDescription,
			   LODIndexToMaterialInfo,
			   *Context,
			   GetPrim()
		   );
		   StaticMesh = UsdGeomMeshTranslatorImpl::CreateStaticMesh(
			   GetPrim(),
			   LODIndexToMeshDescription,
			   *Context,
			   MeshName,
			   bShouldEnableNanite,
			   bIsNew
		   );

		   if (StaticMesh)
		   {
			   if (Context->PrimLinkCache)
			   {
				   const UE::FSdfPath& TargetPath = AlternativePrimToLinkAssetsTo.IsSet() ? AlternativePrimToLinkAssetsTo.GetValue() : PrimPath;
				   Context->PrimLinkCache->LinkAssetToPrim(TargetPath, StaticMesh);
			   }

			   if (UUsdMeshAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdMeshAssetUserData>(StaticMesh))
			   {
				   if (LODIndexToMaterialInfo.Num() > 0)
				   {
					   // We use the same primvar mapping for all LODs
					   UserData->PrimvarToUVIndex = LODIndexToMaterialInfo[0].PrimvarToUVIndex;
				   }
				   UserData->PrimPaths.AddUnique(PrimPathString);

				   if (Context->MetadataOptions.bCollectMetadata)
				   {
					   // If we already have collected metadata just stash it into our UserData directly (the GeomMeshTranslator task
					   // chain itself does this as it parses lods, but the regular FBuildStaticMeshTaskChain won't do this on
					   // its own, and is reused for other task chains)
					   if (bCollectedMetadata)
					   {
						   UserData->StageIdentifierToMetadata.Add(GetPrim().GetStage().GetRootLayer().GetIdentifier(), LODMetadata);
					   }
					   else
					   {
						   UsdToUnreal::ConvertMetadata(
							   bParsedLODs ? GetPrim().GetParent() : GetPrim(),
							   UserData,
							   Context->MetadataOptions.BlockedPrefixFilters,
							   Context->MetadataOptions.bInvertFilters,
							   Context->MetadataOptions.bCollectFromEntireSubtrees
						   );
					   }
				   }
				   else
				   {
					   // Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the AssetUserData
					   UserData->StageIdentifierToMetadata.Remove(GetPrim().GetStage().GetRootLayer().GetIdentifier());
				   }

				   MeshTranslationImpl::RecordSourcePrimsForMaterialSlots(LODIndexToMaterialInfo, UserData);
			   }

			   // Only the original creator of the prim at creation time gets to set the material assignments
			   // directly on the mesh, all others prims ensure their materials via material overrides on the
			   // components
			   if (bIsNew && Context->UsdAssetCache && Context->PrimLinkCache)
			   {
#if WITH_EDITOR
				   StaticMesh->GetNaniteSettings().bEnabled = bShouldEnableNanite;
				   StaticMesh->GetNaniteSettings().NormalPrecision = GNaniteSettingsNormalPrecision;
				   StaticMesh->GetNaniteSettings().TangentPrecision = GNaniteSettingsTangentPrecision;
#endif	  // WITH_EDITOR

				   UsdGeomMeshTranslatorImpl::ProcessStaticMeshMaterials(
					   GetPrim(),
					   LODIndexToMaterialInfo,
					   *StaticMesh,
					   *Context->UsdAssetCache,
					   *Context->PrimLinkCache,
					   Context->Time,
					   Context->ObjectFlags,
					   Context->bShareAssetsForIdenticalPrims
				   );

#if WITH_EDITOR
				   const bool bRebuildAll = true;
				   StaticMesh->UpdateUVChannelData(bRebuildAll);
#else
					// UpdateUVChannelData doesn't do anything without the editor
					for (FStaticMaterial& Material : StaticMesh->GetStaticMaterials())
					{
						Material.UVChannelData.bInitialized = true;
					}
#endif	  // WITH_EDITOR
			   }
		   }

		   // Only need to continue building the mesh if we just created it
		   return bIsNew;
	   });

	FBaseBuildStaticMeshTaskChain::SetupTasks();
}

FGeomMeshCreateAssetsTaskChain::FGeomMeshCreateAssetsTaskChain(
	const TSharedRef<FUsdSchemaTranslationContext>& InContext,
	const UE::FSdfPath& InPrimPath,
	const TOptional<UE::FSdfPath>& AlternativePrimToLinkAssetsTo,
	const FTransform& InAdditionalTransform
)
	: FBuildStaticMeshTaskChain(InContext, InPrimPath, AlternativePrimToLinkAssetsTo)
	, AdditionalTransform(InAdditionalTransform)
{
	SetupTasks();
}

void FGeomMeshCreateAssetsTaskChain::SetupTasks()
{
	FScopedUnrealAllocs UnrealAllocs;

	// To parse all LODs we need to actively switch variant sets to other variants (triggering prim loading/unloading and notices),
	// which could cause race conditions if other async translation tasks are trying to access those prims
	bool bParseLODs = Context->bAllowInterpretingLODs && UsdUtils::IsGeomMeshALOD(GetPrim());
	ESchemaTranslationLaunchPolicy LaunchPolicy = ESchemaTranslationLaunchPolicy::Async;
	if (bParseLODs)
	{
		LaunchPolicy = ESchemaTranslationLaunchPolicy::ExclusiveSync;
	}

	// Create mesh descriptions (Async or ExclusiveSync)
	Do(LaunchPolicy,
	   [this, bParseLODs]() -> bool
	   {
		   pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
		   if (!Context->RenderContext.IsNone())
		   {
			   RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
		   }

		   pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		   if (!Context->MaterialPurpose.IsNone())
		   {
			   MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
		   }

		   UsdToUnreal::FUsdMeshConversionOptions Options;
		   Options.TimeCode = Context->Time;
		   Options.PurposesToLoad = Context->PurposesToLoad;
		   Options.RenderContext = RenderContextToken;
		   Options.MaterialPurpose = MaterialPurposeToken;
		   Options.bMergeIdenticalMaterialSlots = Context->bMergeIdenticalMaterialSlots;
		   Options.AdditionalTransform = AdditionalTransform;
		   Options.SubdivisionLevel = Context->SubdivisionLevel;

		   UsdGeomMeshTranslatorImpl::LoadMeshDescriptions(
			   GetPrim(),
			   LODIndexToMeshDescription,
			   LODIndexToMaterialInfo,
			   LODMetadata,
			   Options,
			   Context->MetadataOptions,
			   Context->bAllowInterpretingLODs && bParseLODs
		   );

		   // If we're parsing LODs, LoadMeshDescriptions will have already collected metadata from the Mesh
		   // prims directly, but we still have to collect stuff from the actual LOD root prim, which is kind
		   // of absorbed into the asset
		   if (bParseLODs && Context->MetadataOptions.bCollectMetadata)
		   {
			   const bool bCollectMetadataFromSubtree = false;
			   UsdToUnreal::ConvertMetadata(
				   GetPrim().GetParent(),
				   LODMetadata,
				   Context->MetadataOptions.BlockedPrefixFilters,
				   Context->MetadataOptions.bInvertFilters,
				   bCollectMetadataFromSubtree
			   );
		   }
		   bCollectedMetadata = true;

		   // Strip empty MeshDescriptions: If we have some valid LODs and some empty our StaticMesh RenderData
		   // will end up with NaN bounds and will have to be discarded anyway
		   bool bHasValidLOD = false;
		   for (int32 LODIndex = LODIndexToMeshDescription.Num() - 1; LODIndex >= 0; --LODIndex)
		   {
			   const FMeshDescription& MeshDescription = LODIndexToMeshDescription[LODIndex];
			   if (MeshDescription.IsEmpty())
			   {
				   LODIndexToMeshDescription.RemoveAt(LODIndex, EAllowShrinking::No);

				   if (LODIndexToMaterialInfo.IsValidIndex(LODIndex))
				   {
					   LODIndexToMaterialInfo.RemoveAt(LODIndex, EAllowShrinking::No);
				   }

				   USD_LOG_USERWARNING(FText::Format(
					   LOCTEXT("EmpytMeshData", "Ignoring mesh data collected for LOD{0} of prim '{1}' as it is empty. Is the prim invisible?"),
					   LODIndex,
					   FText::FromString(bParseLODs ? *GetPrim().GetParent().GetPrimPath().GetString() : *GetPrim().GetPrimPath().GetString())
				   ));
			   }
			   else
			   {
				   bHasValidLOD = true;
			   }
		   }

		   // If we have at least one valid LOD, we should keep going
		   return bHasValidLOD;
	   });

	FBuildStaticMeshTaskChain::SetupTasks();
}

void FUsdGeomMeshTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomMeshTranslator::CreateAssets);

	if (!IsMeshPrim())
	{
		return Super::CreateAssets();
	}

	if (ShouldSkipInstance())
	{
		return;
	}

	UE::FUsdPrim Prim = GetPrim();

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(Prim);
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	if (UsdUtils::IsCollisionMesh(Prim))
	{
		return;
	}

	if (ShouldSkipSkinnablePrim())
	{
		return;
	}

	UE::FSdfPath PrototypePrimPath = GetPrototypePrimPath();
	TSharedRef<FGeomMeshCreateAssetsTaskChain> AssetsTaskChain = MakeShared<FGeomMeshCreateAssetsTaskChain>(Context, PrimPath, PrototypePrimPath);

	Context->TranslatorTasks.Add(MoveTemp(AssetsTaskChain));
}

USceneComponent* FUsdGeomMeshTranslator::CreateComponents()
{
	if (UsdUtils::IsCollisionMesh(GetPrim()))
	{
		return nullptr;
	}

	// It's not great that we have to check for an alternate draw mode here and Super::CreateComponents will
	// also do it, however we must check for ShouldSkipSkinnablePrim before we call Super::CreateComponents,
	// and we need the alt draw mode check to take priority over ShouldSkipSkinnablePrim...
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		const bool bCheckForComponent = true;
		if (ShouldSkipSkinnablePrim(bCheckForComponent))
		{
			return nullptr;
		}
	}

	return Super::CreateComponents();
}

void FUsdGeomMeshTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	// If we're a bounds component we don't need to handle any mesh stuff, as drawMode takes priority
	// over even checking for whether we're skinned or not
	if (Cast<UUsdDrawModeComponent>(SceneComponent))
	{
		Super::UpdateComponents(SceneComponent);
		return;
	}

	const bool bCheckForComponent = true;
	if (ShouldSkipSkinnablePrim(bCheckForComponent))
	{
		return;
	}

	if (IsMeshPrim())
	{
		if (SceneComponent)
		{
			SceneComponent->Modify();
		}

		if (UsdUtils::IsAnimatedMesh(GetPrim()))
		{
			// The assets might have changed since our attributes are animated
			// Note that we must wait for these to complete as they make take a while and we want to
			// reassign our new static meshes when we get to FUsdGeomXformableTranslator::UpdateComponents
			CreateAssets();
			Context->CompleteTasks();
		}
	}

	Super::UpdateComponents(SceneComponent);
}

bool FUsdGeomMeshTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	if (ShouldSkipSkinnablePrim())
	{
		return false;
	}

	if (IsMeshPrim())
	{
		// We can't claim we collapse anything here since we'll just parse the mesh for this prim and that's it,
		// otherwise the translation context wouldn't spawn translators for our child prims.
		// Another approach would be to actually recursively collapse our child mesh prims, but that leads to a few
		// issues. For example this translator could end up globbing a child Mesh prim, while the translation context
		// could simultaneously spawn other translators that could also end up accounting for that same mesh.
		// Generally Gprims shouldn't be nested into each other anyway (see https://graphics.pixar.com/usd/release/glossary.html#usdglossary-gprim)
		// so it's likely best to just not collapse anything here.
		return false;
	}

	return Super::CollapsesChildren(CollapsingType);
}

bool FUsdGeomMeshTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	if (!IsMeshPrim())
	{
		return Super::CanBeCollapsed(CollapsingType);
	}

	// If we're a skinned mesh prim we're inside of a SkelRoot, that won't be collapsed anyway
	if (ShouldSkipSkinnablePrim())
	{
		return false;
	}

	UE::FUsdPrim Prim = GetPrim();

	// Don't collapse if our final UStaticMesh would have multiple LODs
	if (Context->bAllowInterpretingLODs && CollapsingType == ECollapsingType::Assets && UsdUtils::IsGeomMeshALOD(Prim))
	{
		return false;
	}

	// Prevent collapse of mesh with collision enabled
	if (pxr::UsdPhysicsCollisionAPI CollisionAPI{GetPrim()})
	{
		bool bIsCollisionEnabled = false;
		if (pxr::UsdAttribute CollisionAttr = CollisionAPI.GetCollisionEnabledAttr())
		{
			CollisionAttr.Get(&bIsCollisionEnabled);
		}

		if (bIsCollisionEnabled)
		{
			return false;
		}
	}

	// Prevent collapse of custom collision mesh
	if (UsdUtils::IsCollisionMesh(Prim))
	{
		return false;
	}

	return Super::CanBeCollapsed(CollapsingType);
}

TSet<UE::FSdfPath> FUsdGeomMeshTranslator::CollectAuxiliaryPrims() const
{
	if (!IsMeshPrim())
	{
		return Super::CollectAuxiliaryPrims();
	}

	if (ShouldSkipSkinnablePrim())
	{
		return {};
	}

	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	TSet<UE::FSdfPath> Result;
	if (UsdUtils::IsCollisionMesh(GetPrim()))
	{
		return Result;
	}

	{
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim Prim = GetPrim();

		// The UsdGeomSubset prims are used to express multiple material assignments per mesh. A change in them could
		// mean a change in triangle to material slot mapping
		TArray<TUsdStore<pxr::UsdPrim>> ChildPrims = UsdUtils::GetAllPrimsOfType(Prim, pxr::TfType::Find<pxr::UsdGeomSubset>());

		Result.Reserve(ChildPrims.Num());
		for (const TUsdStore<pxr::UsdPrim>& ChildPrim : ChildPrims)
		{
			Result.Add(UE::FSdfPath{ChildPrim.Get().GetPrimPath()});
		}

		// For mesh prim, collect the sibling collision meshes as auxiliary prims
		for (const pxr::UsdPrim& ChildPrim : GetPrim().GetParent().GetChildren())
		{
			if (UsdUtils::IsCollisionMesh(ChildPrim))
			{
				Result.Add(UE::FSdfPath{ChildPrim.GetPrimPath()});
			}
		}
	}

	return Result;
}

bool FUsdGeomMeshTranslator::IsMeshPrim() const
{
	UE::FUsdPrim Prim = GetPrim();
	if (Prim && (Prim.IsA(TEXT("Mesh")) || (Context->bAllowInterpretingLODs && UsdUtils::DoesPrimContainMeshLODs(Prim))))
	{
		return true;
	}

	return false;
}

bool FUsdGeomMeshTranslator::ShouldSkipSkinnablePrim(bool bCheckForComponent) const
{
	// At runtime we don't handle skinned meshes as SkeletalMeshes anyway, so early out and pretend it is not skinned
	if (!GIsEditor)
	{
		return false;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim Prim = GetPrim();
	if (!Prim)
	{
		return false;
	}

	// Search for the API schema on the mesh prims
	bool bHasAPISchema = false;
	if (pxr::UsdGeomMesh GeomMesh{Prim})
	{
		if (Prim.HasAPI<pxr::UsdSkelBindingAPI>())
		{
			bHasAPISchema = true;
		}
	}
	// The groom translator means we could potentially have been called on an Xform LOD prim with skinned meshes in the LODs.
	// We could even be a LOD prim setup where the SkelRoot is the "container" and the different LOD meshes are inside of it. In that
	// particular case though then we still want to spawn/keep our SceneComponent for the SkelRoot (we always keep the SkelRoots)
	else if (Context->bAllowInterpretingLODs && UsdUtils::DoesPrimContainMeshLODs(Prim))
	{
		pxr::UsdPrimSiblingRange PrimRange = Prim.GetChildren();
		for (pxr::UsdPrimSiblingRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
		{
			const pxr::UsdPrim& Child = *PrimRangeIt;
			if (pxr::UsdGeomMesh ChildMesh{Child})
			{
				if (Prim.HasAPI<pxr::UsdSkelBindingAPI>())
				{
					bHasAPISchema = true;
				}
			}
		}
	}
	if (!bHasAPISchema)
	{
		return false;
	}

	// If we're here we know we have a valid API schema. Check for an ancestor SkelRoot then,
	// which is required to enable skinning
	if (pxr::UsdPrim ParentSkelRoot = UsdUtils::GetClosestParentSkelRoot(Prim))
	{
		// For components there is still one edge case to worry about: If the user has placed a non-skinnable
		// prim as a child of our skinnable prim (our 'Prim' here). We'll translate this skinnable prim data when
		// generating the SkeletalMesh for it from somewhere else, and we'll still never translate an asset for 'Prim'
		// directly again here (e.g. as a StaticMesh), but when it comes to the *component* then we may still need to spawn it.
		// This because the component for 'Prim' may have some transform, visibility, animation (or something) that affects
		// its children
		if (bCheckForComponent)
		{
			bool bHasRelevantChildPrim = false;
			for (pxr::UsdPrim Child : Prim.GetChildren())
			{
				if (!Child.IsA<pxr::UsdGeomSubset>())
				{
					bHasRelevantChildPrim = true;
					break;
				}
			}

			return !bHasRelevantChildPrim;
		}
		else
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
