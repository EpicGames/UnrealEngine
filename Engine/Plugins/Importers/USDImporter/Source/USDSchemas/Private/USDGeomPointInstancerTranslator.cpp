// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGeomPointInstancerTranslator.h"

#include "MeshTranslationImpl.h"
#include "Objects/USDPrimLinkCache.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDGeomMeshTranslator.h"
#include "USDMemory.h"
#include "USDPrimConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdTyped.h"

#if WITH_EDITOR
#include "Editor.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK

#include "Async/Async.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usd/stage.h"
#include "pxr/usd/usdGeom/camera.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/pointInstancer.h"
#include "pxr/usd/usdGeom/xform.h"
#include "pxr/usd/usdGeom/xformable.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDGeomPointInstancer"

static bool GCollapseTopLevelPointInstancers = false;
static FAutoConsoleVariableRef CVarCollapseTopLevelPointInstancers(
	TEXT("USD.CollapseTopLevelPointInstancers"),
	GCollapseTopLevelPointInstancers,
	TEXT("If this is true will cause any point instancer to be collapsed to a single static mesh. Point instancers that are used as prototypes for "
		 "other point instancers will always be collapsed.")
);

namespace UsdGeomPointInstancerTranslatorImpl
{
	void ApplyPointInstanceTransforms(UInstancedStaticMeshComponent* Component, TArray<FTransform>& InstanceTransforms)
	{
		if (Component)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ApplyPointInstanceTransforms);

			Component->ClearInstances();
			Component->AddInstances(InstanceTransforms, false);
		}
	}

	void SetStaticMesh(UStaticMesh* StaticMesh, UStaticMeshComponent& MeshComponent)
	{
		if (StaticMesh == MeshComponent.GetStaticMesh())
		{
			return;
		}

		MeshComponent.Modify();

		if (MeshComponent.IsRegistered())
		{
			MeshComponent.UnregisterComponent();
		}

		if (StaticMesh)
		{
			StaticMesh->CreateBodySetup();					// BodySetup is required for ISM component
			StaticMesh->MarkAsNotHavingNavigationData();	// Needed or else it will warn if we try cooking with body setup
		}

		MeshComponent.SetStaticMesh(StaticMesh);

		MeshComponent.RegisterComponent();
	}
}	 // namespace UsdGeomPointInstancerTranslatorImpl

FUsdGeomPointInstancerCreateAssetsTaskChain::FUsdGeomPointInstancerCreateAssetsTaskChain(
	const TSharedRef<FUsdSchemaTranslationContext>& InContext,
	const UE::FSdfPath& InPrimPath,
	bool bInIgnoreTopLevelTransform,
	bool bInIgnoreTopLevelVisibility,
	const TOptional<UE::FSdfPath>& InAlternativePrimToLinkAssetsTo
)
	: FBuildStaticMeshTaskChain(InContext, InPrimPath, InAlternativePrimToLinkAssetsTo)
	, bIgnoreTopLevelTransform(bInIgnoreTopLevelTransform)
	, bIgnoreTopLevelVisibility(bInIgnoreTopLevelVisibility)
{
	SetupTasks();
}

void FUsdGeomPointInstancerCreateAssetsTaskChain::SetupTasks()
{
	FScopedUnrealAllocs UnrealAllocs;

	// Create mesh description (Async)
	Do(ESchemaTranslationLaunchPolicy::Async,
	   [this]() -> bool
	   {
		   LODIndexToMeshDescription.Reset(1);
		   LODIndexToMaterialInfo.Reset(1);

		   FMeshDescription& AddedMeshDescription = LODIndexToMeshDescription.Emplace_GetRef();
		   UsdUtils::FUsdPrimMaterialAssignmentInfo& AssignmentInfo = LODIndexToMaterialInfo.Emplace_GetRef();

		   pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
		   if (!Context->RenderContext.IsNone())
		   {
			   RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
		   }

		   pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		   if (!Context->MaterialPurpose.IsNone())
		   {
			   MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
		   }

		   UsdToUnreal::FUsdMeshConversionOptions Options;
		   Options.TimeCode = Context->Time;
		   Options.PurposesToLoad = Context->PurposesToLoad;
		   Options.RenderContext = RenderContextToken;
		   Options.MaterialPurpose = MaterialPurposeToken;
		   Options.bMergeIdenticalMaterialSlots = Context->bMergeIdenticalMaterialSlots;
		   Options.SubdivisionLevel = Context->SubdivisionLevel;

		   UsdToUnreal::ConvertGeomMeshSubtree(
			   GetPrim(),
			   AddedMeshDescription,
			   AssignmentInfo,
			   Options,
			   bIgnoreTopLevelTransform,
			   bIgnoreTopLevelVisibility
		   );

		   return !AddedMeshDescription.IsEmpty();
	   });

	FBuildStaticMeshTaskChain::SetupTasks();
}

void FUsdGeomPointInstancerTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomPointInstancerTranslator::CreateAssets);

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim Prim = GetPrim();
	pxr::UsdGeomPointInstancer PointInstancer(Prim);
	if (!PointInstancer)
	{
		return;
	}

	// If another FUsdGeomXformableTranslator is collapsing the point instancer prim, it will do so by calling
	// UsdToUnreal::ConvertGeomMeshHierarchy which will consume the prim directly.
	// This case right here is if we're collapsing *ourselves*, where we'll essentially pretend we're a single static mesh.
	if (GCollapseTopLevelPointInstancers)
	{
		// Don't bake our actual point instancer's transform or visibility into the mesh as its nice to have these on the static mesh component
		// instead
		const bool bIgnoreTopLevelTransformAndVisibility = true;
		Context->TranslatorTasks.Add(MakeShared<FUsdGeomPointInstancerCreateAssetsTaskChain>(
			Context,
			PrimPath,
			bIgnoreTopLevelTransformAndVisibility,
			bIgnoreTopLevelTransformAndVisibility
		));
	}
	// Otherwise we're just going to spawn ISM components instead
	else
	{
		const pxr::UsdRelationship& Prototypes = PointInstancer.GetPrototypesRel();

		pxr::SdfPathVector PrototypePaths;
		if (!Prototypes.GetTargets(&PrototypePaths))
		{
			return;
		}

		for (const pxr::SdfPath& PrototypePath : PrototypePaths)
		{
			// Note how we will spawn a task chain for the prototype *regardless of where it is*. This prototype
			// could even be external to the point instancer itself, and so will already have been handled by
			// another translator. Unfortunately we need to do this because we need to generate a task chain for
			// it in case it is another point instancer itself

			pxr::UsdPrim PrototypeUsdPrim = Prim.GetStage()->GetPrimAtPath(PrototypePath);
			UE::FSdfPath UEPrototypePath{PrototypePath};

			if (!PrototypeUsdPrim)
			{
				USD_LOG_WARNING(
					TEXT("Failed to find prototype '%s' for PointInstancer '%s' when collapsing assets"),
					*UEPrototypePath.GetString(),
					*PrimPath.GetString()
				);
				continue;
			}

			if (Context->bAllowInterpretingLODs && UsdUtils::DoesPrimContainMeshLODs(PrototypeUsdPrim))
			{
				// We have to provide one of the LOD meshes to the task chain, so find the path to one
				UE::FSdfPath ChildMeshPath;
				pxr::UsdPrimSiblingRange PrimRange = PrototypeUsdPrim.GetChildren();
				for (pxr::UsdPrimSiblingRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
				{
					const pxr::UsdPrim& Child = *PrimRangeIt;
					if (pxr::UsdGeomMesh ChildMesh{Child})
					{
						ChildMeshPath = UE::FSdfPath{Child.GetPrimPath()};
						break;
					}
				}

				// This is in charge of baking in 'Prim's transform into the generated static mesh for the prototype, as it
				// otherwise wouldn't end up anywhere else. Note that in the default/export case 'Prim' (the prim that actually contains the LOD
				// variant set) is schema-less (and so not an Xform), but this is just in case the user manually made it an Xform instead
				FTransform AdditionalUESpaceTransform = FTransform::Identity;
				if (pxr::UsdGeomXform ParentXform{PrototypeUsdPrim})
				{
					// Skip this LOD mesh if its invisible
					pxr::TfToken Visibility;
					pxr::UsdAttribute VisibilityAttr = ParentXform.GetVisibilityAttr();
					if (VisibilityAttr && VisibilityAttr.Get(&Visibility, Context->Time) && Visibility == pxr::UsdGeomTokens->invisible)
					{
						continue;
					}

					// TODO: Handle the resetXformStack op for LOD parents
					bool bOutResetTransformStack = false;
					UsdToUnreal::ConvertXformable(Prim.GetStage(), ParentXform, AdditionalUESpaceTransform, Context->Time, &bOutResetTransformStack);
				}

				Context->TranslatorTasks.Add(MakeShared<FGeomMeshCreateAssetsTaskChain>(Context, ChildMeshPath, PrimPath, AdditionalUESpaceTransform)
				);
			}
			else
			{
				// Bake the prototype transform into the mesh as it must always be applied before the instancing transform, and if we
				// put it on the ISM component instead it would have been applied *after* the instancing transform
				const bool bIgnoreTopLevelTransform = false;
				// Don't bake the visibility into the mesh as we can control it by toggling HiddenInGame on the ISM component directly
				const bool bIgnoreTopLevelVisibility = true;
				Context->TranslatorTasks.Add(MakeShared<FUsdGeomPointInstancerCreateAssetsTaskChain>(
					Context,
					UEPrototypePath,
					bIgnoreTopLevelTransform,
					bIgnoreTopLevelVisibility,
					PrimPath
				));
			}
		}
	}
}

USceneComponent* FUsdGeomPointInstancerTranslator::CreateComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomPointInstancerTranslator::CreateComponents);

	// If we're collapsing ourselves, we're really just a collapsed Xform prim, so let that translator handle it
	if (GCollapseTopLevelPointInstancers)
	{
		return FUsdGeomXformableTranslator::CreateComponents();
	}

	// Otherwise, the plan here is to create an USceneComponent that corresponds to the PointInstancer prim itself, and then spawn a child
	// ISM component for each prototype.
	// We always request a scene component here explicitly or else we'll be upgraded to a static mesh component by the mechanism that
	// handles collapsed meshes/static mesh components for the GeomXFormable translator.
	bool bCreateChildISMs = false;
	USceneComponent* MainSceneComponent = nullptr;
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		bCreateChildISMs = true;
		MainSceneComponent = CreateComponentsEx({USceneComponent::StaticClass()}, {});
		MainSceneComponent->ComponentTags.AddUnique(UnrealIdentifiers::PointInstancerTag);
	}
	else
	{
		MainSceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	// Actually create the child ISM components for each point instancer prototype
	if (bCreateChildISMs)
	{
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim Prim = GetPrim();
		pxr::UsdGeomPointInstancer PointInstancer(Prim);
		if (!PointInstancer)
		{
			return MainSceneComponent;
		}

		pxr::SdfPathVector PrototypePaths;
		if (!PointInstancer.GetPrototypesRel().GetTargets(&PrototypePaths))
		{
			return MainSceneComponent;
		}

		if (!Context->UsdInfoCache)
		{
			return MainSceneComponent;
		}

		// Lets pretend ParentComponent is pointing to the parent USceneComponent while we create the child ISMs, so they get
		// automatically attached to it as children
		TGuardValue<USceneComponent*> ParentComponentGuard{Context->ParentComponent, MainSceneComponent};

		TArray<TFuture<TTuple<UInstancedStaticMeshComponent*, TArray<FTransform>>>> Tasks;
		FScopedSlowTask PrototypePathsSlowTask(
			(float)PrototypePaths.size(),
			LOCTEXT("GeomPointCreateComponents", "Creating HierarchicalInstancedStaticMeshComponents for point instancers")
		);
		for (int32 PrototypeIndex = 0; PrototypeIndex < PrototypePaths.size(); ++PrototypeIndex)
		{
			PrototypePathsSlowTask.EnterProgressFrame();

			const pxr::SdfPath& PrototypePath = PrototypePaths[PrototypeIndex];
			FString PrototypePathStr = UsdToUnreal::ConvertPath(PrototypePath);

			pxr::UsdPrim PrototypeUsdPrim = Prim.GetStage()->GetPrimAtPath(PrototypePath);
			if (!PrototypeUsdPrim)
			{
				USD_LOG_WARNING(
					TEXT("Failed to find prototype '%s' for PointInstancer '%s' when creating components"),
					*PrototypePathStr,
					*PrimPath.GetString()
				);
				continue;
			}

			// If our prototype was a LOD mesh we will have used the path of one of the actual LOD meshes to start the FGeomMeshCreateAssetsTaskChain,
			// so we have to look for our resulting mesh with the same path
			if (Context->bAllowInterpretingLODs && UsdUtils::DoesPrimContainMeshLODs(PrototypeUsdPrim))
			{
				pxr::UsdPrimSiblingRange PrimRange = PrototypeUsdPrim.GetChildren();
				for (pxr::UsdPrimSiblingRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
				{
					const pxr::UsdPrim& Child = *PrimRangeIt;
					if (pxr::UsdGeomMesh ChildMesh{Child})
					{
						PrototypeUsdPrim = Child;
						PrototypePathStr = UsdToUnreal::ConvertPath(Child.GetPrimPath());
						break;
					}
				}
			}

			UClass* ComponentClass = Context->bAllowInterpretingLODs ? UHierarchicalInstancedStaticMeshComponent::StaticClass()
																	 : UInstancedStaticMeshComponent::StaticClass();

			FUsdGeomXformableTranslator XformableTranslator{ComponentClass, Context, UE::FUsdTyped(PrototypeUsdPrim)};

			const bool bNeedsActor = false;
			UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(
				XformableTranslator.CreateComponentsEx({ComponentClass}, bNeedsActor)
			);

			if (!ISMComponent)
			{
				USD_LOG_WARNING(
					TEXT("Failed to generate ISM/HISM component for prototype '%s' for PointInstancer '%s'"),
					*PrototypePathStr,
					*PrimPath.GetString()
				);
				continue;
			}
		}
	}

	UpdateComponents(MainSceneComponent);

	return MainSceneComponent;
}

void FUsdGeomPointInstancerTranslator::UpdateComponents(USceneComponent* PointInstancerRootComponent)
{
	if (!PointInstancerRootComponent)
	{
		return;
	}

	// Do this first as we can use it's resulting HiddenInGame value to propagate visibility to the ISM components
	Super::UpdateComponents(PointInstancerRootComponent);

	// We always spawn exactly an USceneComponent for the "parent" component of the point instancer, so early out if
	// we dont' have one. This can happen now if we have an alternative draw mode for this point instancer, at which
	// point this could be an UUsdDrawModeComponent
	if (PointInstancerRootComponent->GetClass() == USceneComponent::StaticClass())
	{
		pxr::UsdPrim Prim = GetPrim();
		pxr::UsdGeomPointInstancer PointInstancer(Prim);
		if (!PointInstancer)
		{
			return;
		}

		TUsdStore<pxr::SdfPathVector> PrototypePaths;
		if (!PointInstancer.GetPrototypesRel().GetTargets(&PrototypePaths.Get()))
		{
			return;
		}

		if (!Context->UsdAssetCache.IsValid() || !Context->PrimLinkCache || !Context->UsdInfoCache)
		{
			return;
		}

		// Lets pretend ParentComponent is pointing to the parent USceneComponent while we create the child ISMs, so they get
		// automatically attached to it as children
		TGuardValue<USceneComponent*> ParentComponentGuard{Context->ParentComponent, PointInstancerRootComponent};

		const TArray<TObjectPtr<USceneComponent>>& AttachedChildren = PointInstancerRootComponent->GetAttachChildren();
		TArray<UInstancedStaticMeshComponent*> AttachedISMs;
		AttachedISMs.Reserve(AttachedChildren.Num());
		for (const TObjectPtr<USceneComponent>& AttachedChild : AttachedChildren)
		{
			if (UInstancedStaticMeshComponent* ISM = Cast<UInstancedStaticMeshComponent>(AttachedChild))
			{
				AttachedISMs.Add(ISM);
			}
		}

		// We "link" the prototype meshes to the point instancer, but we don't know which mesh corresponds to each
		// prototype, as we translate these with task pools and some of those prototypes may have generated nullptr.
		// We always put the prototype path on the asset import data though, so here we use that to figure out where
		// each mesh should go
		TArray<UStaticMesh*> PrototypeMeshArr = Context->PrimLinkCache->GetAssetsForPrim<UStaticMesh>(PrimPath);
		std::unordered_map<pxr::SdfPath, UStaticMesh*, pxr::SdfPath::Hash> PrototypeMeshes;
		PrototypeMeshes.reserve(PrototypeMeshArr.Num());
		for (UStaticMesh* PrototypeMesh : PrototypeMeshArr)
		{
			if (UUsdAssetUserData* UserData = PrototypeMesh->GetAssetUserData<UUsdAssetUserData>())
			{
				for (const FString& SourcePrimPath : UserData->PrimPaths)
				{
					pxr::SdfPath PrototypePath = UnrealToUsd::ConvertPath(*SourcePrimPath).Get();
					PrototypeMeshes[PrototypePath] = PrototypeMesh;
				}
			}
		}

		TArray<TFuture<TTuple<UInstancedStaticMeshComponent*, TArray<FTransform>>>> Tasks;
		size_t NumPrototypePaths = PrototypePaths.Get().size();
		FScopedSlowTask PrototypePathsSlowTask(
			(float)NumPrototypePaths,
			LOCTEXT("GeomPointUpdateComponents", "Updating HierarchicalInstancedStaticMeshComponents for point instancers")
		);
		for (uint32 PrototypeIndex = 0; PrototypeIndex < NumPrototypePaths; ++PrototypeIndex)
		{
			PrototypePathsSlowTask.EnterProgressFrame();

			pxr::SdfPath PrototypePath = PrototypePaths.Get()[PrototypeIndex];

			TUsdStore<pxr::UsdPrim> PrototypeUsdPrim = Prim.GetStage()->GetPrimAtPath(PrototypePath);
			if (!PrototypeUsdPrim.Get())
			{
				USD_LOG_WARNING(
					TEXT("Failed to find prototype '%s' for PointInstancer '%s' when updating components"),
					*UsdToUnreal::ConvertPath(PrototypePath),
					*PrimPath.GetString()
				);
				continue;
			}

			UInstancedStaticMeshComponent* ISMComponent = nullptr;

			// The user could have just manually deleted the component, so we must check
			if (!AttachedISMs.IsValidIndex(PrototypeIndex))
			{
				USD_LOG_WARNING(
					TEXT("Failed to find corresponding ISM/HISM component for prototype '%s' of PointInstancer '%s'. Cancelling component update"),
					*UsdToUnreal::ConvertPath(PrototypePath),
					*PrimPath.GetString()
				);
				break;
			}
			ISMComponent = AttachedISMs[PrototypeIndex];

			// If our prototype was a LOD mesh we will have used the path of one of the actual LOD meshes to start the FGeomMeshCreateAssetsTaskChain,
			// so we have to look for our resulting mesh with the same path
			if (Context->bAllowInterpretingLODs && UsdUtils::DoesPrimContainMeshLODs(PrototypeUsdPrim.Get()))
			{
				pxr::UsdPrimSiblingRange PrimRange = PrototypeUsdPrim.Get().GetChildren();
				for (pxr::UsdPrimSiblingRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
				{
					const pxr::UsdPrim& Child = *PrimRangeIt;
					if (pxr::UsdGeomMesh ChildMesh{Child})
					{
						PrototypeUsdPrim = Child;
						PrototypePath = Child.GetPrimPath();
						break;
					}
				}
			}

			// This mesh could be nullptr, but that's OK
			UStaticMesh* StaticMesh = nullptr;
			std::unordered_map<pxr::SdfPath, UStaticMesh*, pxr::SdfPath::Hash>::iterator Iter = PrototypeMeshes.find(PrototypePath);
			if (Iter != PrototypeMeshes.end())
			{
				StaticMesh = Iter->second;
			}
			UsdGeomPointInstancerTranslatorImpl::SetStaticMesh(StaticMesh, *ISMComponent);

			// Evaluating point instancer can take a long time and is thread-safe. Move to async task while we work on something else.
			pxr::UsdTimeCode TimeCode{Context->Time};
			FUsdStageInfo StageInfo{Prim.GetStage()};
			Tasks.Emplace(Async(
				EAsyncExecution::ThreadPool,
				[TimeCode, StageInfo, PointInstancer, PrototypeIndex, ISMComponent]()
				{
					TArray<FTransform> InstanceTransforms;
					UsdUtils::GetPointInstancerTransforms(StageInfo, PointInstancer, PrototypeIndex, TimeCode, InstanceTransforms);

					return MakeTuple(ISMComponent, MoveTemp(InstanceTransforms));
				}
			));

			// Handle material overrides
			if (StaticMesh)
			{
				TArray<UMaterialInterface*> ExistingAssignments;
				for (FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
				{
					ExistingAssignments.Add(StaticMaterial.MaterialInterface);
				}

				MeshTranslationImpl::SetMaterialOverrides(PrototypeUsdPrim.Get(), ExistingAssignments, *ISMComponent, *Context);
			}

			// Handle visibility for the ISM component
			//
			// We will in general no longer put the visibility of the prototype itself baked into the mesh, so we
			// must add it to the ISM component. These ISM components sort of correspond to both the PointInstancer prim
			// and a particular prototype at the same time (kinda), so here we'll hide the ISM if either the PointInstancer or the
			// prototype are hidden, as if they were parent/child prims
			bool bIsVisible = !PointInstancerRootComponent->bHiddenInGame;
			if (pxr::UsdGeomImageable PrototypeImageable = pxr::UsdGeomImageable{PrototypeUsdPrim.Get()})
			{
				pxr::TfToken Visibility;
				pxr::UsdAttribute VisibilityAttr = PrototypeImageable.GetVisibilityAttr();
				if (VisibilityAttr && VisibilityAttr.Get(&Visibility, Context->Time) && Visibility == pxr::UsdGeomTokens->invisible)
				{
					bIsVisible = false;
				}
			}
			ISMComponent->SetHiddenInGame(!bIsVisible);
		}

		// Wait on and assign results of the point instancer.
		for (auto& Future : Tasks)
		{
			TTuple<UInstancedStaticMeshComponent*, TArray<FTransform>> Result{Future.Get()};
			UsdGeomPointInstancerTranslatorImpl::ApplyPointInstanceTransforms(Result.Key, Result.Value);
		}
	}
}

bool FUsdGeomPointInstancerTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	return true;
}

bool FUsdGeomPointInstancerTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return GCollapseTopLevelPointInstancers;
}

TSet<UE::FSdfPath> FUsdGeomPointInstancerTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim Prim = GetPrim();
	pxr::UsdGeomPointInstancer PointInstancer(Prim);
	if (!PointInstancer)
	{
		return {};
	}

	pxr::SdfPathVector PrototypePaths;
	if (!PointInstancer.GetPrototypesRel().GetTargets(&PrototypePaths))
	{
		return {};
	}

	TSet<UE::FSdfPath> Result;
	Result.Reserve(PrototypePaths.size());
	for (int32 PrototypeIndex = 0; PrototypeIndex < PrototypePaths.size(); ++PrototypeIndex)
	{
		UE::FSdfPath PrototypePath = UE::FSdfPath{PrototypePaths[PrototypeIndex]};
		UE::FUsdPrim PrototypePrim = Context->Stage.GetPrimAtPath(PrototypePath);

		Result.Add(PrototypePath);

		// Internal prototype
		// We must depend on all prims of the prototype subtree, because we're in charge of collapsing it
		if (PrototypePath.HasPrefix(PrimPath))
		{
			TArray<TUsdStore<pxr::UsdPrim>> ChildPrims = UsdUtils::GetAllPrimsOfType(PrototypePrim, pxr::TfType::Find<pxr::UsdGeomImageable>());

			for (const TUsdStore<pxr::UsdPrim>& ChildPrim : ChildPrims)
			{
				Result.Add(UE::FSdfPath{ChildPrim.Get().GetPrimPath()});

				TSharedPtr<FUsdSchemaTranslator> ChildSchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(
					Context,
					UE::FUsdTyped(ChildPrim.Get())
				);

				if (ChildSchemaTranslator)
				{
					TSet<UE::FSdfPath> RecursiveDependencies = ChildSchemaTranslator->CollectAuxiliaryPrims();
					for (const UE::FSdfPath& RecursiveDependency : RecursiveDependencies)
					{
						Result.Add(RecursiveDependency);
					}
				}
			}
		}
		// External prototype
		// Depend on prims until they collapse into something, at which point we can stop as they will depend on their
		// own subtree by themselves already
		else
		{
			pxr::UsdPrimRange PrimRange(PrototypePrim, pxr::UsdTraverseInstanceProxies());

			for (pxr::UsdPrimRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
			{
				Result.Add(UE::FSdfPath{PrimRangeIt->GetPrimPath()});

				TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(
					Context,
					UE::FUsdTyped(*PrimRangeIt)
				);
				if (SchemaTranslator && SchemaTranslator->CollapsesChildren(ECollapsingType::Assets))
				{
					PrimRangeIt.PruneChildren();
				}
			}
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
