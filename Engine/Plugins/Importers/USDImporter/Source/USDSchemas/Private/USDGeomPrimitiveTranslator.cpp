// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGeomPrimitiveTranslator.h"

#if USE_USD_SDK

#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "UnrealUSDWrapper.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDGeomMeshConversion.h"
#include "USDGeomMeshTranslator.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "MeshTranslationImpl.h"
#include "StaticMeshAttributes.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "USDIncludesEnd.h"

namespace UsdGeomPrimitiveTranslatorImpl
{
	void LoadMeshDescriptions(
		UE::FUsdPrim Prim,
		TArray<FMeshDescription>& OutLODIndexToMeshDescription,
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& OutLODIndexToMaterialInfo,
		const UsdToUnreal::FUsdMeshConversionOptions& Options
	)
	{
		if (!Prim)
		{
			return;
		}

		FMeshDescription TempMeshDescription;
		UsdUtils::FUsdPrimMaterialAssignmentInfo TempMaterialInfo;

		FStaticMeshAttributes StaticMeshAttributes(TempMeshDescription);
		StaticMeshAttributes.Register();

		const bool bSuccess = UsdToUnreal::ConvertGeomPrimitive(Prim, TempMeshDescription, TempMaterialInfo, Options);
		if (bSuccess)
		{
			OutLODIndexToMeshDescription = {MoveTemp(TempMeshDescription)};
			OutLODIndexToMaterialInfo = {MoveTemp(TempMaterialInfo)};
		}
	}

	class FGeomPrimitiveCreateAssetsTaskChain : public FBuildStaticMeshTaskChain
	{
	public:
		explicit FGeomPrimitiveCreateAssetsTaskChain(
			const TSharedRef<FUsdSchemaTranslationContext>& InContext,
			const UE::FSdfPath& InPrimPath,
			const TOptional<UE::FSdfPath>& AlternativePrimToLinkAssetsTo = {}
		)
			: FBuildStaticMeshTaskChain(InContext, InPrimPath, AlternativePrimToLinkAssetsTo)
		{
			SetupTasks();
		}

	protected:
		virtual void SetupTasks() override
		{
			FScopedUnrealAllocs UnrealAllocs;

			// Create mesh descriptions (Async)
			Do(ESchemaTranslationLaunchPolicy::Async,
			   [this]() -> bool
			   {
				   pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
				   if (!Context->RenderContext.IsNone())
				   {
					   RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
				   }

				   pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
				   if (!Context->MaterialPurpose.IsNone())
				   {
					   MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
				   }

				   UsdToUnreal::FUsdMeshConversionOptions Options;
				   Options.TimeCode = Context->Time;
				   Options.PurposesToLoad = Context->PurposesToLoad;
				   Options.RenderContext = RenderContextToken;
				   Options.MaterialPurpose = MaterialPurposeToken;
				   Options.bMergeIdenticalMaterialSlots = Context->bMergeIdenticalMaterialSlots;
				   Options.SubdivisionLevel = Context->SubdivisionLevel;

				   UsdGeomPrimitiveTranslatorImpl::LoadMeshDescriptions(GetPrim(), LODIndexToMeshDescription, LODIndexToMaterialInfo, Options);

				   // If we have at least one valid LOD, we should keep going
				   for (const FMeshDescription& MeshDescription : LODIndexToMeshDescription)
				   {
					   if (!MeshDescription.IsEmpty())
					   {
						   return true;
					   }
				   }
				   return false;
			   });

			FBuildStaticMeshTaskChain::SetupTasks();
		}
	};
}	 // namespace UsdGeomPrimitiveTranslatorImpl

void FUsdGeomPrimitiveTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomPrimitiveTranslator::CreateAssets);

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	using namespace UsdGeomPrimitiveTranslatorImpl;
	TSharedRef<FGeomPrimitiveCreateAssetsTaskChain> AssetsTaskChain = MakeShared<FGeomPrimitiveCreateAssetsTaskChain>(Context, PrimPath);

	Context->TranslatorTasks.Add(MoveTemp(AssetsTaskChain));
}

USceneComponent* FUsdGeomPrimitiveTranslator::CreateComponents()
{
	USceneComponent* SceneComponent = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		SceneComponent = CreateComponentsEx({}, {});
	}
	else
	{
		SceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	UpdateComponents(SceneComponent);

	// Handle material overrides
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComponent))
	{
		if (Context->PrimLinkCache)
		{
			if (UStaticMesh* StaticMesh = Context->PrimLinkCache->GetSingleAssetForPrim<UStaticMesh>(PrimPath))
			{
				TArray<UMaterialInterface*> ExistingAssignments;
				for (FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
				{
					ExistingAssignments.Add(StaticMaterial.MaterialInterface);
				}

				MeshTranslationImpl::SetMaterialOverrides(GetPrim(), ExistingAssignments, *StaticMeshComponent, *Context);
			}
		}
	}

	return SceneComponent;
}

void FUsdGeomPrimitiveTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	if (UsdUtils::IsAnimated(GetPrim()))
	{
		// The assets might have changed since our attributes are animated
		// Note that we must wait for these to complete as they make take a while and we want to
		// reassign our new static meshes when we get to FUsdGeomXformableTranslator::UpdateComponents
		CreateAssets();
		Context->CompleteTasks();
	}

	Super::UpdateComponents(SceneComponent);
}

bool FUsdGeomPrimitiveTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	// Gprims shouldn't really have any children so this is not very well defined.
	// We're going with 'false' here to match FUsdGeomMeshTranslator::CollapsesChildren
	return false;
}

bool FUsdGeomPrimitiveTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return Super::CanBeCollapsed(CollapsingType);
}

TSet<UE::FSdfPath> FUsdGeomPrimitiveTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	// Let's assume we can't specify UsdGeomSubsets on primitives for now
	return {};
}

#endif	  // #if USE_USD_SDK
