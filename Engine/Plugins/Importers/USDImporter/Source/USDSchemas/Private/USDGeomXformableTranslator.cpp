// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGeomXformableTranslator.h"

#include "MeshTranslationImpl.h"
#include "Objects/USDPrimLinkCache.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDGeomMeshTranslator.h"
#include "USDIntegrationUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDSchemasModule.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"

#include "Camera/CameraActor.h"
#include "Components/LightComponentBase.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkRole.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PhysicsEngine/BodySetup.h"
#include "Roles/LiveLinkTransformRole.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdGeomBBoxCache.h"
#include "UsdWrappers/UsdGeomXformable.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/usd/usd/modelAPI.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usdGeom/modelAPI.h"
#include "pxr/usd/usdGeom/pointInstancer.h"
#include "pxr/usd/usdGeom/xformable.h"
#include "pxr/usd/usdSkel/root.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDGeomXformableTranslator"

static bool GEnableCollision = true;
static FAutoConsoleVariableRef CVarEnableCollision(
	TEXT("USD.EnableCollision"),
	GEnableCollision,
	TEXT("Whether to have collision enabled for spawned components and generated meshes")
);

namespace UE::UsdXformableTranslatorImpl::Private
{
	void SetUpSceneComponentForLiveLink(const FUsdSchemaTranslationContext& Context, USceneComponent* Component, const pxr::UsdPrim& Prim)
	{
		if (!Component || !Prim)
		{
			return;
		}

		AActor* Parent = Component->GetOwner();
		if (!Parent)
		{
			return;
		}

		USceneComponent* RootComponent = Parent->GetRootComponent();
		if (!RootComponent)
		{
			return;
		}

		ULiveLinkComponentController* Controller = nullptr;
		{
			// We would have to traverse all top-level actor components to know if our component is set up for live link already
			// or not, so this just helps us make that a little bit faster. Its important because UpdateComponents (which calls us)
			// is the main function that is called to animate components, so it can be spammed in case this prim has animations
			static TMap<TWeakObjectPtr<USceneComponent>, TWeakObjectPtr<ULiveLinkComponentController>> LiveLinkEnabledComponents;
			if (ULiveLinkComponentController* ExistingController = LiveLinkEnabledComponents.FindRef(Component).Get())
			{
				// We found an existing controller we created to track this component, so use that
				Controller = ExistingController;
			}
			// We don't know of any controllers handling this component yet, get a new one
			else
			{
				TArray<ULiveLinkComponentController*> LiveLinkComponents;
				Parent->GetComponents(LiveLinkComponents);

				for (ULiveLinkComponentController* LiveLinkComponent : LiveLinkComponents)
				{
					if (LiveLinkComponent->GetControlledComponent(ULiveLinkTransformRole::StaticClass()) == Component)
					{
						// We found some other controller handling this component somehow, use that
						Controller = LiveLinkComponent;
						break;
					}
				}

				if (!Controller)
				{
					// We'll get a warning from the live link controller component in case the component its controlling is not movable
					Component->Mobility = EComponentMobility::Movable;

					Controller = NewObject<ULiveLinkComponentController>(Parent, NAME_None, Context.ObjectFlags);
					Controller->bUpdateInEditor = true;

					// Important because of how ULiveLinkComponentController::TickComponent also checks for the sequencer
					// tag to try and guess if the controlled component is a spawnable
					Controller->bDisableEvaluateLiveLinkWhenSpawnable = false;

					Parent->AddInstanceComponent(Controller);
					Controller->RegisterComponent();
				}

				if (Controller)
				{
					LiveLinkEnabledComponents.Add(Component, Controller);
				}
			}
		}

		// Configure controller with our desired parameters
		if (Controller)
		{
			FScopedUsdAllocs Allocs;

			FLiveLinkSubjectRepresentation SubjectRepresentation;
			SubjectRepresentation.Role = ULiveLinkTransformRole::StaticClass();

			if (pxr::UsdAttribute Attr = Prim.GetAttribute(UnrealIdentifiers::UnrealLiveLinkSubjectName))
			{
				std::string SubjectName;
				if (Attr.Get(&SubjectName))
				{
					SubjectRepresentation.Subject = FName{*UsdToUnreal::ConvertString(SubjectName)};
				}
			}

			{
				FScopedUnrealAllocs UEAllocs;

				Controller->SetSubjectRepresentation(SubjectRepresentation);

				// This should be done after setting the subject representation to ensure that the LiveLink component's ControllerMap has a transform
				// controller
				Controller->SetControlledComponent(ULiveLinkTransformRole::StaticClass(), Component);
			}

			if (pxr::UsdAttribute Attr = Prim.GetAttribute(UnrealIdentifiers::UnrealLiveLinkEnabled))
			{
				bool bEnabled = true;
				if (Attr.Get(&bEnabled))
				{
					Controller->bEvaluateLiveLink = bEnabled;
				}
			}
		}
	}

	void RemoveLiveLinkFromComponent(USceneComponent* Component)
	{
		if (!Component)
		{
			return;
		}

		AActor* Parent = Component->GetOwner();
		if (!Parent)
		{
			return;
		}

		TArray<ULiveLinkComponentController*> LiveLinkComponents;
		Parent->GetComponents(LiveLinkComponents);

		for (ULiveLinkComponentController* LiveLinkComponent : LiveLinkComponents)
		{
			if (LiveLinkComponent->GetControlledComponent(ULiveLinkTransformRole::StaticClass()) == Component)
			{
				LiveLinkComponent->SetControlledComponent(ULiveLinkTransformRole::StaticClass(), nullptr);
				Parent->RemoveInstanceComponent(LiveLinkComponent);
				break;
			}
		}
	}
}	 // namespace UE::UsdXformableTranslatorImpl::Private

class FUsdGeomXformableCreateAssetsTaskChain : public FBuildStaticMeshTaskChain
{
public:
	explicit FUsdGeomXformableCreateAssetsTaskChain(
		const TSharedRef<FUsdSchemaTranslationContext>& InContext,
		const UE::FSdfPath& InPrimPath,
		const TOptional<UE::FSdfPath>& AlternativePrimToLinkAssetsTo = {}
	)
		: FBuildStaticMeshTaskChain(InContext, InPrimPath, AlternativePrimToLinkAssetsTo)
	{
		SetupTasks();
	}

protected:
	virtual void SetupTasks() override;
};

void FUsdGeomXformableCreateAssetsTaskChain::SetupTasks()
{
	FScopedUnrealAllocs UnrealAllocs;

	// Create mesh description (Async)
	Do(ESchemaTranslationLaunchPolicy::Async,
	   [this]() -> bool
	   {
		   // We will never have multiple LODs of meshes that were collapsed together, as LOD'd meshes don't collapse. So just parse the mesh we get
		   // as LOD0
		   LODIndexToMeshDescription.Reset(1);
		   LODIndexToMaterialInfo.Reset(1);

		   FMeshDescription& AddedMeshDescription = LODIndexToMeshDescription.Emplace_GetRef();
		   UsdUtils::FUsdPrimMaterialAssignmentInfo& AssignmentInfo = LODIndexToMaterialInfo.Emplace_GetRef();

		   pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
		   if (!Context->RenderContext.IsNone())
		   {
			   RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
		   }

		   pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		   if (!Context->MaterialPurpose.IsNone())
		   {
			   MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
		   }

		   // We're going to put Prim's transform and visibility on the component, so we don't need to bake it into the combined mesh
		   const bool bSkipRootPrimTransform = true;
		   const bool bSkipRootPrimVisibility = true;

		   UsdToUnreal::FUsdMeshConversionOptions Options;
		   Options.TimeCode = Context->Time;
		   Options.PurposesToLoad = Context->PurposesToLoad;
		   Options.RenderContext = RenderContextToken;
		   Options.MaterialPurpose = MaterialPurposeToken;
		   Options.bMergeIdenticalMaterialSlots = Context->bMergeIdenticalMaterialSlots;
		   Options.SubdivisionLevel = Context->SubdivisionLevel;

		   UsdToUnreal::ConvertGeomMeshSubtree(
			   GetPrim(),
			   AddedMeshDescription,
			   AssignmentInfo,
			   Options,
			   bSkipRootPrimTransform,
			   bSkipRootPrimVisibility
		   );

		   return !AddedMeshDescription.IsEmpty();
	   });

	FBuildStaticMeshTaskChain::SetupTasks();
}

void FUsdGeomXformableTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomMeshTranslator::CreateAssets);

	if (!CollapsesChildren(ECollapsingType::Assets))
	{
		// We only have to create assets when our children are collapsed together
		return;
	}

	if (ShouldSkipInstance())
	{
		return;
	}

	UE::FUsdPrim Prim = GetPrim();

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(Prim);
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	UE::FSdfPath PrototypePrimPath = GetPrototypePrimPath();
	Context->TranslatorTasks.Add(MakeShared<FUsdGeomXformableCreateAssetsTaskChain>(Context, PrimPath, PrototypePrimPath));
}

FUsdGeomXformableTranslator::FUsdGeomXformableTranslator(
	TSubclassOf<USceneComponent> InComponentTypeOverride,
	TSharedRef<FUsdSchemaTranslationContext> InContext,
	const UE::FUsdTyped& InSchema
)
	: FUsdSchemaTranslator(InContext, InSchema)
	, ComponentTypeOverride(InComponentTypeOverride)
{
}

USceneComponent* FUsdGeomXformableTranslator::CreateComponents()
{
	USceneComponent* SceneComponent = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		SceneComponent = CreateComponentsEx({}, {});
	}
	else
	{
		SceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	// We pulled UpdateComponents outside CreateComponentsEx as in some cases we don't want to do it
	// right away (like on FUsdGeomPointInstancerTranslator::CreateComponents)
	UpdateComponents(SceneComponent);

	return SceneComponent;
}

USceneComponent* FUsdGeomXformableTranslator::CreateComponentsEx(TOptional<TSubclassOf<USceneComponent>> ComponentType, TOptional<bool> bNeedsActor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomXformableTranslator::CreateComponentsEx);

	if (!Context->IsValid())
	{
		return nullptr;
	}

	UE::FUsdPrim Prim = GetPrim();
	if (!Prim)
	{
		return nullptr;
	}

	FScopedUnrealAllocs UnrealAllocs;

	if (!bNeedsActor.IsSet())
	{
		// Don't add components to the AUsdStageActor or the USDStageImport 'scene actor'
		UE::FUsdPrim ParentPrim = Prim.GetParent();
		bool bIsTopLevelPrim = ParentPrim.IsValid() && ParentPrim.IsPseudoRoot();

		// If we don't have any parent prim with a type that generates a component, we are still technically a top-level prim
		if (!bIsTopLevelPrim)
		{
			bool bHasParentComponent = false;
			while (ParentPrim.IsValid())
			{
				if (UE::FUsdGeomXformable(ParentPrim))
				{
					bHasParentComponent = true;
					break;
				}

				ParentPrim = ParentPrim.GetParent();
			}
			if (!bHasParentComponent)
			{
				bIsTopLevelPrim = true;
			}
		}

		auto PrimNeedsActor = [](const UE::FUsdPrim& UsdPrim) -> bool
		{
			// clang-format off
			return  UsdPrim.IsPseudoRoot() ||
					UsdPrim.IsModel() ||
					UsdPrim.IsGroup() ||
					UsdUtils::HasCompositionArcs( UsdPrim ) ||
					UsdPrim.HasAttribute( TEXT( "unrealCameraPrimName" ) ) ||  // If we have this, then we correspond to the root component
																			   // of an exported ACineCameraActor. Let's create an actual
																			   // CineCameraActor here so that our child camera prim can just
																			   // take it's UCineCameraComponent instead
					UsdPrim.IsA(TEXT("SkelRoot"));  // Now that we use the UsdSkelSkeletonTranslator, UsdSkelRoots will be handled like regular
													// Xforms. We likely always want then to show up on the outliner though, as they are important
													// prims
			// clang-format on
		};

		bNeedsActor = (bIsTopLevelPrim || Context->ParentComponent == nullptr || PrimNeedsActor(Prim));

		// We don't want to start a component hierarchy if one of our child will break it by being an actor
		if (!bNeedsActor.GetValue())
		{
			TFunction<bool(const UE::FUsdPrim&)> RecursiveChildPrimsNeedsActor;
			RecursiveChildPrimsNeedsActor = [PrimNeedsActor, &RecursiveChildPrimsNeedsActor](const UE::FUsdPrim& UsdPrim) -> bool
			{
				const bool bTraverseInstanceProxies = true;
				for (const pxr::UsdPrim& Child : UsdPrim.GetFilteredChildren(bTraverseInstanceProxies))
				{
					if (PrimNeedsActor(UE::FUsdPrim(Child)))
					{
						return true;
					}
					else if (RecursiveChildPrimsNeedsActor(UE::FUsdPrim(Child)))
					{
						return true;
					}
				}

				return false;
			};

			bNeedsActor = RecursiveChildPrimsNeedsActor(UE::FUsdPrim(Prim));
		}
	}

	USceneComponent* SceneComponent = nullptr;
	UObject* ComponentOuter = nullptr;

	// Can't have public or standalone on spawned actors and components because that
	// will lead to asserts when trying to collect them during a level change, or when
	// trying to replace them (right-clicking from the world outliner). Also, must set
	// the transient flag after spawn to make sure the spawned actor get an external
	// package if needed.
	const EObjectFlags PreComponentFlags = Context->ObjectFlags & ~(RF_Standalone | RF_Public | RF_Transient);
	EObjectFlags PostComponentFlags = Context->ObjectFlags & ~(RF_Standalone | RF_Public);
	if (!Context->bIsImporting)
	{
		PostComponentFlags |= RF_Transient;
	}

	if (bNeedsActor.GetValue())
	{
		// Spawn actor
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.ObjectFlags = PreComponentFlags;
		SpawnParameters.OverrideLevel = Context->Level;
		SpawnParameters.Name = Prim.GetName();
		SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;	 // Will generate a unique name in case of a conflict

		UClass* ActorClass = nullptr;
		if (ComponentType.Get({}) == UUsdDrawModeComponent::StaticClass())
		{
			// If we've been told to spawn a bounds component, we never want to create a light or camera actor or etc., as those
			// come with their own specific root components
			ActorClass = AActor::StaticClass();
		}
		else
		{
			ActorClass = UsdUtils::GetActorTypeForPrim(Prim);
		}

		AActor* SpawnedActor = Context->Level->GetWorld()->SpawnActor(ActorClass, nullptr, SpawnParameters);

		if (SpawnedActor)
		{
			SpawnedActor->SetFlags(PostComponentFlags);

#if WITH_EDITOR
			const bool bMarkDirty = false;
			SpawnedActor->SetActorLabel(Prim.GetName().ToString(), bMarkDirty);

			// If our AUsdStageActor is in a hidden level/layer and we spawn actors, they should also be hidden
			if (Context->ParentComponent)
			{
				if (AActor* ParentActor = Context->ParentComponent->GetOwner())
				{
					SpawnedActor->bHiddenEdLevel = ParentActor->bHiddenEdLevel;
					SpawnedActor->bHiddenEdLayer = ParentActor->bHiddenEdLayer;
				}
			}
#endif	  // WITH_EDITOR

			SceneComponent = SpawnedActor->GetRootComponent();

			ComponentOuter = SpawnedActor;
		}
	}
	else
	{
		ComponentOuter = Context->ParentComponent->GetOwner();
	}

	if (!ComponentOuter)
	{
		USD_LOG_WARNING(TEXT("Invalid outer when trying to create SceneComponent for prim (%s)"), *PrimPath.GetString());
		return nullptr;
	}

	if (!SceneComponent)
	{
		if (!ComponentType.IsSet())
		{
			if (ComponentTypeOverride.IsSet())
			{
				ComponentType = ComponentTypeOverride.GetValue();
			}
			else
			{
				ComponentType = UsdUtils::GetComponentTypeForPrim(Prim);

				// For now only upgrade actual scene components to static mesh components (important because skeletal mesh components will also fit
				// this criteria but we don't want to use a static mesh component for those)
				if (CollapsesChildren(ECollapsingType::Assets) && ComponentType.IsSet() && ComponentType.GetValue() == USceneComponent::StaticClass())
				{
					// If we're a type that collapses assets, we should probably be a static mesh component as we only really collapse static meshes
					// together right now. We can't just check if there's a static mesh for this prim on the cache, because the prims with meshes
					// could be potentially invisible (and so we don't have parsed their meshes yet), so here we traverse our child hierarchy and if
					// we have any chance of ever generating a Mesh, we go for a static mesh component
					TArray<UE::FUsdPrim> ChildGprims = UsdUtils::GetAllPrimsOfType(Prim, TEXT("UsdGeomGprim"));
					if (ChildGprims.Num() > 0)
					{
						ComponentType = UStaticMeshComponent::StaticClass();
					}
				}
				// If this is a component for a point instancer that just collapsed itself into a static mesh, just make
				// a static mesh component that can receive it
				else if (pxr::UsdPrim{Prim}.IsA<pxr::UsdGeomPointInstancer>())
				{
					static IConsoleVariable* CollapseCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.CollapseTopLevelPointInstancers"));
					if (CollapseCvar && CollapseCvar->GetBool())
					{
						ComponentType = UStaticMeshComponent::StaticClass();
					}
				}
			}
		}

		if (ComponentType.IsSet() && ComponentType.GetValue() != nullptr)
		{
			const FName ComponentName = MakeUniqueObjectName(
				ComponentOuter,
				ComponentType.GetValue(),
				*UsdUnreal::ObjectUtils::SanitizeObjectName(Prim.GetName().ToString())
			);
			SceneComponent = NewObject<USceneComponent>(ComponentOuter, ComponentType.GetValue(), ComponentName, PreComponentFlags);
			SceneComponent->SetFlags(PostComponentFlags);

			if (AActor* Owner = SceneComponent->GetOwner())
			{
				Owner->AddInstanceComponent(SceneComponent);
			}
		}
	}

	if (Context->MetadataOptions.bCollectMetadata && Context->MetadataOptions.bCollectOnComponents)
	{
		UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(SceneComponent);

		// It makes sense for asset metadata to "include all prims in the subtree", as when we generate an
		// asset we don't generate additional separate assets for child prims. This is not the same behavior
		// for components though, so it doesn't feel like "collecting from the entire subtree" should be
		// allowed for them. In other words, if we allowed this the root scene component for the stage will
		// contain metadata from the entire stage every time...
		const bool bCollectMetadataFromSubtree = false;
		UsdToUnreal::ConvertMetadata(
			Prim,
			UserData,
			Context->MetadataOptions.BlockedPrefixFilters,
			Context->MetadataOptions.bInvertFilters,
			bCollectMetadataFromSubtree
		);
	}
	else if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(SceneComponent))
	{
		// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the AssetUserData
		UserData->StageIdentifierToMetadata.Remove(Prim.GetStage().GetRootLayer().GetIdentifier());
	}

	if (SceneComponent)
	{
		if (!GEnableCollision)
		{
			// In most cases this will have no benefit memory-wise, as regular UStaticMeshComponents build their physics meshes anyway
			// when registering, regardless of these. ISM components will *not* build them though, so disabling the cvar may lead
			// to some memory savings
			if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(SceneComponent))
			{
				PrimComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
				PrimComp->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			}
		}

		if (!SceneComponent->GetOwner()->GetRootComponent())
		{
			SceneComponent->GetOwner()->SetRootComponent(SceneComponent);
		}

		// If we're spawning into a level that is being streamed in, our construction scripts will be rerun, and may want to set the scene component
		// location again. Since our spawned actors are already initialized, that may trigger a warning about the component not being movable,
		// so we must force them movable here
		const bool bIsAssociating = Context->Level && Context->Level->bIsAssociatingLevel;
		const bool bParentIsMovable = Context->ParentComponent && Context->ParentComponent->Mobility == EComponentMobility::Movable;
		const bool bParentIsStationary = Context->ParentComponent && Context->ParentComponent->Mobility == EComponentMobility::Stationary;
		const bool bIsCamera = SceneComponent->GetOwner() && SceneComponent->GetOwner()->IsA(ACameraActor::StaticClass());

		// Don't call SetMobility as it would trigger a reregister, queuing unnecessary rhi commands since this is a brand new component
		// Always have movable Skeletal mesh components or else we get some warnings when building physics assets
		if (bIsAssociating || bParentIsMovable || bIsCamera || SceneComponent->IsA<USkeletalMeshComponent>() || UsdUtils::IsAnimated(Prim))
		{
			SceneComponent->Mobility = EComponentMobility::Movable;
		}
		else if (bParentIsStationary || SceneComponent->IsA<ULightComponentBase>())
		{
			SceneComponent->Mobility = EComponentMobility::Stationary;
		}
		else
		{
			SceneComponent->Mobility = EComponentMobility::Static;
		}

		// Attach to parent
		// Do this before UpdatingComponents as we may need to use the parent transform to set a world transform directly
		// (in case of resetXformStack). Besides, this is more consistent anyway as during stage updates we'll call
		// UpdateComponents with all the components already attached
		SceneComponent->AttachToComponent(Context->ParentComponent, FAttachmentTransformRules::KeepRelativeTransform);

		if (!SceneComponent->IsRegistered())
		{
			SceneComponent->RegisterComponent();
		}
	}

	return SceneComponent;
}

void FUsdGeomXformableTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomXformableTranslator::UpdateComponents);

	if (SceneComponent && Context->PrimLinkCache)
	{
		SceneComponent->Modify();

		// UsdToUnreal::ConvertXformable will set a new transform, which will emit warnings during PIE/Runtime if the component
		// is not movable, so here we unregister, set the new transform value, and reregister below
		if (SceneComponent->Mobility != EComponentMobility::Movable && SceneComponent->IsRegistered())
		{
			SceneComponent->UnregisterComponent();
		}

		UE::FUsdPrim Prim = GetPrim();

		// If the user modified a mesh parameter (e.g. vertex color), the hash will be different and it will become a separate asset
		// so we must check for this and assign the new StaticMesh
		bool bHasMultipleLODs = false;
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComponent))
		{
			UE::FSdfPath PrototypePath = GetPrototypePrimPath();
			UStaticMesh* PrimStaticMesh = Context->PrimLinkCache->GetSingleAssetForPrim<UStaticMesh>(PrototypePath);

			if (PrimStaticMesh)
			{
				bHasMultipleLODs = PrimStaticMesh->GetNumLODs() > 1;
			}

			if (PrimStaticMesh != StaticMeshComponent->GetStaticMesh())
			{
				// Need to make sure the mesh's resources are initialized here as it may have just been built in another thread
				// Only do this if required though, as this mesh could using these resources currently (e.g. PIE and editor world sharing the mesh)
				if (PrimStaticMesh && !PrimStaticMesh->AreRenderingResourcesInitialized()
					&& (FApp::CanEverRender() || !FPlatformProperties::RequiresCookedData()))
				{
					PrimStaticMesh->InitResources();
				}

				if (StaticMeshComponent->IsRegistered())
				{
					StaticMeshComponent->UnregisterComponent();
				}

				StaticMeshComponent->SetStaticMesh(PrimStaticMesh);

				// We can't register yet, as UsdToUnreal::ConvertXformable below us may want to move the component.
				// We'll always re-register when needed below, though.
			}

			// Update the collision settings of the component in case they changed on the static mesh
			if (PrimStaticMesh && PrimStaticMesh->GetBodySetup())
			{
				StaticMeshComponent->BodyInstance.SetCollisionEnabled(PrimStaticMesh->GetBodySetup()->DefaultInstance.GetCollisionEnabled());
				StaticMeshComponent->BodyInstance.SetCollisionProfileName(PrimStaticMesh->GetBodySetup()->DefaultInstance.GetCollisionProfileName());
			}
		}
		else if (UUsdDrawModeComponent* DrawModeComponent = Cast<UUsdDrawModeComponent>(SceneComponent))
		{
			TOptional<FWriteScopeLock> BBoxLock;
			pxr::UsdGeomBBoxCache* PxrBBoxCache = nullptr;
			if (UE::FUsdGeomBBoxCache* UEBBoxCache = Context->BBoxCache.Get())
			{
				BBoxLock.Emplace(UEBBoxCache->Lock);
				PxrBBoxCache = &static_cast<pxr::UsdGeomBBoxCache&>(*UEBBoxCache);
			}
			UsdToUnreal::ConvertDrawMode(Prim, DrawModeComponent, Context->Time, PxrBBoxCache);
		}

		// Handle LiveLink, but only if we're not a skeletal case: The SkelSkeletonTranslator will deal with the
		// skeletal version of the LiveLink configuration, we only handle setting up LiveLink for simple transforms
		if (!Prim.IsA(TEXT("SkelRoot")) && !Prim.IsA(TEXT("Skeleton")))
		{
			if (UsdUtils::PrimHasSchema(Prim, UnrealIdentifiers::LiveLinkAPI))
			{
				UE::UsdXformableTranslatorImpl::Private::SetUpSceneComponentForLiveLink(Context.Get(), SceneComponent, Prim);
			}
			else
			{
				UE::UsdXformableTranslatorImpl::Private::RemoveLiveLinkFromComponent(SceneComponent);
			}
		}

		// Only put the transform into the component if we haven't parsed LODs for our static mesh: The Mesh transforms will already be baked
		// into the mesh at that case, as each LOD could technically have a separate transform
		if (!Context->bSequencerIsAnimating && (!Context->bAllowInterpretingLODs || !bHasMultipleLODs))
		{
			// Don't update the component's transform if this is already factored in as root motion within the AnimSequence
			bool bConvertTransform = true;
			switch (Context->RootMotionHandling)
			{
				default:
				case EUsdRootMotionHandling::NoAdditionalRootMotion:
				{
					break;
				}
				case EUsdRootMotionHandling::UseMotionFromSkelRoot:
				{
					if (Prim.IsA(TEXT("SkelRoot")))
					{
						bConvertTransform = false;
					}
					break;
				}
				case EUsdRootMotionHandling::UseMotionFromSkeleton:
				{
					if (Prim.IsA(TEXT("Skeleton")))
					{
						bConvertTransform = false;
					}
					break;
				}
			}

			UsdToUnreal::ConvertXformable(Context->Stage, pxr::UsdGeomXformable(Prim), *SceneComponent, Context->Time, bConvertTransform);
		}

		// Note how we should only register if we unregistered ourselves: If we did this every time we would
		// register too early during the process of duplicating into PIE, and that would prevent a future RegisterComponent
		// call from naturally creating the required render state
		if (!SceneComponent->IsRegistered())
		{
			SceneComponent->RegisterComponent();
		}

		// Handle material overrides for all kinds of meshes, even collapsed meshes.
		// This can happen if we have two separate subtrees that collapse the same: A single static mesh will be shared between
		// them and one of the task chains will manage to put their material assignments on the mesh directly. To ensure the
		// correct materials for the second subtree, we need to set overrides.
		// Note how we don't have to handle geometry caches in here as they're handled by the geometry cache translator now
		if (Context->bAllowRecomputingMaterialOverrides)
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SceneComponent))
			{
				UE::FSdfPath PrototypePath = GetPrototypePrimPath();
				if (UStaticMesh* StaticMesh = Context->PrimLinkCache->GetSingleAssetForPrim<UStaticMesh>(PrototypePath))
				{
					TArray<UMaterialInterface*> ExistingAssignments;
					for (FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
					{
						ExistingAssignments.Add(StaticMaterial.MaterialInterface);
					}

					MeshTranslationImpl::SetMaterialOverrides(GetPrim(), ExistingAssignments, *StaticMeshComponent, *Context);
				}
			}
		}
	}
}

namespace UE::UsdXformableTranslatorImpl::Private
{
	void AssignDrawModeComponentTextures(
		UE::FUsdPrim Prim,
		UUsdDrawModeComponent* DrawModeComponent,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache
	)
	{
		if (!Prim)
		{
			return;
		}

		FScopedUsdAllocs Allocs;

		pxr::UsdPrim UsdPrim{Prim};
		pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
		FUsdStageInfo StageInfo{Stage};

		pxr::UsdGeomModelAPI GeomModelAPI{UsdPrim};
		if (!GeomModelAPI)
		{
			return;
		}

		// Collect textures from the info cache (they will all be linked to this Prim but will have within their AssetUserData the attribute
		// that they originated from).
		// The info cache may have old textures in case we're changing editing the stage, so we track multiple textures per attribute.
		TArray<UTexture2D*> Textures = PrimLinkCache.GetAssetsForPrim<UTexture2D>(Prim.GetPrimPath());
		std::unordered_map<pxr::SdfPath, TArray<UTexture2D*>, pxr::SdfPath::Hash> AttrPathToTextures;
		for (UTexture2D* Texture : Textures)
		{
			if (UUsdAssetUserData* AssetUserData = Texture->GetAssetUserData<UUsdAssetUserData>())
			{
				for (const FString& TexturePath : AssetUserData->PrimPaths)
				{
					if (TexturePath.IsEmpty())
					{
						continue;
					}

					pxr::SdfPath SdfPath = UnrealToUsd::ConvertPath(*TexturePath).Get();
					if (SdfPath.IsPropertyPath())
					{
						TArray<UTexture2D*>& TexturesForAttr = AttrPathToTextures[SdfPath];
						TexturesForAttr.AddUnique(Texture);
					}
				}
			}
		}

		// Switch up the faces depending on stage up axis. The effect of metersPerUnit is already baked in the size of the bounds,
		// but here we "convert the faces" to swap the axes so that the UUsdDrawModeComponent component properties can reference faces in
		// the UE coordinate system (e.g. PosY in the USD stage will become PosZ in UE coordinate system if the stage is Y up, but then
		// you will actually see the +Z face in UE, and the PosZ property on the component will be set to match it).
		pxr::UsdAttribute XPosAttr = GeomModelAPI.GetModelCardTextureXPosAttr();
		pxr::UsdAttribute YPosAttr = GeomModelAPI.GetModelCardTextureYPosAttr();
		pxr::UsdAttribute ZPosAttr = GeomModelAPI.GetModelCardTextureZPosAttr();
		pxr::UsdAttribute XNegAttr = GeomModelAPI.GetModelCardTextureXNegAttr();
		pxr::UsdAttribute YNegAttr = GeomModelAPI.GetModelCardTextureYNegAttr();
		pxr::UsdAttribute ZNegAttr = GeomModelAPI.GetModelCardTextureZNegAttr();
		if (StageInfo.UpAxis == EUsdUpAxis::ZAxis)
		{
			Swap(YPosAttr, YNegAttr);
		}
		else
		{
			Swap(YPosAttr, ZPosAttr);
			Swap(YNegAttr, ZNegAttr);
		}

		EUsdModelCardFace AuthoredFaces = EUsdModelCardFace::None;

		using TextureSetterFunc = void (UUsdDrawModeComponent::*)(UTexture2D*);

		TFunction<void(const pxr::UsdAttribute&, TextureSetterFunc, EUsdModelCardFace)> HandleCardFace =
			[&AuthoredFaces,
			 DrawModeComponent,
			 &AttrPathToTextures](const pxr::UsdAttribute& Attr, TextureSetterFunc TextureSetter, EUsdModelCardFace Face)
		{
			if (Attr && Attr.HasAuthoredValue())
			{
				AuthoredFaces |= Face;

				if (TextureSetter)
				{
					std::unordered_map<pxr::SdfPath, TArray<UTexture2D*>, pxr::SdfPath::Hash>::iterator iter = AttrPathToTextures.find(Attr.GetPath()
					);
					if (iter != AttrPathToTextures.end())
					{
						const FString TexturePath = UsdUtils::GetResolvedAssetPath(Attr);

						const TArray<UTexture2D*>& Textures = iter->second;
						for (UTexture2D* Texture : Textures)
						{
							if (!Texture)
							{
								continue;
							}

#if WITH_EDITOR
							// Check that the texture is the desired one for the attribute by checking
							// its source path. Unfortunately we can only do this check in the editor for now,
							// but this should only be relevant during workflows with card editing, which shouldn't
							// happen at runtime anyway (see UE-200918)
							if (UAssetImportData* ImportData = Texture->AssetImportData)
							{
								if (!FPaths::IsSamePath(TexturePath, ImportData->GetFirstFilename()))
								{
									continue;
								}
							}
#endif	  // WITH_EDITOR

							(DrawModeComponent->*TextureSetter)(Texture);
						}
					}
				}
			}
		};
		HandleCardFace(XPosAttr, &UUsdDrawModeComponent::SetCardTextureXPos, EUsdModelCardFace::XPos);
		HandleCardFace(YPosAttr, &UUsdDrawModeComponent::SetCardTextureYPos, EUsdModelCardFace::YPos);
		HandleCardFace(ZPosAttr, &UUsdDrawModeComponent::SetCardTextureZPos, EUsdModelCardFace::ZPos);
		HandleCardFace(XNegAttr, &UUsdDrawModeComponent::SetCardTextureXNeg, EUsdModelCardFace::XNeg);
		HandleCardFace(YNegAttr, &UUsdDrawModeComponent::SetCardTextureYNeg, EUsdModelCardFace::YNeg);
		HandleCardFace(ZNegAttr, &UUsdDrawModeComponent::SetCardTextureZNeg, EUsdModelCardFace::ZNeg);

		// Override the AuthoredFaces with the correct value. The texture setter functions will all set the authored faces
		// when we set any texture in the component, but we also want to set as authored the faces where there *was* some
		// texture authored in USD but we failed to resolve it, so that we can show that face with vertex color like USD specifies
		DrawModeComponent->SetAuthoredFaces(AuthoredFaces);
	}
}	 // namespace UE::UsdXformableTranslatorImpl::Private

USceneComponent* FUsdGeomXformableTranslator::CreateAlternativeDrawModeComponents(EUsdDrawMode DrawMode)
{
	// If we're in here, our prim is a model, and we always need actors for model prims anyway
	const bool bNeedsActor = true;

	switch (DrawMode)
	{
		case EUsdDrawMode::Origin:
		case EUsdDrawMode::Bounds:
		{
			return CreateComponentsEx({UUsdDrawModeComponent::StaticClass()}, bNeedsActor);
		}
		case EUsdDrawMode::Cards:
		{
			UUsdDrawModeComponent* Component = Cast<UUsdDrawModeComponent>(CreateComponentsEx({UUsdDrawModeComponent::StaticClass()}, bNeedsActor));
			if (ensure(Component) && Context->UsdAssetCache && Context->PrimLinkCache)
			{
				// For now we only assign textures when creating components, not when updating. Maybe in the future we can
				// add support for "texture animations"
				UE::UsdXformableTranslatorImpl::Private::AssignDrawModeComponentTextures(
					GetPrim(),
					Component,
					*Context->UsdAssetCache,
					*Context->PrimLinkCache
				);
			}
			return Component;
		}
		case EUsdDrawMode::Default:
		case EUsdDrawMode::Inherited:
		{
			ensure(false);
			break;
		}
	}

	return nullptr;
}

void FUsdGeomXformableTranslator::CreateAlternativeDrawModeAssets(EUsdDrawMode DrawMode)
{
	// Currently we just use this function to create the textures that we're going to use on the bounds components,
	// if applicable
	if (DrawMode != EUsdDrawMode::Cards || !Context->UsdAssetCache || !Context->PrimLinkCache)
	{
		return;
	}

	FScopedUsdAllocs Allocs;

	pxr::UsdPrim Prim = GetPrim();
	pxr::UsdGeomModelAPI GeomModelAPI{Prim};
	if (!GeomModelAPI)
	{
		return;
	}

	TFunction<void(const pxr::UsdAttribute&)> HandleCardFace = [this](const pxr::UsdAttribute& Attr)
	{
		if (Attr && Attr.HasAuthoredValue())
		{
			const FString ResolvedPath = UsdUtils::GetResolvedAssetPath(Attr);
			if (ResolvedPath.IsEmpty())
			{
				pxr::SdfAssetPath TextureAssetPath;
				Attr.Get(&TextureAssetPath, Context->Time);
				FString TargetAssetPath = UsdToUnreal::ConvertString(TextureAssetPath.GetAssetPath());

				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"CardTextureFailResolve",
						"Failed to resolve texture path '{0}' specified as a card geometry texture on prim '{1}' at time '{2}'"
					),
					FText::FromString(TargetAssetPath),
					FText::FromString(UsdToUnreal::ConvertPath(Attr.GetPrim().GetPrimPath())),
					Context->Time
				));
			}
			else
			{
				const FString HashPrefix = UsdUtils::GetAssetHashPrefix(GetPrim(), Context->bShareAssetsForIdenticalPrims);
				const FString PrefixedTextureHash = HashPrefix + LexToString(FMD5Hash::HashFile(*ResolvedPath));

				const FString& DesiredTextureName = FPaths::GetBaseFilename(ResolvedPath);

				const EObjectFlags DesiredFlags = Context->ObjectFlags;

				bool bCreatedTexture = false;
				UTexture2D* Texture = Context->UsdAssetCache.Get()->GetOrCreateCustomCachedAsset<UTexture2D>(
					PrefixedTextureHash,
					DesiredTextureName,
					DesiredFlags,
					[&ResolvedPath](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
					{
						const TextureGroup Group = TextureGroup::TEXTUREGROUP_World;
						return UsdUtils::CreateTexture(ResolvedPath, SanitizedName, Group, FlagsToUse, Outer);
					},
					&bCreatedTexture
				);

				// We link the textures to the prim, so that if the prim is reloaded the AUsdStageActor knows to potentially
				// drop the textures. However we put the full attribute path on AssetUserData, so that when we're filling in
				// our UUsdDrawModeComponent later, we know which texture came from which attribute
				Context->PrimLinkCache->LinkAssetToPrim(PrimPath, Texture);
				if (UUsdAssetUserData* TextureUserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Texture))
				{
					TextureUserData->PrimPaths.AddUnique(UsdToUnreal::ConvertPath(Attr.GetPath()));
				}
			}
		}
	};
	HandleCardFace(GeomModelAPI.GetModelCardTextureXPosAttr());
	HandleCardFace(GeomModelAPI.GetModelCardTextureYPosAttr());
	HandleCardFace(GeomModelAPI.GetModelCardTextureZPosAttr());
	HandleCardFace(GeomModelAPI.GetModelCardTextureXNegAttr());
	HandleCardFace(GeomModelAPI.GetModelCardTextureYNegAttr());
	HandleCardFace(GeomModelAPI.GetModelCardTextureZNegAttr());
}

bool FUsdGeomXformableTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeomXformableTranslator::CollapsesChildren);

	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->DoesPathCollapseChildren(PrimPath, CollapsingType);
	}

	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	bool bCollapsesChildren = false;

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim Prim = GetPrim();
	pxr::UsdModelAPI Model{pxr::UsdTyped(Prim)};

	// Now that we use UsdSkelSkeletonTranslator the SkelRoots will be handled by the FUsdGeomXformableTranslator (here).
	// SkelRoots are likely going to end up with SkeletalMeshes though, so we can assume we won't be collapsing them just from that
	if (Prim.IsA<pxr::UsdSkelRoot>())
	{
		return false;
	}

	if (Model)
	{
		// If we're not using prim kinds to collapse, we don't want anything to *try collapsing its children* by default
		const bool bCanCollapseByDefault = false;
		const bool bUseSchemaForCollapsing = true;
		bCollapsesChildren = UsdUtils::PrimCollapses(
			Prim,
			Context->KindsToCollapse,
			Context->bUsePrimKindsForCollapsing,
			bUseSchemaForCollapsing,
			bCanCollapseByDefault
		);
		if (bCollapsesChildren)
		{
			// This indicates whether the subtree *can* be collapsed
			TOptional<bool> bSubtreeCanBeCollapsed = Context->UsdInfoCache->CanXformableSubtreeBeCollapsed(PrimPath, *Context);
			if (bSubtreeCanBeCollapsed.IsSet())
			{
				return bSubtreeCanBeCollapsed.GetValue();
			}
		}
	}

	return bCollapsesChildren;
}

bool FUsdGeomXformableTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim UsdPrim{GetPrim()};
	if (!UsdPrim)
	{
		return false;
	}

	if (UsdUtils::IsAnimated(UsdPrim) || 
		UsdUtils::PrimHasSchema(UsdPrim, UnrealIdentifiers::LiveLinkAPI) || 
		UsdPrim.IsA<pxr::UsdSkelRoot>() || 
		(Context->bAllowInterpretingLODs && UsdUtils::DoesPrimContainMeshLODs(UsdPrim)) || 
		UsdUtils::GetAppliedDrawMode(UsdPrim) != EUsdDrawMode::Default
	)
	{
		return false;
	}

	// If we're not using prim kinds to collapse, we still want things to *be collapsible* by default (so that they can be controlled with the
	// collapsing attributes)
	const bool bCanCollapseByDefault = true;
	const bool bUseSchemaForCollapsing = true;
	return UsdUtils::PrimCollapses(
		UsdPrim,
		Context->KindsToCollapse,
		Context->bUsePrimKindsForCollapsing,
		bUseSchemaForCollapsing,
		bCanCollapseByDefault
	);
}

TSet<UE::FSdfPath> FUsdGeomXformableTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	if (!Context->UsdInfoCache->DoesPathCollapseChildren(PrimPath, ECollapsingType::Assets))
	{
		return {};
	}

	TSet<UE::FSdfPath> Result;
	{
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim Prim = GetPrim();

		// We check imageable because that is the most basal schema that is still relevant for collapsed meshes (it
		// holds the visibility attribute)
		TArray<TUsdStore<pxr::UsdPrim>> ChildPrims = UsdUtils::GetAllPrimsOfType(Prim, pxr::TfType::Find<pxr::UsdGeomImageable>());

		Result.Reserve(ChildPrims.Num());
		for (const TUsdStore<pxr::UsdPrim>& ChildPrim : ChildPrims)
		{
			Result.Add(UE::FSdfPath{ChildPrim.Get().GetPrimPath()});
		}
	}
	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
