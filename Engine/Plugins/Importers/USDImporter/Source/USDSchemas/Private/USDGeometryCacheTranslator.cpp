// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGeometryCacheTranslator.h"

#if USE_USD_SDK && WITH_EDITOR
// The GeometryCacheStreamer module is editor-only, so is the translator

#include "MeshTranslationImpl.h"
#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGroomTranslatorUtils.h"
#include "USDIntegrationUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Async/ParallelFor.h"
#include "FeedbackContextEditor.h"
#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheComponent.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheTrackUSD.h"
#include "GeometryCacheUSDComponent.h"
#include "GroomComponent.h"
#include "HAL/IConsoleManager.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Misc/ScopedSlowTask.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UObject/Package.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/typed.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdGeom/subset.h"
#include "pxr/usd/usdShade/materialBindingAPI.h"
#include "USDIncludesEnd.h"

static int32 GUsdGeometryCacheParallelMeshReads = 16;
static FAutoConsoleVariableRef CVarUsdGeometryCacheParallelMeshReads(
	TEXT("USD.GeometryCache.Import.ParallelMeshReads"),
	GUsdGeometryCacheParallelMeshReads,
	TEXT("Maximum number of mesh to process in parallel")
);

static int32 GUsdGeometryCacheParallelFrameReads = 16;
static FAutoConsoleVariableRef CVarUsdGeometryCacheParallelFrameReads(
	TEXT("USD.GeometryCache.Import.ParallelFrameReads"),
	GUsdGeometryCacheParallelFrameReads,
	TEXT("Maximum number of mesh frames to read in parallel")
);

static bool GEnableSubdiv = false;
static FAutoConsoleVariableRef CVarEnableSubdiv(
	TEXT("USD.GeometryCache.EnableSubdiv"),
	GEnableSubdiv,
	TEXT("Whether to subdivide Mesh prim data when parsing GeometryCaches via OpenSubdiv, the same way we subdivide the Mesh data that ends up in "
		 "StaticMeshes")
);

static bool GStoreImportedVertexNumbers = false;
static FAutoConsoleVariableRef CVarStoreImportedVertexNumbers(
	TEXT("USD.GeometryCache.Import.StoreImportedVertexNumbers"),
	GStoreImportedVertexNumbers,
	TEXT("Whether to store the imported vertex numbers. This lets you know the vertex numbers inside the DCC.")
);

namespace UsdGeometryCacheTranslatorImpl
{
	bool ProcessGeometryCacheMaterials(
		const pxr::UsdPrim& UsdPrim,
		const TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& LODIndexToMaterialInfo,
		UGeometryCache& GeometryCache,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache,
		float Time,
		EObjectFlags Flags,
		bool bShareAssetsForIdenticalPrims
	)
	{
		TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolvedMaterials = MeshTranslationImpl::ResolveMaterialAssignmentInfo(
			UsdPrim,
			LODIndexToMaterialInfo,
			AssetCache,
			PrimLinkCache,
			Flags,
			bShareAssetsForIdenticalPrims
		);

		uint32 SlotIndex = 0;
		bool bMaterialAssignementsHaveChanged = false;
		for (int32 LODIndex = 0; LODIndex < LODIndexToMaterialInfo.Num(); ++LODIndex)
		{
			const TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToMaterialInfo[LODIndex].Slots;
			for (int32 LODSlotIndex = 0; LODSlotIndex < LODSlots.Num(); ++LODSlotIndex, ++SlotIndex)
			{
				const UsdUtils::FUsdPrimMaterialSlot& Slot = LODSlots[LODSlotIndex];
				UMaterialInterface* Material = UMaterial::GetDefaultMaterial(MD_Surface);
				if (UMaterialInterface** FoundMaterial = ResolvedMaterials.Find(&Slot))
				{
					Material = *FoundMaterial;
				}
				else
				{
					// Warn, but still add a material slot to preserve the materials order
					USD_LOG_WARNING(
						TEXT("Failed to resolve material '%s' for slot '%d' for geometry cache '%s'"),
						*Slot.MaterialSource,
						LODSlotIndex,
						*UsdToUnreal::ConvertPath(UsdPrim.GetPath())
					);
				}

				if (!GeometryCache.Materials.IsValidIndex(SlotIndex))
				{
					GeometryCache.Materials.Add(Material);
					bMaterialAssignementsHaveChanged = true;
				}
				else if (!(GeometryCache.Materials[SlotIndex] == Material))
				{
					GeometryCache.Materials[SlotIndex] = Material;
					bMaterialAssignementsHaveChanged = true;
				}
			}
		}

		return bMaterialAssignementsHaveChanged;
	}

	void LoadMeshDescription(
		pxr::UsdTyped UsdMesh,
		FMeshDescription& OutMeshDescription,
		UsdUtils::FUsdPrimMaterialAssignmentInfo& OutMaterialInfo,
		const UsdToUnreal::FUsdMeshConversionOptions& Options
	)
	{
		if (!UsdMesh)
		{
			return;
		}

		// MeshDescriptions are always allocated on the UE allocator as the allocation happens within
		// another dll, so we need to deallocate them using it too
		FScopedUnrealAllocs Allocs;

		FMeshDescription TempMeshDescription;
		UsdUtils::FUsdPrimMaterialAssignmentInfo TempMaterialInfo;

		FStaticMeshAttributes StaticMeshAttributes(TempMeshDescription);
		StaticMeshAttributes.Register();

		bool bSuccess = UsdToUnreal::ConvertGeomMesh(pxr::UsdGeomMesh{UsdMesh}, TempMeshDescription, TempMaterialInfo, Options);
		if (bSuccess)
		{
			OutMeshDescription = MoveTemp(TempMeshDescription);
			OutMaterialInfo = MoveTemp(TempMaterialInfo);
		}
	}

	struct FReadMeshDataArgs
	{
		FReadMeshDataArgs(const UE::FUsdStage& InStage, const UE::FUsdPrim& InRootPrim)
			: Stage(InStage)
			, RootPrim(InRootPrim)
		{
		}

		UE::FUsdStageWeak Stage;
		UE::FUsdPrim RootPrim;
		UsdToUnreal::FUsdMeshConversionOptions Options;
		int32 StartFrame = 0;
		int32 EndFrame = 0;
		float FramesPerSecond = 24.0f;
		bool bPropagateTransform = false;
	};

	FReadMeshDataArgs GetReadMeshDataArgs(TSharedRef<FUsdSchemaTranslationContext> Context, const FString& RootPrimPath)
	{
		const UE::FUsdStage& Stage = Context->Stage;
		UE::FUsdPrim RootPrim = Stage.GetPrimAtPath(UE::FSdfPath(*RootPrimPath));

		FReadMeshDataArgs Args(Stage, RootPrim);

		// Fetch the animated mesh start/end frame as they may be different from just the stage's start and end time codes
		int32 StartFrame = FMath::FloorToInt(Stage.GetStartTimeCode());
		int32 EndFrame = FMath::CeilToInt(Stage.GetEndTimeCode());
		UsdUtils::GetAnimatedMeshTimeCodes(Stage, RootPrimPath, StartFrame, EndFrame);

		double FramesPerSecond = Stage.GetTimeCodesPerSecond();
		if (FramesPerSecond == 0)
		{
			ensureMsgf(false, TEXT("Invalid USD GeometryCache FPS detected. Falling back to 1 FPS"));
			FramesPerSecond = 1;
		}

		// The GeometryCache module expects the end frame to be one past the last animation frame
		EndFrame += 1;

		Args.StartFrame = StartFrame;
		Args.EndFrame = EndFrame;
		Args.FramesPerSecond = FramesPerSecond;

		pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
		if (!Context->RenderContext.IsNone())
		{
			RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
		}

		pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		if (!Context->MaterialPurpose.IsNone())
		{
			MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
		}

		Args.Options.PurposesToLoad = Context->PurposesToLoad;
		Args.Options.RenderContext = RenderContextToken;
		Args.Options.MaterialPurpose = MaterialPurposeToken;
		Args.Options.bMergeIdenticalMaterialSlots = false;	  // Don't merge because the GeometryCache is processed as unflattened (ie. one track per
															  // mesh)
		Args.Options.SubdivisionLevel = GEnableSubdiv ? Context->SubdivisionLevel : 0;

		return Args;
	}

	bool ReadMeshData(
		const FReadMeshDataArgs& Args,
		const UE::FUsdPrim& MeshPrim,
		int32 MaterialOffset,
		float Time,
		FGeometryCacheMeshData& OutMeshData
	)
	{
		// MeshDescriptions are always allocated on the UE allocator as the allocation happens within
		// another dll, so we need to deallocate them using it too
		FScopedUnrealAllocs Allocs;

		FTransform PropatagedTransform = FTransform::Identity;
		if (Args.bPropagateTransform)
		{
			UsdToUnreal::PropagateTransform(Args.Stage, Args.RootPrim, MeshPrim, Time, PropatagedTransform);
		}

		// Need a local copy of Options to set the TimeCode since this function is called from multiple worker threads
		UsdToUnreal::FUsdMeshConversionOptions LocalOptions(Args.Options);
		LocalOptions.TimeCode = pxr::UsdTimeCode(Time);
		LocalOptions.AdditionalTransform = PropatagedTransform;

		FMeshDescription MeshDescription;
		UsdUtils::FUsdPrimMaterialAssignmentInfo MaterialInfo;
		UsdGeometryCacheTranslatorImpl::LoadMeshDescription(pxr::UsdTyped(MeshPrim), MeshDescription, MaterialInfo, LocalOptions);

		// Convert the MeshDescription to MeshData
		if (!MeshDescription.IsEmpty())
		{
			// Compute the normals and tangents for the mesh
			const float ComparisonThreshold = THRESH_POINTS_ARE_SAME;

			// This function make sure the Polygon Normals Tangents Binormals are computed and also remove degenerated triangle from the render mesh
			// description.
			FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription, ComparisonThreshold);

			// Compute any missing normals or tangents.
			// Static meshes always blend normals of overlapping corners.
			EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
			ComputeNTBsOptions |= EComputeNTBsFlags::IgnoreDegenerateTriangles;
			ComputeNTBsOptions |= EComputeNTBsFlags::UseMikkTSpace;

			FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, ComputeNTBsOptions);

			UE::GeometryCache::Utils::FMeshDataConversionArguments ConversionArgs;
			ConversionArgs.MaterialOffset = MaterialOffset;
			ConversionArgs.FramesPerSecond = Args.FramesPerSecond;
			ConversionArgs.bUseVelocitiesAsMotionVectors = true;
			ConversionArgs.bStoreImportedVertexNumbers = GStoreImportedVertexNumbers;
			UE::GeometryCache::Utils::GetGeometryCacheMeshDataFromMeshDescription(OutMeshData, MeshDescription, ConversionArgs);

			return true;
		}
		return false;
	}

	UGeometryCacheTrackUsd* CreateUsdStreamTrack(
		UGeometryCache* GeometryCache,
		const FReadMeshDataArgs& Args,
		const FString& PrimPath,
		int32 MaterialOffset
	)
	{
		// Create and configure a new USDTrack to be added to the GeometryCache
		UGeometryCacheTrackUsd* UsdTrack = NewObject<UGeometryCacheTrackUsd>(GeometryCache);
		UsdTrack->MeshConversionOptions = Args.Options;	   // Also pass along the options we'll use for mesh conversion so that we can properly hash
														   // the prim
		UsdTrack->Initialize(
			Args.Stage,
			PrimPath,
			Args.StartFrame,
			Args.EndFrame,
			[Args, MaterialOffset](const TWeakObjectPtr<UGeometryCacheTrackUsd> TrackPtr, float Time, FGeometryCacheMeshData& OutMeshData) mutable
			{
				UGeometryCacheTrackUsd* Track = TrackPtr.Get();
				if (!Track)
				{
					return false;
				}

				if (!Track->CurrentStagePinned)
				{
					return false;
				}

				UE::FUsdPrim Prim = Track->CurrentStagePinned.GetPrimAtPath(UE::FSdfPath(*Track->PrimPath));
				if (!Prim)
				{
					return false;
				}

				return ReadMeshData(Args, Prim, MaterialOffset, Time, OutMeshData);
			}
		);
		return UsdTrack;
	}

	UGeometryCacheTrackStreamable* CreateStreamableTrack(UGeometryCache* GeometryCache, const FString& PrimPath)
	{
		// Create and configure a new StreamableTrack to be added to the GeometryCache
		FString ObjectName = UsdUnreal::ObjectUtils::SanitizeObjectName(FPaths::GetBaseFilename(PrimPath));

		FName CodecName = MakeUniqueObjectName(GeometryCache, UGeometryCacheCodecV1::StaticClass(), FName(ObjectName + FString(TEXT("_Codec"))));
		UGeometryCacheCodecV1* Codec = NewObject<UGeometryCacheCodecV1>(GeometryCache, CodecName, RF_Public);

		// Compression settings for good quality
		const float VertexQuantizationPrecision = 0.0005f;
		const int32 UVBits = 16;
		Codec->InitializeEncoder(VertexQuantizationPrecision, UVBits);

		FName TrackName = MakeUniqueObjectName(GeometryCache, UGeometryCacheTrackStreamable::StaticClass(), FName(ObjectName));
		UGeometryCacheTrackStreamable* StreamableTrack = NewObject<UGeometryCacheTrackStreamable>(GeometryCache, TrackName, RF_Public);

		const bool bForceSingleOptimization = false;
		const bool bCalculateMotionVectors = false;
		const bool bOptimizeIndexBuffers = false;
		StreamableTrack->BeginCoding(Codec, bForceSingleOptimization, bCalculateMotionVectors, bOptimizeIndexBuffers);
		// EndCoding has to be called from the main thread once all the frame data have been added to the track

		return StreamableTrack;
	}

	void FillGeometryCacheTracks(
		const FString& RootPrimPath,
		const TArray<UE::FSdfPath>& MeshPrims,
		const TArray<int32>& MaterialOffsets,
		TSharedRef<FUsdSchemaTranslationContext> Context,
		UGeometryCache* GeometryCache
	);

	void FinalizeGeometryCache(UGeometryCache* GeometryCache);

	UGeometryCache* CreateGeometryCache(
		const UE::FUsdPrim& RootPrim,
		const FMeshDescription& MeshDescription,
		const TArray<UE::FSdfPath>& MeshPaths,
		const TArray<int32>& MaterialOffsets,
		TSharedRef<FUsdSchemaTranslationContext> Context,
		bool& bOutIsNew,
		float& StartOffsetTime
	)
	{
		FString RootPrimPath = RootPrim.GetPrimPath().GetString();
		FReadMeshDataArgs Args(GetReadMeshDataArgs(Context, RootPrimPath));

		StartOffsetTime = static_cast<float>(Args.StartFrame) / Args.FramesPerSecond;

		// Compute the asset hash from the merged mesh description
		FSHA1 SHA1;
		FSHAHash MeshHash = FStaticMeshOperations::ComputeSHAHash(MeshDescription);
		SHA1.Update(&MeshHash.Hash[0], sizeof(MeshHash.Hash));

		const bool bIsImporting = Context->bIsImporting || Context->GeometryCacheImport == EGeometryCacheImport::OnLoad;

		// Frame rate must be taken into account as well since different frame rates must produce different sampling in the tracks
		SHA1.Update(reinterpret_cast<uint8*>(&Args.FramesPerSecond), sizeof(Args.FramesPerSecond));
		SHA1.Update(reinterpret_cast<uint8*>(&Args.StartFrame), sizeof(Args.StartFrame));
		SHA1.Update(reinterpret_cast<uint8*>(&Args.EndFrame), sizeof(Args.EndFrame));

		// Track type depends on how geometry caches are handled. Import needs to generate a persistent asset with all the frames already sampled
		SHA1.Update(reinterpret_cast<const uint8*>(&Context->GeometryCacheImport), sizeof(Context->GeometryCacheImport));
		SHA1.Final();

		FSHAHash GeoCacheHash;
		SHA1.GetHash(&GeoCacheHash.Hash[0]);
		const FString PrefixedGeoCacheHash = UsdUtils::GetAssetHashPrefix(RootPrim, Context->bShareAssetsForIdenticalPrims) + GeoCacheHash.ToString();

		const FString DesiredName = FPaths::GetBaseFilename(RootPrimPath);

		// In Never import mode, make the geometry cache transient so it doesn't get saved to disk. It will get recreated since it's lightweight.
		EObjectFlags ObjectFlags = Context->ObjectFlags;
		if (Context->GeometryCacheImport == EGeometryCacheImport::Never)
		{
			ObjectFlags |= RF_Transient;
		}

		UGeometryCache* GeometryCache = Context->UsdAssetCache
											->GetOrCreateCachedAsset<UGeometryCache>(PrefixedGeoCacheHash, DesiredName, ObjectFlags, &bOutIsNew);

		if (GeometryCache && bOutIsNew)
		{
			if (Context->GeometryCacheImport == EGeometryCacheImport::OnSave)
			{
				// In OnSave import mode, register a PreSave callback to convert the USD tracks to streamable tracks
				GeometryCache->OnPreSave = UGeometryCache::FOnPreSave::CreateLambda(
					[MeshPaths, RootPrimPath, MaterialOffsets, Context](UGeometryCache* GeometryCache)
					{
						// Convert only if there's any USD tracks
						bool bHasUsdTracks = false;
						for (UGeometryCacheTrack* Track : GeometryCache->Tracks)
						{
							if (UGeometryCacheTrackUsd* UsdTrack = Cast<UGeometryCacheTrackUsd>(Track))
							{
								bHasUsdTracks = true;
								// Make sure to unregister the USD track from the streamer since it will get replaced with a streamable track
								UsdTrack->UnregisterStream();
							}
						}

						if (!bHasUsdTracks)
						{
							return;
						}

						GeometryCache->Tracks.Reset();

						// Create a track for each mesh to be processed and add it to the GeometryCache
						for (int32 Index = 0; Index < MeshPaths.Num(); ++Index)
						{
							const FString& PrimPath = MeshPaths[Index].GetString();
							UGeometryCacheTrack* Track = CreateStreamableTrack(GeometryCache, PrimPath);
							GeometryCache->AddTrack(Track);

							TArray<FMatrix> Mats;
							Mats.Add(FMatrix::Identity);
							Mats.Add(FMatrix::Identity);

							TArray<float> MatTimes;
							MatTimes.Add(0.0f);
							MatTimes.Add(0.0f);
							Track->SetMatrixSamples(Mats, MatTimes);
						}

						FillGeometryCacheTracks(RootPrimPath, MeshPaths, MaterialOffsets, Context, GeometryCache);

						FinalizeGeometryCache(GeometryCache);
					}
				);
			}

			GeometryCache->SetFrameStartEnd(Args.StartFrame, Args.EndFrame - 1);

			// Create a track for each mesh to be processed and add it to the GeometryCache
			for (int32 Index = 0; Index < MeshPaths.Num(); ++Index)
			{
				const FString& PrimPath = MeshPaths[Index].GetString();
				UGeometryCacheTrack* Track = nullptr;
				if (!bIsImporting)
				{
					Track = CreateUsdStreamTrack(GeometryCache, Args, PrimPath, MaterialOffsets[Index]);
				}
				else
				{
					Track = CreateStreamableTrack(GeometryCache, PrimPath);
				}
				GeometryCache->AddTrack(Track);

				TArray<FMatrix> Mats;
				Mats.Add(FMatrix::Identity);
				Mats.Add(FMatrix::Identity);

				TArray<float> MatTimes;
				MatTimes.Add(0.0f);
				MatTimes.Add(0.0f);
				Track->SetMatrixSamples(Mats, MatTimes);
			}
		}

		return GeometryCache;
	}

	void FillGeometryCacheTracks(
		const FString& RootPrimPath,
		const TArray<UE::FSdfPath>& MeshPrims,
		const TArray<int32>& MaterialOffsets,
		TSharedRef<FUsdSchemaTranslationContext> Context,
		UGeometryCache* GeometryCache
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdGeometryCacheTranslatorImpl::FillGeometryCacheTracks);

		FReadMeshDataArgs Args = GetReadMeshDataArgs(Context, RootPrimPath);
		Args.bPropagateTransform = true;

		// Use new feedback context to override the one coming from the USDImporter to be able to show the ScopedSlowTask below
		FFeedbackContextEditor GeoCacheContext;
		const FString AssetName = GeometryCache->GetName();
		const int32 NumMeshes = MeshPrims.Num();
		const int32 NumFrames = (Args.EndFrame - Args.StartFrame) * NumMeshes;
		int32 TotalFrames = 0;

		FString Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), {AssetName, 0, NumFrames});
		FScopedSlowTask SlowTask(static_cast<float>(NumFrames), FText::FromString(Title), true, GeoCacheContext);
		SlowTask.MakeDialog(true);

		const bool bSingleThreaded = !FApp::ShouldUseThreadingForPerformance();
		if (bSingleThreaded)
		{
			for (int32 Index = 0; Index < MeshPrims.Num(); ++Index)
			{
				UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(GeometryCache->Tracks[Index]);
				UE::FUsdPrim MeshPrim = Args.Stage.GetPrimAtPath(MeshPrims[Index]);
				const bool bConstantTopology = UsdUtils::GetMeshTopologyVariance(pxr::UsdGeomMesh(MeshPrim))
											   != UsdUtils::EMeshTopologyVariance::Heterogenous;

				for (int32 FrameIndex = Args.StartFrame; FrameIndex < Args.EndFrame; ++FrameIndex)
				{
					if (SlowTask.ShouldCancel())
					{
						return;
					}

					// Read frame data
					FGeometryCacheMeshData MeshData;
					ReadMeshData(Args, MeshPrim, MaterialOffsets[Index], FrameIndex, MeshData);

					// Add it to the track
					StreamableTrack->AddMeshSample(MeshData, (FrameIndex - Args.StartFrame) / Args.FramesPerSecond, bConstantTopology);

					++TotalFrames;
					Title = FString::Format(TEXT("Importing frames for {0} ({1}/{2})"), {AssetName, TotalFrames, NumFrames});
					SlowTask.EnterProgressFrame(1, FText::FromString(Title));
				}
			}
		}
		else
		{
			// Balance the number of threads for mesh reads vs frame reads
			const int32 MaxWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
			int32 NumMeshThreads = FMath::Clamp(MaxWorkerThreads, 1, FMath::Min(NumMeshes, GUsdGeometryCacheParallelMeshReads));
			int32 NumFrameThreads = FMath::Clamp(MaxWorkerThreads, 1, GUsdGeometryCacheParallelFrameReads);

			int32 NumLoops = 0;
			while (NumMeshThreads * NumFrameThreads > MaxWorkerThreads)
			{
				if (NumLoops % 2 == 0)
				{
					NumFrameThreads = FMath::Max(FMath::RoundToInt(static_cast<float>(NumFrameThreads) * 0.8f), 1);
				}
				else
				{
					NumMeshThreads = FMath::Max(--NumMeshThreads, 1);
				}
				++NumLoops;
			};

			TArray<FEvent*> SyncEvents;
			SyncEvents.SetNum(NumMeshThreads);
			for (int32 Index = 0; Index < NumMeshThreads; ++Index)
			{
				SyncEvents[Index] = FPlatformProcess::GetSynchEventFromPool();
			}

			std::atomic<float> CompletedFrames = 0.0f;

			// Parallel mesh reads: Meshes can be read independently of each other
			bool bCancelled = false;
			ParallelFor(
				NumMeshThreads,
				[&Args,
				 &MeshPrims,
				 &MaterialOffsets,
				 &SyncEvents,
				 NumMeshThreads,
				 NumFrameThreads,
				 GeometryCache,
				 NumMeshes,
				 &SlowTask,
				 &AssetName,
				 &CompletedFrames,
				 &NumFrames,
				 &TotalFrames,
				 &bCancelled](int32 MeshThreadIndex)
				{
					int32 MeshIndex = MeshThreadIndex;

					while (!bCancelled && (MeshIndex < NumMeshes))
					{
						UE::FUsdPrim MeshPrim = Args.Stage.GetPrimAtPath(MeshPrims[MeshIndex]);
						int32 MaterialOffset = MaterialOffsets[MeshIndex];
						UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(GeometryCache->Tracks[MeshIndex]);
						FEvent* FrameWrittenEvent = SyncEvents[MeshThreadIndex];
						const bool bConstantTopology = UsdUtils::GetMeshTopologyVariance(pxr::UsdGeomMesh(MeshPrim))
													   != UsdUtils::EMeshTopologyVariance::Heterogenous;

						std::atomic<int32> WriteFrameIndex = Args.StartFrame;
						FCriticalSection Mutex;

						// Parallel frame read: frame data can be read concurrently but have to be processed in order for AddMeshSample
						ParallelFor(
							NumFrameThreads,
							[&Args,
							 &MeshPrim,
							 &FrameWrittenEvent,
							 &WriteFrameIndex,
							 &Mutex,
							 NumFrameThreads,
							 MaterialOffset,
							 StreamableTrack,
							 bConstantTopology,
							 &SlowTask,
							 &AssetName,
							 &CompletedFrames,
							 &NumFrames,
							 &TotalFrames,
							 &bCancelled](int32 FrameThreadIndex)
							{
								int32 FrameIndex = Args.StartFrame + FrameThreadIndex;

								while (!bCancelled && (FrameIndex < Args.EndFrame))
								{
									if (IsInGameThread() && SlowTask.ShouldCancel())
									{
										bCancelled = true;
										break;
									}

									// Read frame data into memory
									FGeometryCacheMeshData MeshData;
									ReadMeshData(Args, MeshPrim, MaterialOffset, FrameIndex, MeshData);

									// Wait until it's our turn to process this frame.
									while (WriteFrameIndex < FrameIndex)
									{
										if (bCancelled)
										{
											break;
										}
										const uint32 WaitTimeInMs = 10;
										FrameWrittenEvent->Wait(WaitTimeInMs);
									}

									{
										FScopeLock WriteLock(&Mutex);

										// Add it to the track
										StreamableTrack
											->AddMeshSample(MeshData, (FrameIndex - Args.StartFrame) / Args.FramesPerSecond, bConstantTopology);

										// Mark the next frame index as ready for processing.
										++WriteFrameIndex;

										CompletedFrames = CompletedFrames + 1;

										if (IsInGameThread())
										{
											float TempCompletedFrames = CompletedFrames;
											CompletedFrames = 0.0f;
											TotalFrames += int32(TempCompletedFrames);
											FString Title = FString::Format(
												TEXT("Importing frames for {0} ({1}/{2})"),
												{AssetName, TotalFrames, NumFrames}
											);
											SlowTask.EnterProgressFrame(TempCompletedFrames, FText::FromString(Title));
										}

										FrameWrittenEvent->Trigger();
									}

									// Get new frame index to read for next run cycle
									FrameIndex += NumFrameThreads;
								};
							}
						);

						MeshIndex += NumMeshThreads;
					}
				}
			);

			for (FEvent* SyncEvent : SyncEvents)
			{
				FPlatformProcess::ReturnSynchEventToPool(SyncEvent);
			}
		}
	}

	void FinalizeGeometryCache(UGeometryCache* GeometryCache)
	{
		for (int32 Index = GeometryCache->Tracks.Num() - 1; Index >= 0; --Index)
		{
			if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(GeometryCache->Tracks[Index]))
			{
				// Remove track that have no data (either import cancelled or there was no mesh data imported)
				if (!StreamableTrack->EndCoding())
				{
					GeometryCache->Tracks.RemoveAt(Index);
				}
			}
		}
	}
}	 // namespace UsdGeometryCacheTranslatorImpl

class FGeometryCacheCreateAssetsTaskChain : public FBuildStaticMeshTaskChain
{
public:
	explicit FGeometryCacheCreateAssetsTaskChain(const TSharedRef<FUsdSchemaTranslationContext>& InContext, const UE::FSdfPath& InPrimPath)
		: FBuildStaticMeshTaskChain(InContext, InPrimPath)
	{
		SetupTasks();
	}

protected:
	virtual void SetupTasks() override;
	TArray<UE::FSdfPath> MeshPrimPaths;
	TArray<int32> MaterialOffsets;
	TStrongObjectPtr<UGeometryCache> GeometryCache;
};

void FGeometryCacheCreateAssetsTaskChain::SetupTasks()
{
	// Create the mesh description (Async)
	Do(ESchemaTranslationLaunchPolicy::Async,
	   [this]() -> bool
	   {
		   FScopedUnrealAllocs UnrealAllocs;

		   pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
		   if (!Context->RenderContext.IsNone())
		   {
			   RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
		   }

		   pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		   if (!Context->MaterialPurpose.IsNone())
		   {
			   MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
		   }

		   UsdToUnreal::FUsdMeshConversionOptions Options;
		   Options.TimeCode = UsdUtils::GetEarliestTimeCode();
		   Options.PurposesToLoad = Context->PurposesToLoad;
		   Options.RenderContext = RenderContextToken;
		   Options.MaterialPurpose = MaterialPurposeToken;
		   Options.bMergeIdenticalMaterialSlots = false;	// Don't merge because the GeometryCache is processed as unflattened (ie. one track per
															// mesh)
		   Options.SubdivisionLevel = GEnableSubdiv ? Context->SubdivisionLevel : 0;

		   // GeometryCache has only one LOD so add just one MeshDescription and MaterialAssignmentInfo
		   FMeshDescription& AddedMeshDescription = LODIndexToMeshDescription.Emplace_GetRef();
		   UsdUtils::FUsdPrimMaterialAssignmentInfo& AssignmentInfo = LODIndexToMaterialInfo.Emplace_GetRef();

		   // The collapsed mesh description here will be used to cache the GeometryCache asset, but not to fill it since its content will be
		   // unflattened Bake the prim's transform into the mesh data
		   bool bSkipRootPrimTransform = false;
		   bool bSkipRootPrimVisiblity = false;
		   UsdToUnreal::ConvertGeomMeshSubtree(
			   GetPrim(),
			   AddedMeshDescription,
			   AssignmentInfo,
			   Options,
			   bSkipRootPrimTransform,
			   bSkipRootPrimVisiblity
		   );

		   // If we have at least one valid LOD, we should proceed to the next step
		   for (const FMeshDescription& MeshDescription : LODIndexToMeshDescription)
		   {
			   if (!MeshDescription.IsEmpty())
			   {
				   return true;
			   }
		   }
		   return false;
	   });

	// Create the GeometryCache (Main thread)
	Then(
		ESchemaTranslationLaunchPolicy::Sync,
		[this]() -> bool
		{
			{
				// Collect the visible meshes, animated or not, under Prim to be processed into the GeometryCache
				FScopedUsdAllocs UsdAllocs;
				TArray<UE::FUsdPrim> ChildVisiblePrims = UsdUtils::GetVisibleChildren(GetPrim(), Context->PurposesToLoad);

				MeshPrimPaths.Reserve(ChildVisiblePrims.Num());
				for (const UE::FUsdPrim& ChildPrim : ChildVisiblePrims)
				{
					if (ChildPrim.IsA(TEXT("Mesh")))
					{
						MeshPrimPaths.Add(UE::FSdfPath(ChildPrim.GetPrimPath()));
					}
				}
			}

			if (MeshPrimPaths.Num() > 1)
			{
				FScopedUsdAllocs UsdAllocs;

				// Compute the material offsets that will be needed to generate the GeometryCacheMeshData
				// Each mesh will have its own material slots, but they are all appended into one GeometryCache
				// so the offsets are just the number of material slots for each mesh added together in order
				// of traversal of the meshes
				MaterialOffsets.SetNum(MeshPrimPaths.Num());
				int32 MaterialOffset = 0;
				for (int32 Index = 0; Index < MeshPrimPaths.Num(); ++Index)
				{
					MaterialOffsets[Index] = MaterialOffset;

					// A mesh has at least one material associated with it, but can have multiple material assignments based on its GeomSubsets
					UE::FUsdPrim Prim = Context->Stage.GetPrimAtPath(MeshPrimPaths[Index]);

					std::vector<pxr::UsdGeomSubset> GeomSubsets = pxr::UsdShadeMaterialBindingAPI(Prim).GetMaterialBindSubsets();
					MaterialOffset += FMath::Max(1, static_cast<int32>(GeomSubsets.size()));
				}
			}
			else
			{
				// Only one mesh, so there's no offset
				MaterialOffsets.Add(0);
			}

			bool bIsNew = true;
			float StartTimeOffset = 0.0f;
			GeometryCache.Reset(UsdGeometryCacheTranslatorImpl::CreateGeometryCache(
				GetPrim(),
				LODIndexToMeshDescription[0],
				MeshPrimPaths,
				MaterialOffsets,
				Context,
				bIsNew,
				StartTimeOffset
			));

			if (GeometryCache)
			{
				UUsdGeometryCacheAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdGeometryCacheAssetUserData>(
					GeometryCache.Get()
				);
				if (UserData)
				{
					// TODO: We're overwriting some data even if we reuse the asset from the asset cache,
					// but then again we do that for the other asset types too...
					UserData->PrimvarToUVIndex = LODIndexToMaterialInfo[0].PrimvarToUVIndex;	// We use the same primvar mapping for all LODs
					UserData->StageStartOffsetSeconds = StartTimeOffset;
					UserData->PrimPaths.AddUnique(PrimPath.GetString());

					if (Context->MetadataOptions.bCollectMetadata)
					{
						UsdToUnreal::ConvertMetadata(
							GetPrim(),
							UserData,
							Context->MetadataOptions.BlockedPrefixFilters,
							Context->MetadataOptions.bInvertFilters,
							Context->MetadataOptions.bCollectFromEntireSubtrees
						);
					}
					else
					{
						// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the AssetUserData
						UserData->StageIdentifierToMetadata.Remove(GetPrim().GetStage().GetRootLayer().GetIdentifier());
					}

					MeshTranslationImpl::RecordSourcePrimsForMaterialSlots(LODIndexToMaterialInfo, UserData);
				}

				if (bIsNew && Context->UsdAssetCache && Context->PrimLinkCache)
				{
					// Only the original creator of the prim at creation time gets to set the material assignments
					// directly on the geometry cache, all others prims ensure their materials via material overrides on the
					// components
					UsdGeometryCacheTranslatorImpl::ProcessGeometryCacheMaterials(
						GetPrim(),
						LODIndexToMaterialInfo,
						*GeometryCache,
						*Context->UsdAssetCache,
						*Context->PrimLinkCache,
						Context->Time,
						Context->ObjectFlags,
						Context->bShareAssetsForIdenticalPrims
					);
				}

				if (Context->PrimLinkCache)
				{
					const UE::FSdfPath& TargetPath = AlternativePrimToLinkAssetsTo.IsSet() ? AlternativePrimToLinkAssetsTo.GetValue() : PrimPath;
					Context->PrimLinkCache->LinkAssetToPrim(TargetPath, GeometryCache.Get());
				}
			}

			const bool bIsImporting = Context->bIsImporting || Context->GeometryCacheImport == EGeometryCacheImport::OnLoad;

			// Continue with the import steps
			return bIsImporting && GeometryCache && bIsNew;
		}
	);

	// Fill the GeometryCache tracks with the frame data
	// It is done Sync to avoid starvation issue because FillGeometryCacheTracks is highly parallelized based on the number of meshes and frames to
	// read Filling GeometryCaches in parallel could cause deadlocks
	Then(
		ESchemaTranslationLaunchPolicy::Sync,
		[this]() -> bool
		{
			UsdGeometryCacheTranslatorImpl::FillGeometryCacheTracks(
				PrimPath.GetString(),
				MeshPrimPaths,
				MaterialOffsets,
				Context,
				GeometryCache.Get()
			);
			return true;
		}
	);

	// Finalize the GeometryCache (Main Thread)
	Then(
		ESchemaTranslationLaunchPolicy::Sync,
		[this]() -> bool
		{
			UsdGeometryCacheTranslatorImpl::FinalizeGeometryCache(GeometryCache.Get());
			return false;
		}
	);
}

void FUsdGeometryCacheTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGeometryCacheTranslator::CreateAssets);

	if (!IsPotentialGeometryCacheRoot())
	{
		Super::CreateAssets();
		return;
	}

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	if (ShouldSkipSkinnablePrim())
	{
		return;
	}

	// Create the GeometryCache TaskChain
	TSharedRef<FGeometryCacheCreateAssetsTaskChain> AssetsTaskChain = MakeShared<FGeometryCacheCreateAssetsTaskChain>(Context, PrimPath);

	Context->TranslatorTasks.Add(MoveTemp(AssetsTaskChain));
}

USceneComponent* FUsdGeometryCacheTranslator::CreateComponents()
{
	if (!IsPotentialGeometryCacheRoot())
	{
		return Super::CreateComponents();
	}

	USceneComponent* SceneComponent = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		const bool bCheckForComponent = true;
		if (ShouldSkipSkinnablePrim(bCheckForComponent))
		{
			return nullptr;
		}

		SceneComponent = CreateComponentsEx({UGeometryCacheUsdComponent::StaticClass()}, {});
	}
	else
	{
		SceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	UpdateComponents(SceneComponent);

	if (UGeometryCacheComponent* Component = Cast<UGeometryCacheComponent>(SceneComponent))
	{
		if (Context->PrimLinkCache && Context->UsdAssetCache)
		{
			if (UGeometryCache* GeometryCache = Context->PrimLinkCache->GetSingleAssetForPrim<UGeometryCache>(PrimPath))
			{
				// Geometry caches don't support LODs
				const bool bAllowInterpretingLODs = false;

				MeshTranslationImpl::SetMaterialOverrides(GetPrim(), GeometryCache->Materials, *Component, *Context);

				// Check if the prim has the GroomBinding schema and setup the component and assets necessary to bind the groom to the GeometryCache
				if (UsdUtils::PrimHasSchema(GetPrim(), UnrealIdentifiers::GroomBindingAPI) && Context->bAllowParsingGroomAssets)
				{
					UsdGroomTranslatorUtils::CreateGroomBindingAsset(
						GetPrim(),
						*Context->UsdAssetCache,
						*Context->PrimLinkCache,
						Context->ObjectFlags,
						Context->bShareAssetsForIdenticalPrims
					);

					// For the groom binding to work, the GroomComponent must be a child of the SceneComponent
					// so the Context ParentComponent is set to the SceneComponent temporarily
					TGuardValue<USceneComponent*> ParentComponentGuard{Context->ParentComponent, SceneComponent};
					const bool bNeedsActor = false;
					UGroomComponent* GroomComponent = Cast<UGroomComponent>(
						CreateComponentsEx(TSubclassOf<USceneComponent>(UGroomComponent::StaticClass()), bNeedsActor)
					);
					if (GroomComponent)
					{
						UpdateComponents(SceneComponent);
					}
				}
			}
		}
	}

	return SceneComponent;
}

void FUsdGeometryCacheTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	UGeometryCacheComponent* GeometryCacheComponent = Cast<UGeometryCacheComponent>(SceneComponent);

	const bool bCheckForComponent = true;
	if (!Cast<UUsdDrawModeComponent>(SceneComponent) && ShouldSkipSkinnablePrim(bCheckForComponent))
	{
		return;
	}

	if (!GeometryCacheComponent || !IsPotentialGeometryCacheRoot())
	{
		Super::UpdateComponents(SceneComponent);
		return;
	}

	if (SceneComponent)
	{
		SceneComponent->Modify();
	}

	// Set the initial GeometryCache on the GeometryCacheComponent
	if (GeometryCacheComponent)
	{
		UGeometryCache* GeometryCache = nullptr;
		if (Context->PrimLinkCache)
		{
			GeometryCache = Context->PrimLinkCache->GetSingleAssetForPrim<UGeometryCache>(PrimPath);
		}

		bool bShouldRegister = false;
		if (GeometryCache != GeometryCacheComponent->GetGeometryCache())
		{
			bShouldRegister = true;

			if (GeometryCacheComponent->IsRegistered())
			{
				GeometryCacheComponent->UnregisterComponent();
			}

			// Skip the extra handling in SetGeometryCache
			GeometryCacheComponent->GeometryCache = GeometryCache;
		}

		// Manually tick USD GeometryCache only when their tracks are disabled in Sequencer
		// but also need to tick for the initial setup
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.DisableGeoCacheTracks"));
		bool bDisableGeoCacheTracks = CVar && CVar->GetBool();
		if (bDisableGeoCacheTracks || !Context->bSequencerIsAnimating)
		{
			float TimeCode = Context->Time;
			if (FMath::IsNaN(TimeCode))
			{
				int32 StartFrame = FMath::FloorToInt(Context->Stage.GetStartTimeCode());
				int32 EndFrame = FMath::CeilToInt(Context->Stage.GetEndTimeCode());
				UsdUtils::GetAnimatedMeshTimeCodes(Context->Stage, PrimPath.GetString(), StartFrame, EndFrame);

				TimeCode = static_cast<float>(StartFrame);
			}

			// This is the main call responsible for animating the geometry cache.
			// It needs to happen after setting the geometry cache and before registering, because we must force the
			// geometry cache to register itself at Context->Time so that it will synchronously load that frame right away.
			// Otherwise the geometry cache will start at t=0 regardless of Context->Time
			GeometryCacheComponent->SetManualTick(true);

			// Looping is disabled since the animation is driven by Sequencer
			const bool bIsLooping = false;
			GeometryCacheComponent->SetLooping(bIsLooping);

			float StageStartOffsetSeconds = 0.0f;
			if (GeometryCache)
			{
				if (UUsdGeometryCacheAssetUserData* UserData = GeometryCache->GetAssetUserData<UUsdGeometryCacheAssetUserData>())
				{
					StageStartOffsetSeconds = UserData->StageStartOffsetSeconds;
				}
			}

			// The Time from the stage has to be adjusted to be relative to the time range of the geometry cache
			// by applying the start offset. Thus, the adjusted time has to be clamped between 0 and the duration.
			const double FramesPerSecond = Context->Stage.GetTimeCodesPerSecond();
			float AdjustedTime = static_cast<float>(TimeCode / FramesPerSecond - StageStartOffsetSeconds);
			if (GeometryCache)
			{
				const float Duration = GeometryCache->CalculateDuration();
				AdjustedTime = FMath::Clamp(AdjustedTime, 0.0f, Duration);
			}

			const bool bIsRunning = true;
			const bool bIsBackwards = false;
			GeometryCacheComponent->TickAtThisTime(AdjustedTime, bIsRunning, bIsBackwards, bIsLooping);
		}

		// If the prim has a GroomBinding schema, apply the target groom to its associated GroomComponent
		if (UsdUtils::PrimHasSchema(GetPrim(), UnrealIdentifiers::GroomBindingAPI))
		{
			UsdGroomTranslatorUtils::SetGroomFromPrim(GetPrim(), *Context->PrimLinkCache, SceneComponent);
		}

		const bool bIsImporting = Context->bIsImporting || Context->GeometryCacheImport == EGeometryCacheImport::OnLoad;

		// Defer to xformable translator to set our transforms, visibility, etc. but only when opening the stage: This will be baked in for import.
		// Don't go through FUsdGeomMeshTranslator::UpdateComponents as it will want to create a static mesh if PrimPath is an animated mesh prim
		// (which is likely, given that we're running this FUsdGeometryCacheTranslator for it)
		if (!bIsImporting)
		{
			FUsdGeomXformableTranslator::UpdateComponents(GeometryCacheComponent);
		}

		// Note how we should only register if our geometry cache changed: If we did this every time we would
		// register too early during the process of duplicating into PIE, and that would prevent a future RegisterComponent
		// call from naturally creating the required render state
		if (bShouldRegister && !GeometryCacheComponent->IsRegistered())
		{
			GeometryCacheComponent->RegisterComponent();
		}
	}
}

bool FUsdGeometryCacheTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	if (IsPotentialGeometryCacheRoot() || UsdUtils::GetAppliedDrawMode(GetPrim()) != EUsdDrawMode::Default)
	{
		return true;
	}

	if (ShouldSkipSkinnablePrim())
	{
		return false;
	}

	return Super::CollapsesChildren(CollapsingType);
}

bool FUsdGeometryCacheTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	if (IsPotentialGeometryCacheRoot() || ShouldSkipSkinnablePrim())
	{
		return false;
	}

	return Super::CanBeCollapsed(CollapsingType);
}

TSet<UE::FSdfPath> FUsdGeometryCacheTranslator::CollectAuxiliaryPrims() const
{
	if (!IsPotentialGeometryCacheRoot())
	{
		return Super::CollectAuxiliaryPrims();
	}

	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	if (ShouldSkipSkinnablePrim())
	{
		return {};
	}

	TSet<UE::FSdfPath> AuxPrims;

	// Here, we collect all meshes even non-animated ones since they'll be collapse into the cache
	TArray<UE::FUsdPrim> VisibleChildPrims = UsdUtils::GetVisibleChildren(GetPrim(), Context->PurposesToLoad);
	AuxPrims.Reserve(VisibleChildPrims.Num());
	for (const UE::FUsdPrim& VisibleChild : VisibleChildPrims)
	{
		if (VisibleChild.IsA(TEXT("Imageable")))
		{
			AuxPrims.Add(VisibleChild.GetPrimPath());
		}
	}
	return AuxPrims;
}

bool FUsdGeometryCacheTranslator::IsPotentialGeometryCacheRoot() const
{
	// The logic to check for GeometryCache is completely in the UsdInfoCache
	return Context->UsdInfoCache->IsPotentialGeometryCacheRoot(GetPrim());
}

#endif	  // #if USE_USD_SDK
