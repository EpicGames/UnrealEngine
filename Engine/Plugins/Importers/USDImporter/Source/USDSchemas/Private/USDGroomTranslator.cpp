// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGroomTranslator.h"

#if USE_USD_SDK && WITH_EDITOR

#include "GroomAsset.h"
#include "GroomBuilder.h"
#include "GroomCache.h"
#include "GroomCacheImporter.h"
#include "GroomComponent.h"
#include "GroomImportOptions.h"
#include "HairDescription.h"
#include "HairStrandsImporter.h"
#include "Misc/ArchiveMD5.h"
#include "Templates/UniquePtr.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

#include "Objects/USDPrimLinkCache.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGroomConversion.h"
#include "USDGroomTranslatorUtils.h"
#include "USDIntegrationUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDTranslatorUtils.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/timeCode.h"
#include "pxr/usd/usdGeom/basisCurves.h"
#include "pxr/usd/usdGeom/imageable.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDGroomTranslator"

namespace UE::UsdGroomTranslator::Private
{
	FSHAHash ComputeHairDescriptionHash(FHairDescription& HairDescription, TArray<FHairGroupsInterpolation>& BuildSettings)
	{
		// The computed hash takes the hair description and group build settings into account since the groom builder
		// will use those settings to build the groom asset from the description
		FArchiveMD5 ArMD5;
		HairDescription.Serialize(ArMD5);

		for (FHairGroupsInterpolation& GroupSettings : BuildSettings)
		{
			GroupSettings.BuildDDCKey(ArMD5);
		}

		FMD5Hash MD5Hash;
		ArMD5.GetHash(MD5Hash);

		FSHA1 SHA1;
		SHA1.Update(MD5Hash.GetBytes(), MD5Hash.GetSize());
		SHA1.Final();

		FSHAHash SHAHash;
		SHA1.GetHash(SHAHash.Hash);

		return SHAHash;
	}

	void ComputeFrameHairDescriptionHash(const pxr::UsdPrim& Prim, TArray<FHairGroupsInterpolation>& BuildSettings, int32 FrameNumber, FSHA1& Hash)
	{
		FHairDescription FrameHairDescription;
		if (UsdToUnreal::ConvertGroomHierarchy(Prim, pxr::UsdTimeCode(FrameNumber), FTransform::Identity, FrameHairDescription))
		{
			FHairDescriptionGroups GroupsDescription;
			FGroomBuilder::BuildHairDescriptionGroups(FrameHairDescription, GroupsDescription);

			FSHAHash SHAHash = ComputeHairDescriptionHash(FrameHairDescription, BuildSettings);
			Hash.Update((const uint8*)SHAHash.Hash, sizeof(SHAHash.Hash));
		}
	}
}	 // namespace UE::UsdGroomTranslator::Private

class FUsdGroomCreateAssetsTaskChain : public FUsdSchemaTranslatorTaskChain
{
public:
	explicit FUsdGroomCreateAssetsTaskChain(const TSharedRef<FUsdSchemaTranslationContext>& InContext, const UE::FSdfPath& InPrimPath)
		: PrimPath(InPrimPath)
		, Context(InContext)
	{
		AnimInfo.Attributes = EGroomCacheAttributes::None;
		AnimInfo.StartFrame = TNumericLimits<int32>::Max();
		AnimInfo.EndFrame = TNumericLimits<int32>::Min();

		SetupTasks();
	}

protected:

	UE::FSdfPath PrimPath;
	TSharedRef<FUsdSchemaTranslationContext> Context;

	FHairDescription HairDescription;

	TStrongObjectPtr<UGroomImportOptions> ImportOptions;
	UGroomAsset* GroomAsset = nullptr;
	UGroomCache* GroomCache = nullptr;
	FGroomAnimationInfo AnimInfo;
	FString PrefixedGroomCacheHash;

protected:
	UE::FUsdPrim GetPrim() const
	{
		return Context->Stage.GetPrimAtPath(PrimPath);
	}

protected:
	void SetupTasks()
	{
		FScopedUnrealAllocs UnrealAllocs;

		// Create hair description (Async)
		Do(ESchemaTranslationLaunchPolicy::Async,
		   [this]() -> bool
		   {
			   const bool bSuccess = UsdToUnreal::ConvertGroomHierarchy(
				   GetPrim(),
				   pxr::UsdTimeCode::EarliestTime(),
				   FTransform::Identity,
				   HairDescription,
				   &AnimInfo
			   );

			   const double StageTimeCodesPerSecond = Context->Stage.GetTimeCodesPerSecond();
			   AnimInfo.SecondsPerFrame = float(1.0f / StageTimeCodesPerSecond);

			   if (bSuccess && AnimInfo.IsValid())
			   {
				   AnimInfo.Duration = AnimInfo.NumFrames * AnimInfo.SecondsPerFrame;
				   AnimInfo.StartTime = AnimInfo.StartFrame * AnimInfo.SecondsPerFrame;
				   AnimInfo.EndTime = AnimInfo.EndFrame * AnimInfo.SecondsPerFrame;
			   }

			   return bSuccess && HairDescription.IsValid();
		   });

		// Build groom asset from hair description (Sync)
		Then(
			ESchemaTranslationLaunchPolicy::Sync,
			[this]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FUsdGroomCreateAssetsTaskChain::Build);

				// Extract the groom groups info from the hair description to get the number of groups
				FHairDescriptionGroups GroupsDescription;
				FGroomBuilder::BuildHairDescriptionGroups(HairDescription, GroupsDescription);

				ImportOptions.Reset(CreateGroomImportOptions(GroupsDescription, Context->GroomInterpolationSettings));

				FSHAHash SHAHash = UE::UsdGroomTranslator::Private::ComputeHairDescriptionHash(HairDescription, ImportOptions->InterpolationSettings);

				const FString PrefixedAssetHash = UsdUtils::GetAssetHashPrefix(GetPrim(), Context->bShareAssetsForIdenticalPrims)
												  + SHAHash.ToString();

				const FString PrimPathString = PrimPath.GetString();
				const FString DesiredName = FPaths::GetBaseFilename(PrimPathString);

				bool bGroomAssetIsNew = false;
				GroomAsset = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<UGroomAsset>(
					PrefixedAssetHash,
					DesiredName,
					Context->ObjectFlags,
					[this](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
					{
						FHairImportContext HairImportContext(ImportOptions.Get(), Outer, UGroomAsset::StaticClass(), SanitizedName, FlagsToUse);
						UGroomAsset* ExistingAsset = nullptr;
						return FHairStrandsImporter::ImportHair(HairImportContext, HairDescription, ExistingAsset);
					},
					&bGroomAssetIsNew
				);

				if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(GroomAsset))
				{
					UserData->PrimPaths.AddUnique(PrimPathString);

					if (Context->MetadataOptions.bCollectMetadata)
					{
						UsdToUnreal::ConvertMetadata(
							GetPrim(),
							UserData,
							Context->MetadataOptions.BlockedPrefixFilters,
							Context->MetadataOptions.bInvertFilters,
							Context->MetadataOptions.bCollectFromEntireSubtrees
						);
					}
					else
					{
						// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the
						// AssetUserData
						UserData->StageIdentifierToMetadata.Remove(GetPrim().GetStage().GetRootLayer().GetIdentifier());
					}
				}

				if (GroomAsset && Context->PrimLinkCache)
				{
					Context->PrimLinkCache->LinkAssetToPrim(PrimPath, GroomAsset);
				}

				// Next step is to parse the GroomCache data if it was determined that the groom has animated attributes
				return GroomAsset != nullptr && AnimInfo.IsValid();
			}
		);

		// Compute the GroomCache hash (Async)
		Then(
			ESchemaTranslationLaunchPolicy::Async,
			[this]() -> bool
			{
				if (!Context->PrimLinkCache)
				{
					return false;
				}

				GroomAsset = Context->PrimLinkCache->GetSingleAssetForPrim<UGroomAsset>(PrimPath);
				if (!GroomAsset)
				{
					return false;
				}

				// Compute GroomCache hash from the first and last frame HairDescription...
				FSHA1 SHA1;
				UE::UsdGroomTranslator::Private::ComputeFrameHairDescriptionHash(
					GetPrim(),
					ImportOptions->InterpolationSettings,
					AnimInfo.StartFrame,
					SHA1
				);
				UE::UsdGroomTranslator::Private::ComputeFrameHairDescriptionHash(
					GetPrim(),
					ImportOptions->InterpolationSettings,
					AnimInfo.EndFrame,
					SHA1
				);

				// Along with relevant AnimInfo data
				SHA1.Update((const uint8*)&AnimInfo.NumFrames, sizeof(AnimInfo.NumFrames));
				SHA1.Update((const uint8*)&AnimInfo.Attributes, sizeof(AnimInfo.Attributes));
				SHA1.Final();

				FSHAHash Hash;
				SHA1.GetHash(Hash.Hash);

				PrefixedGroomCacheHash = UsdUtils::GetAssetHashPrefix(GetPrim(), Context->bShareAssetsForIdenticalPrims) + Hash.ToString();
				return true;
			}
		);

		// Get or create GroomCache asset (Sync)
		Then(
			ESchemaTranslationLaunchPolicy::Sync,
			[this]() -> bool
			{
				const FString StrandsGroomCachePrimPath = UsdGroomTranslatorUtils::GetStrandsGroomCachePrimPath(PrimPath);
				const FString DesiredName = FPaths::GetBaseFilename(StrandsGroomCachePrimPath);

				bool bGroomCacheIsNew = false;
				GroomCache = Context->UsdAssetCache->GetOrCreateCachedAsset<UGroomCache>(	 //
					PrefixedGroomCacheHash,
					DesiredName,
					Context->ObjectFlags,
					&bGroomCacheIsNew
				);

				if (GroomCache && Context->PrimLinkCache)
				{
					Context->PrimLinkCache->LinkAssetToPrim(PrimPath, GroomCache);

					if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(GroomCache))
					{
						UserData->PrimPaths.AddUnique(PrimPath.GetString());

						if (Context->MetadataOptions.bCollectMetadata)
						{
							UsdToUnreal::ConvertMetadata(
								GetPrim(),
								UserData,
								Context->MetadataOptions.BlockedPrefixFilters,
								Context->MetadataOptions.bInvertFilters,
								Context->MetadataOptions.bCollectFromEntireSubtrees
							);
						}
						else
						{
							// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the AssetUserData
							UserData->StageIdentifierToMetadata.Remove(GetPrim().GetStage().GetRootLayer().GetIdentifier());
						}
					}
				}

				// We only need to continue if we need to finish building this groom cache
				return GroomCache && bGroomCacheIsNew;
			}
		);

		// Parse GroomCache data into processor (Async)
		Then(
			ESchemaTranslationLaunchPolicy::Async,
			[this]() -> bool
			{
				if (!GroomAsset || !GroomCache)
				{
					return false;
				}

				FGroomCacheProcessor GroomCacheProcessor{EGroomCacheType::Strands, AnimInfo.Attributes};

				// ref. FGroomCacheImporter::ImportGroomCache
				const TArray<FHairGroupPlatformData>& GroomHairGroupsData = GroomAsset->GetHairGroupsPlatformData();

				// Process data and store results into GroomCacheProcessor.
				// Each frame is translated into a HairDescription and processed into HairGroupData
				// Sample one extra frame so that we can interpolate between EndFrame - 1 and EndFrame
				bool bSuccess = true;
				for (int32 FrameIndex = AnimInfo.StartFrame; FrameIndex < AnimInfo.EndFrame + 1; ++FrameIndex)
				{
					FHairDescription FrameHairDescription;
					bSuccess = UsdToUnreal::ConvertGroomHierarchy(
						GetPrim(),
						pxr::UsdTimeCode(FrameIndex),
						FTransform::Identity,
						FrameHairDescription
					);

					if (!bSuccess)
					{
						break;
					}

					FHairDescriptionGroups HairDescriptionGroups;
					if (!FGroomBuilder::BuildHairDescriptionGroups(FrameHairDescription, HairDescriptionGroups))
					{
						bSuccess = false;
						break;
					}

					const uint32 GroupCount = HairDescriptionGroups.HairGroups.Num();

					TArray<FHairGroupInfoWithVisibility> HairGroupsInfo = GroomAsset->GetHairGroupsInfo();
					TArray<FGroomCacheInputData> CacheInputData;
					CacheInputData.SetNum(GroupCount);
					for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
					{
						const FHairDescriptionGroup& HairGroup = HairDescriptionGroups.HairGroups[GroupIndex];
						FGroomBuilder::BuildData(
							HairGroup,
							GroomAsset->GetHairGroupsInterpolation()[GroupIndex],
							HairGroupsInfo[GroupIndex],
							CacheInputData[GroupIndex].Strands,
							CacheInputData[GroupIndex].Guides
						);
					}

					// Validate that the GroomCache has the same topology as the static groom
					if (CacheInputData.Num() == GroomHairGroupsData.Num())
					{
						for (uint32 GroupIndex = 0; GroupIndex < GroupCount; ++GroupIndex)
						{
							if (CacheInputData[GroupIndex].Strands.GetNumPoints() != GroomHairGroupsData[GroupIndex].Strands.BulkData.GetNumPoints())
							{
								bSuccess = false;
								USD_LOG_USERWARNING(FText::Format(
									LOCTEXT(
										"GroomCacheWrongNumVertices",
										"GroomCache frame {0} does not have the same number of vertices as the static groom ({1} instead of {2}). Aborting GroomCache import."
									),
									FrameIndex,
									CacheInputData[GroupIndex].Strands.GetNumPoints(),
									GroomHairGroupsData[GroupIndex].Strands.BulkData.GetNumPoints()
								));
								break;
							}
						}
					}
					else
					{
						bSuccess = false;
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"GroomCacheWrongNumGroups",
								"GroomCache does not have the same number of groups as the static groom ({0} instead of {1}). Aborting GroomCache import."
							),
							CacheInputData.Num(),
							GroomHairGroupsData.Num()
						));
					}

					if (!bSuccess)
					{
						break;
					}

					// The HairGroupData is converted into animated groom data by the GroomCacheProcessor
					GroomCacheProcessor.AddGroomSample(MoveTemp(CacheInputData));
				}

				if (bSuccess)
				{
					// We used to call FGroomCacheImporter::ProcessToGroomCache here, but all it does is a bunch of package
					// management for reimport/creation, which the asset cache will already do. So here just copy what it does
					// internally, which luckily is all public
					GroomCache->Initialize(GroomCacheProcessor.GetType());
					GroomCacheProcessor.TransferChunks(GroomCache);
					GroomCache->SetGroomAnimationInfo(AnimInfo);
				}
				else
				{
					const FString StrandsGroomCachePrimPath = UsdGroomTranslatorUtils::GetStrandsGroomCachePrimPath(PrimPath);
					USD_LOG_WARNING(TEXT("Failed to create GroomCache for prim '%s'"), *StrandsGroomCachePrimPath);

					UsdUnreal::TranslatorUtils::AbandonFailedAsset(GroomCache, Context->UsdAssetCache.Get(), Context->PrimLinkCache);
				}

				return bSuccess;
			}
		);

		// Post import groom cache
		Then(
			ESchemaTranslationLaunchPolicy::Sync,
			[this]() -> bool
			{
				if (GroomCache)
				{
					// This needs to be called from the main thread
					GroomCache->MarkPackageDirty();
					GroomCache->PostEditChange();
				}

				return false;	 // No other tasks
			}
		);
	}
};

bool FUsdGroomTranslator::IsGroomPrim() const
{
	return UsdUtils::PrimHasSchema(GetPrim(), UnrealIdentifiers::GroomAPI);
}

void FUsdGroomTranslator::CreateAssets()
{
	if (!Context->bAllowParsingGroomAssets || !IsGroomPrim())
	{
		return Super::CreateAssets();
	}

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	Context->TranslatorTasks.Add(MakeShared<FUsdGroomCreateAssetsTaskChain>(Context, PrimPath));
}

USceneComponent* FUsdGroomTranslator::CreateComponents()
{
	if (!Context->bAllowParsingGroomAssets || !IsGroomPrim())
	{
		return Super::CreateComponents();
	}

	// Display the groom as a standalone actor only if the stage loads the matching purpose.
	// The groom asset is processed regardless of the purpose so that it can be bound to mesh prims.
	if (!EnumHasAllFlags(Context->PurposesToLoad, IUsdPrim::GetPurpose(GetPrim())))
	{
		return nullptr;
	}

	USceneComponent* Component = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		const bool bNeedsActor = true;
		Component = CreateComponentsEx({UGroomComponent::StaticClass()}, bNeedsActor);
	}
	else
	{
		Component = CreateAlternativeDrawModeComponents(DrawMode);
	}

	UpdateComponents(Component);

	return Component;
}

void FUsdGroomTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	if (Context->bAllowParsingGroomAssets && IsGroomPrim())
	{
		if (UGroomComponent* GroomComponent = Cast<UGroomComponent>(SceneComponent))
		{
			GroomComponent->Modify();

			UGroomAsset* Groom = nullptr;
			if (Context->PrimLinkCache)
			{
				Groom = Context->PrimLinkCache->GetSingleAssetForPrim<UGroomAsset>(PrimPath);
			}

			bool bShouldRegister = false;
			if (Groom != GroomComponent->GroomAsset.Get())
			{
				bShouldRegister = true;

				if (GroomComponent->IsRegistered())
				{
					GroomComponent->UnregisterComponent();
				}

				GroomComponent->SetGroomAsset(Groom);

				if (Groom)
				{
					UGroomCache* GroomCache = Context->PrimLinkCache->GetSingleAssetForPrim<UGroomCache>(PrimPath);
					if (GroomCache != GroomComponent->GroomCache.Get())
					{
						GroomComponent->SetGroomCache(GroomCache);
					}
				}
			}

			// Use the prim purpose in conjunction with the prim's computed visibility to toggle the visibility of the groom component
			// since the component itself cannot be removed if the groom shouldn't be displayed
			const bool bShouldRender = UsdUtils::IsVisible(GetPrim()) && EnumHasAllFlags(Context->PurposesToLoad, IUsdPrim::GetPurpose(GetPrim()));
			GroomComponent->SetVisibility(bShouldRender);

			if (bShouldRegister && !GroomComponent->IsRegistered())
			{
				GroomComponent->RegisterComponent();
			}

			return;
		}
	}

	Super::UpdateComponents(SceneComponent);
}

bool FUsdGroomTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	if (!Context->bAllowParsingGroomAssets || !IsGroomPrim())
	{
		return Super::CollapsesChildren(CollapsingType);
	}

	return true;
}

bool FUsdGroomTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	if (!Context->bAllowParsingGroomAssets || !IsGroomPrim())
	{
		return Super::CanBeCollapsed(CollapsingType);
	}

	return true;
}

TSet<UE::FSdfPath> FUsdGroomTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bAllowParsingGroomAssets || !IsGroomPrim())
	{
		return Super::CollectAuxiliaryPrims();
	}

	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	if (!Context->UsdInfoCache->DoesPathCollapseChildren(PrimPath, ECollapsingType::Assets))
	{
		return {};
	}

	TSet<UE::FSdfPath> Result;
	{
		FScopedUsdAllocs UsdAllocs;

		TFunction<void(const pxr::UsdPrim&)> RecursivelyRegisterPrims;
		RecursivelyRegisterPrims = [&](const pxr::UsdPrim& UsdPrim)
		{
			if (pxr::UsdGeomBasisCurves Curves = pxr::UsdGeomBasisCurves(UsdPrim))
			{
				Result.Add(UE::FSdfPath{UsdPrim.GetPrimPath()});
			}
			else if (pxr::UsdGeomImageable(UsdPrim))
			{
				Result.Add(UE::FSdfPath{UsdPrim.GetPrimPath()});

				for (const pxr::UsdPrim& Child : UsdPrim.GetChildren())
				{
					RecursivelyRegisterPrims(Child);
				}
			}
		};

		pxr::UsdPrim Prim = GetPrim();
		RecursivelyRegisterPrims(Prim);
	}
	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
