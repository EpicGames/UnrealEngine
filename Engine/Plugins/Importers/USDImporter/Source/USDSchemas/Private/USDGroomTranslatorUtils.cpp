// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDGroomTranslatorUtils.h"

#if USE_USD_SDK && WITH_EDITOR

#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "UnrealUSDWrapper.h"
#include "USDAssetCache3.h"
#include "USDClassesModule.h"
#include "USDIntegrationUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"

#include "Engine/SkeletalMesh.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomComponent.h"
#include "Misc/SecureHash.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "USDConversionUtils.h"

#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/relationship.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeleton.h"
#include "USDIncludesEnd.h"

namespace UE::UsdGroomTranslatorUtils::Private
{
	struct FGroomBindingBuildSettings : public FGCObject
	{
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			Collector.AddReferencedObject(Groom);
			Collector.AddReferencedObject(SourceMesh);
			Collector.AddReferencedObject(TargetMesh);
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("FGroomBindingBuildSettings");
		}

		EGroomBindingMeshType GroomBindingType = EGroomBindingMeshType::SkeletalMesh;
		TObjectPtr<UGroomAsset> Groom = nullptr;
		TObjectPtr<UObject> SourceMesh = nullptr;
		TObjectPtr<UObject> TargetMesh = nullptr;
		int32 NumInterpolationPoints = 100;
		int32 MatchingSection = 0;
	};

	FSHAHash ComputeGroomBindingHash(const FGroomBindingBuildSettings& Settings)
	{
		// Ref. GroomBindingAsset BuildDerivedDataKeySuffix
		FString BindingType;
		FString SourceKey;
		FString TargetKey;
		if (Settings.GroomBindingType == EGroomBindingMeshType::SkeletalMesh)
		{
			// Binding type is implicitly SkeletalMesh so keep BindingType empty
			SourceKey = Cast<USkeletalMesh>(Settings.SourceMesh) ? Cast<USkeletalMesh>(Settings.SourceMesh)->GetDerivedDataKey() : FString();
			TargetKey = Cast<USkeletalMesh>(Settings.TargetMesh) ? Cast<USkeletalMesh>(Settings.TargetMesh)->GetDerivedDataKey() : FString();
		}
		else
		{
			BindingType = "GEOCACHE_";
			SourceKey = Cast<UGeometryCache>(Settings.SourceMesh) ? Cast<UGeometryCache>(Settings.SourceMesh)->GetHash() : FString();
			TargetKey = Cast<UGeometryCache>(Settings.TargetMesh) ? Cast<UGeometryCache>(Settings.TargetMesh)->GetHash() : FString();
		}
		const FString GroomKey = Settings.Groom ? Settings.Groom->GetDerivedDataKey() : FString();
		const FString PointKey = FString::FromInt(Settings.NumInterpolationPoints);
		const FString SectionKey = FString::FromInt(Settings.MatchingSection);

		FSHA1 SHA1;
		SHA1.UpdateWithString(*BindingType, BindingType.Len());
		SHA1.UpdateWithString(*SourceKey, SourceKey.Len());
		SHA1.UpdateWithString(*TargetKey, TargetKey.Len());
		SHA1.UpdateWithString(*GroomKey, GroomKey.Len());
		SHA1.UpdateWithString(*PointKey, PointKey.Len());
		SHA1.UpdateWithString(*SectionKey, SectionKey.Len());
		SHA1.Final();

		FSHAHash SHAHash;
		SHA1.GetHash(SHAHash.Hash);

		return SHAHash;
	}

	FString GetGroomPrimPath(const pxr::UsdPrim& Prim)
	{
		FScopedUsdAllocs Allocs;

		// Get the groom prim path to bind from the GroomBindingAPI
		if (pxr::UsdRelationship Relationship = Prim.GetRelationship(UnrealIdentifiers::UnrealGroomToBind))
		{
			pxr::SdfPathVector Targets;
			Relationship.GetTargets(&Targets);

			if (Targets.size() > 0)
			{
				// Validate that the target prim is in fact a groom prim
				const pxr::SdfPath& TargetPrimPath = Targets[0];
				pxr::UsdPrim TargetPrim = Prim.GetPrimAtPath(TargetPrimPath);
				if (TargetPrim && UsdUtils::PrimHasSchema(TargetPrim, UnrealIdentifiers::GroomAPI))
				{
					return UsdToUnreal::ConvertPath(TargetPrimPath);
				}
			}
		}
		return {};
	}

	UObject* GetGroomBindingSourceMesh(const pxr::UsdPrim& Prim, const FUsdPrimLinkCache& PrimLinkCache, EGroomBindingMeshType BindingType)
	{
		FScopedUsdAllocs Allocs;

		// Get the reference mesh asset from the GroomBindingAPI; this property is optional
		if (pxr::UsdRelationship Relationship = Prim.GetRelationship(UnrealIdentifiers::UnrealGroomReferenceMesh))
		{
			pxr::SdfPathVector Targets;
			Relationship.GetTargets(&Targets);

			if (Targets.size() > 0)
			{
				const pxr::SdfPath& TargetPrimPath = Targets[0];

				// Validate that the target prim and associated asset are of the expected type for the binding
				pxr::UsdPrim TargetPrim = Prim.GetPrimAtPath(TargetPrimPath);
				if (BindingType == EGroomBindingMeshType::SkeletalMesh && (pxr::UsdSkelRoot(TargetPrim) || pxr::UsdSkelSkeleton(TargetPrim)))
				{
					return PrimLinkCache.GetSingleAssetForPrim<USkeletalMesh>(UE::FSdfPath{TargetPrimPath});
				}
				else if (BindingType == EGroomBindingMeshType::GeometryCache && pxr::UsdGeomMesh(TargetPrim))
				{
					return PrimLinkCache.GetSingleAssetForPrim<UGeometryCache>(UE::FSdfPath{TargetPrimPath});
				}
			}
		}
		return nullptr;
	}
}	 // namespace UE::UsdGroomTranslatorUtils::Private

namespace UsdGroomTranslatorUtils
{
	using namespace UE::UsdGroomTranslatorUtils::Private;

	void CreateGroomBindingAsset(
		const pxr::UsdPrim& Prim,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache,
		EObjectFlags ObjectFlags,
		bool bShareAssetsForIdenticalPrims
	)
	{
		// At this point, the prim (SkelRoot/Skeleton or GeomMesh) has already been checked to have the GroomBindingAPI,
		// so we need to set up the GroomComponent and the groom binding asset to be able to bind it to the mesh

		// The GroomBinding schema must specify a groom prim to bind to the mesh
		const FString GroomPrimPath = GetGroomPrimPath(Prim);
		if (GroomPrimPath.IsEmpty())
		{
			return;
		}

		// The GroomAsset should already be processed and cached by the USDGroomTranslator
		UGroomAsset* GroomAsset = PrimLinkCache.GetSingleAssetForPrim<UGroomAsset>(UE::FSdfPath{*GroomPrimPath});
		if (!GroomAsset)
		{
			return;
		}

		// Determine the type of binding needed based on the prim mesh type
		const FString PrimPath(UsdToUnreal::ConvertPath(Prim.GetPath()));
		EGroomBindingMeshType GroomBindingType = EGroomBindingMeshType::SkeletalMesh;
		UObject* TargetMesh = PrimLinkCache.GetSingleAssetForPrim<USkeletalMesh>(UE::FSdfPath{*PrimPath});
		if (!TargetMesh)
		{
			TargetMesh = PrimLinkCache.GetSingleAssetForPrim<UGeometryCache>(UE::FSdfPath{*PrimPath});
			if (!TargetMesh)
			{
				return;
			}
			GroomBindingType = EGroomBindingMeshType::GeometryCache;
		}

		FGroomBindingBuildSettings Settings;
		Settings.GroomBindingType = GroomBindingType;
		Settings.Groom = GroomAsset;
		Settings.TargetMesh = TargetMesh;
		Settings.SourceMesh = GetGroomBindingSourceMesh(Prim, PrimLinkCache, GroomBindingType);

		const FString PrefixedBindingHash = UsdUtils::GetAssetHashPrefix(Prim, bShareAssetsForIdenticalPrims)
											+ ComputeGroomBindingHash(Settings).ToString();

		const FString GroomBindingPath = FString::Printf(TEXT("%s_groombinding"), *PrimPath);

		bool bIsNew = false;
		UGroomBindingAsset* GroomBinding = nullptr;

		const FString DesiredName = FPaths::GetBaseFilename(GroomBindingPath);

		// Prevent the creation of the GroomBinding or it's internal UObjects (if any) from being added to the transaction.
		// This is important because we call CreateGroomBindingAsset from CreateComponents(), which is outside of the normal
		// transaction suppression around the CreateAssets() calls that the AUsdStageActor will already do in AUsdStageActor::LoadAssets
		// and AUsdStageActor::LoadAsset
		TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};

		// Need at least the groom and the target mesh to build a GroomBindingAsset; the source mesh is optional
		if (!GroomBindingPath.IsEmpty() && GroomAsset && TargetMesh)
		{
			GroomBinding = AssetCache.GetOrCreateCachedAsset<UGroomBindingAsset>(PrefixedBindingHash, DesiredName, ObjectFlags, &bIsNew);
		}

		if (GroomBinding)
		{
			if (bIsNew)
			{
				GroomBinding->SetGroomBindingType(Settings.GroomBindingType);
				GroomBinding->SetGroom(Settings.Groom);
				if (GroomBinding->GetGroomBindingType() == EGroomBindingMeshType::SkeletalMesh)
				{
					GroomBinding->SetSourceSkeletalMesh(Cast<USkeletalMesh>(Settings.SourceMesh));
					GroomBinding->SetTargetSkeletalMesh(Cast<USkeletalMesh>(Settings.TargetMesh));
				}
				else
				{
					GroomBinding->SetSourceGeometryCache(Cast<UGeometryCache>(Settings.SourceMesh));
					GroomBinding->SetTargetGeometryCache(Cast<UGeometryCache>(Settings.TargetMesh));
				}
				GroomBinding->GetHairGroupsPlatformData().Reserve(Settings.Groom->GetHairGroupsPlatformData().Num());
				GroomBinding->SetNumInterpolationPoints(Settings.NumInterpolationPoints);
				GroomBinding->SetMatchingSection(Settings.MatchingSection);

				GroomBinding->Build();
			}

			PrimLinkCache.LinkAssetToPrim(UE::FSdfPath{*GroomBindingPath}, GroomBinding);
		}
	}

	void SetGroomFromPrim(const pxr::UsdPrim& Prim, const FUsdPrimLinkCache& PrimLinkCache, USceneComponent* SceneComponent)
	{
		if (!SceneComponent)
		{
			return;
		}

		FScopedUsdAllocs Allocs;

		const FString GroomPrimPath = GetGroomPrimPath(Prim);
		if (GroomPrimPath.IsEmpty())
		{
			return;
		}

		UGroomAsset* GroomAsset = PrimLinkCache.GetSingleAssetForPrim<UGroomAsset>(UE::FSdfPath{*GroomPrimPath});
		if (!GroomAsset)
		{
			return;
		}

		const FString PrimPath(UsdToUnreal::ConvertPath(Prim.GetPath()));
		const FString GroomBindingPath = FString::Printf(TEXT("%s_groombinding"), *PrimPath);
		UGroomBindingAsset* GroomBinding = PrimLinkCache.GetSingleAssetForPrim<UGroomBindingAsset>(UE::FSdfPath{*GroomBindingPath});

		// Set the GroomAsset and GroomBindingAsset on the child GroomComponent of SceneComponent that was set up in the translator
		TArray<USceneComponent*> Children;
		const bool bIncludeAllDescendants = false;
		SceneComponent->GetChildrenComponents(bIncludeAllDescendants, Children);
		for (USceneComponent* ChildComponent : Children)
		{
			if (UGroomComponent* GroomComponent = Cast<UGroomComponent>(ChildComponent))
			{
				if (GroomComponent->GroomAsset != GroomAsset || GroomComponent->BindingAsset != GroomBinding)
				{
					GroomComponent->SetGroomAsset(GroomAsset, GroomBinding);
				}
				break;
			}
		}
	}

	FString GetStrandsGroomCachePrimPath(const UE::FSdfPath& PrimPath)
	{
		return FString::Printf(TEXT("%s_strands_cache"), *PrimPath.GetString());
	}
}	 // namespace UsdGroomTranslatorUtils

#endif	  // #if USE_USD_SDK && WITH_EDITOR
