// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK && WITH_EDITOR

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE
	class UsdPrim;
PXR_NAMESPACE_CLOSE_SCOPE

class FUsdPrimLinkCache;
class USceneComponent;
class UUsdAssetCache3;

namespace UE
{
	class FSdfPath;
}

namespace UsdGroomTranslatorUtils
{
	// Create the groom binding asset for the given Prim with GroomBindingAPI and cache it in the AssetCache
	void CreateGroomBindingAsset(
		const pxr::UsdPrim& Prim,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache,
		EObjectFlags ObjectFlags,
		bool bShareAssetsForIdenticalPrims
	);

	// Set the groom asset targeted by the given prim with GroomBindingAPI on the SceneComponent (with a GroomComponent as a direct child)
	void SetGroomFromPrim(const pxr::UsdPrim& Prim, const FUsdPrimLinkCache& PrimLinkCache, USceneComponent* SceneComponent);

	// Get the (pseudo) prim path for the strands GroomCache associated with a groom prim
	FString GetStrandsGroomCachePrimPath(const UE::FSdfPath& PrimPath);
}

#endif	  // #if USE_USD_SDK && WITH_EDITOR
