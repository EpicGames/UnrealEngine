// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDLuxLightTranslator.h"

#include "Objects/USDPrimLinkCache.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDLightConversion.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/UsdPrim.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/TextureCube.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/usd/usdLux/diskLight.h"
#include "pxr/usd/usdLux/domeLight.h"
#include "pxr/usd/usdLux/domeLight_1.h"
#include "pxr/usd/usdLux/lightAPI.h"
#include "pxr/usd/usdLux/rectLight.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDLuxLightTranslator"

void FUsdLuxLightTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdLuxLightTranslator::CreateAssets);

	if (!Context->UsdAssetCache || !Context->PrimLinkCache)
	{
		return;
	}

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	const FString VolumePrimPathString = PrimPath.GetString();
	pxr::UsdPrim Prim = GetPrim();

	pxr::UsdAttribute TextureFileAttr;
	if (pxr::UsdLuxDomeLight DomeLight{Prim})
	{
		TextureFileAttr = DomeLight.GetTextureFileAttr();
	}
	else if (pxr::UsdLuxDomeLight_1 DomeLight1{Prim})
	{
		TextureFileAttr = DomeLight1.GetTextureFileAttr();
	}
	if (!TextureFileAttr)
	{
		return;
	}

	const FString ResolvedDomeTexturePath = UsdUtils::GetResolvedAssetPath(TextureFileAttr);
	if (ResolvedDomeTexturePath.IsEmpty())
	{
		FScopedUsdAllocs Allocs;

		pxr::SdfAssetPath TextureAssetPath;
		TextureFileAttr.Get<pxr::SdfAssetPath>(&TextureAssetPath);

		// Show a good warning for this because it's easy to pick some cubemap asset from the engine (that usually don't come with the
		// source texture) and have the dome light silently not work again
		FString TargetAssetPath = UsdToUnreal::ConvertString(TextureAssetPath.GetAssetPath());
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT("FailToFindTexture", "Failed to find texture '{0}' used for DomeLight '{1}'!"),
			FText::FromString(TargetAssetPath),
			FText::FromString(UsdToUnreal::ConvertPath(Prim.GetPath()))
		));

		return;
	}

	const FString PrefixedTextureHash = UsdUtils::GetAssetHashPrefix(Prim, Context->bShareAssetsForIdenticalPrims)
										+ LexToString(FMD5Hash::HashFile(*ResolvedDomeTexturePath));

	const FString& DesiredTextureName = FPaths::GetBaseFilename(ResolvedDomeTexturePath);

	TextureGroup Group = TextureGroup::TEXTUREGROUP_Skybox;

	bool bCreatedTexture = false;
	UTextureCube* Texture = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<UTextureCube>(
		PrefixedTextureHash,
		DesiredTextureName,
		Context->ObjectFlags,
		[&ResolvedDomeTexturePath, Group](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
		{
			return UsdUtils::CreateTexture(ResolvedDomeTexturePath, SanitizedName, Group, FlagsToUse, Outer);
		},
		&bCreatedTexture
	);

	if (UUsdAssetUserData* TextureUserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Texture))
	{
		TextureUserData->PrimPaths.AddUnique(UsdToUnreal::ConvertPath(Prim.GetPath()));
	}

	if (Texture)
	{
		Context->PrimLinkCache->LinkAssetToPrim(PrimPath, Texture);
	}
}

USceneComponent* FUsdLuxLightTranslator::CreateComponents()
{
	USceneComponent* SceneComponent = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		const bool bNeedsActor = true;
		SceneComponent = CreateComponentsEx({}, bNeedsActor);
	}
	else
	{
		SceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	UpdateComponents(SceneComponent);
	return SceneComponent;
}

void FUsdLuxLightTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	Super::UpdateComponents(SceneComponent);

	ULightComponentBase* LightComponent = Cast<ULightComponentBase>(SceneComponent);

	if (!LightComponent || !Context->PrimLinkCache)
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim Prim = GetPrim();

	const pxr::UsdLuxLightAPI LightAPI(Prim);

	if (!LightAPI)
	{
		return;
	}

	LightComponent->UnregisterComponent();

	UsdToUnreal::ConvertLight(Prim, *LightComponent, Context->Time);

	if (UDirectionalLightComponent* DirectionalLightComponent = Cast<UDirectionalLightComponent>(SceneComponent))
	{
		UsdToUnreal::ConvertDistantLight(Prim, *DirectionalLightComponent, Context->Time);
	}
	else if (URectLightComponent* RectLightComponent = Cast<URectLightComponent>(SceneComponent))
	{
		if (pxr::UsdLuxRectLight UsdRectLight{Prim})
		{
			UsdToUnreal::ConvertRectLight(Prim, *RectLightComponent, Context->Time);
		}
		else if (pxr::UsdLuxDiskLight UsdDiskLight{Prim})
		{
			UsdToUnreal::ConvertDiskLight(Prim, *RectLightComponent, Context->Time);
		}
	}
	else if (UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(SceneComponent))
	{
		if (USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(SceneComponent))
		{
			UsdToUnreal::ConvertLuxShapingAPI(Prim, *SpotLightComponent, Context->Time);
		}

		UsdToUnreal::ConvertSphereLight(Prim, *PointLightComponent, Context->Time);
	}
	else if (USkyLightComponent* SkyLightComponent = Cast<USkyLightComponent>(SceneComponent))
	{
		SkyLightComponent->Modify();

		if (UTextureCube* TextuxeCube = Context->PrimLinkCache->GetSingleAssetForPrim<UTextureCube>(PrimPath))
		{
			SkyLightComponent->Cubemap = TextuxeCube;
			SkyLightComponent->SourceType = ESkyLightSourceType::SLS_SpecifiedCubemap;
		}
	}

	if (!LightComponent->IsRegistered())
	{
		LightComponent->RegisterComponent();
	}
}

bool FUsdLuxLightTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	return false;
}

bool FUsdLuxLightTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return false;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
