// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDMediaSpatialAudioTranslator.h"

#include "Objects/USDPrimLinkCache.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "Components/AudioComponent.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Sound/SoundWave.h"

#if WITH_EDITOR
#include "Factories/SoundFactory.h"
#include "Utils.h"
#elif 0
#include "SoundFileIO/SoundFileIO.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include <pxr/usd/usd/prim.h>
#include <pxr/usd/usd/stage.h>
#include <pxr/usd/usdMedia/spatialAudio.h>
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDMediaSpatialAudioTranslator"

namespace UE::UsdMediaSpatialAudioTranslator::Private
{
	USoundWave* CreateSoundWave(const FString& FilePath, FName AssetName, UObject* Outer, EObjectFlags Flags)
	{
		USoundWave* SoundWave = nullptr;

#if WITH_EDITOR
		// Go through the factory if we can
		USoundFactory* SoundWaveFactory = NewObject<USoundFactory>();

		// Setup sane defaults for importing localized sound waves
		SoundWaveFactory->bAutoCreateCue = false;
		SoundWaveFactory->SuppressImportDialogs();

		SoundWave = ImportObject<USoundWave>(Outer, AssetName, Flags, *FilePath, nullptr, SoundWaveFactory);

#elif 0
		// Replicate what the Factory is doing without some extras like cue assets, channel number conversion,
		// AmbiX or FuMa tags for now.
		//
		// Note: This branch is currently disabled as it seems no longer possible to setup these SoundWave assets
		// at runtime. This uses code that supposedly worked well in the past so it's not yet clear whether the
		// issues are with this approach or the audio code downstream itself...
		// For the moment, we'll just support audio import with the Editor, and not at Runtime.
		// When enabling this, note that you'd also need a dependency on the AudioMixer module.
		//
		// References:
		//  - USoundFactory::FactoryCreateBinary
		//  - USoundFactory::CreateObject
		//  - UInterchangeSoundWaveFactory::ImportAsset_Async

		// Read file into a byte buffer
		TArray<uint8> FileBytes;
		if (!FFileHelper::LoadFileToArray(FileBytes, *FilePath))
		{
			USD_LOG_WARNING(TEXT("Failed to load audio file '%s' into array"), *FilePath);
			return nullptr;
		}

		// I don't quite understand why, but having extra capacity in this buffer will cause
		// Audio::SoundFileUtils::ConvertAudioToWav below to trigger some out of bounds access checks.
		// The code path via the SoundFactory "avoids" this problem (unintentionally?) because it reads
		// the file bytes into a TArray64<uint8>, and then memcopies into a TArray<uint8> with the exact
		// size and capacity it needs (ending up with no extra slack)
		FileBytes.Shrink();

		// If it's another file format other than .wav, try converting into wav data
		FString Extension = FPaths::GetExtension(FilePath);
		if (Extension.ToLower() != TEXT("wav"))
		{
			// Convert audio data to a wav file in memory
			TArray<uint8> RawWaveData;
			if (Audio::SoundFileUtils::ConvertAudioToWav(FileBytes, RawWaveData))
			{
				Swap(FileBytes, RawWaveData);
			}
			else
			{
				USD_LOG_WARNING(TEXT("Failed to convert audio file format '%s' to wav"), *Extension);
				return nullptr;
			}
		}

		// Read WAV info about file
		FWaveModInfo WaveInfo;
		FString ErrorMessage;
		if (!WaveInfo.ReadWaveInfo(FileBytes.GetData(), FileBytes.Num(), &ErrorMessage))
		{
			USD_LOG_WARNING(TEXT("Failed to read audio info from file '%s': %s"), *FilePath, *ErrorMessage);
			return nullptr;
		}

		// If we need to change bit depth, or if the wav internal format is not something we know try converting
		if (*WaveInfo.pBitsPerSample != 16 || !WaveInfo.IsFormatSupported())
		{
			const uint32 OrigNumSamples = Audio::SoundFileUtils::GetNumSamples(FileBytes);

			TArray<uint8> ConvertedRawWaveData;

			// Attempt to convert to 16 bit audio
			if (Audio::SoundFileUtils::ConvertAudioToWav(FileBytes, ConvertedRawWaveData))
			{
				WaveInfo = FWaveModInfo();
				if (!WaveInfo.ReadWaveInfo(ConvertedRawWaveData.GetData(), ConvertedRawWaveData.Num(), &ErrorMessage))
				{
					USD_LOG_WARNING(TEXT("Failed to convert audio data from '%s' into a supported wav format: %s"), *FilePath, *ErrorMessage);
					return nullptr;
				}

				// Sanity check that the same number of samples exist in the converted file as the original
				const uint32 ConvertedNumSamples = WaveInfo.GetNumSamples();
				ensure(ConvertedNumSamples == OrigNumSamples);

				Swap(FileBytes, ConvertedRawWaveData);
			}
		}

		if (*WaveInfo.pBitsPerSample != 16)
		{
			USD_LOG_WARNING(TEXT("Failed to parse audio as .wav data from file '%s' was not 16-bit!"), *FilePath);
			return nullptr;
		}
		check(*WaveInfo.pBitsPerSample == 16);

		int32 ChannelCount = (int32)*WaveInfo.pChannels;
		check(ChannelCount > 0);

		SoundWave = NewObject<USoundWave>(Outer, AssetName, Flags);

		int32 SizeOfSample = (*WaveInfo.pBitsPerSample) / 8;
		int32 NumSamples = WaveInfo.SampleDataSize / SizeOfSample;
		int32 NumFrames = NumSamples / ChannelCount;
		int32 SamplesPerSec = (int32)*WaveInfo.pSamplesPerSec;

		// Copy the uncompressed data directly into the SoundWave
		// Note: This currently doesn't work for some reason: We end up with some crash downstream when
		// creating the audio buffer on the audio playing thread
		SoundWave->RawPCMDataSize = FileBytes.Num();
		SoundWave->RawPCMData = (uint8*)FMemory::Malloc(SoundWave->RawPCMDataSize);
		FMemory::Memcpy(SoundWave->RawPCMData, FileBytes.GetData(), SoundWave->RawPCMDataSize);

		SoundWave->Duration = (float)NumFrames / SamplesPerSec;
		SoundWave->SetImportedSampleRate(SamplesPerSec);
		SoundWave->SetSampleRate(SamplesPerSec);
		SoundWave->NumChannels = ChannelCount;
		SoundWave->TotalSamples = SamplesPerSec * SoundWave->Duration;

		SoundWave->PostLoad();
#endif	  // WITH_EDITOR

		return SoundWave;
	}
}

void FUsdMediaSpatialAudioTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdMediaSpatialAudioTranslator::CreateAssets);

	using namespace UE::UsdMediaSpatialAudioTranslator::Private;

	if (!Context->UsdAssetCache || !Context->PrimLinkCache)
	{
		return;
	}

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	if (!Context->bAllowParsingSounds)
	{
		return;
	}

	FScopedUsdAllocs Allocs;

	const FString PrimPathString = PrimPath.GetString();
	pxr::UsdPrim UsdPrim = GetPrim();
	pxr::UsdMediaSpatialAudio UsdAudio{UsdPrim};
	if (!UsdAudio)
	{
		return;
	}
	pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();

	pxr::UsdAttribute FilePathAttr = UsdAudio.GetFilePathAttr();
	FString ResolvedAudioPath = UsdUtils::GetResolvedAssetPath(FilePathAttr, pxr::UsdTimeCode::Default());
	if (!FPaths::FileExists(ResolvedAudioPath))
	{
		USD_LOG_USERWARNING(FText::Format(
			LOCTEXT("MissingAudioFile", "Failed to resolve audio file at path '{0}' from prim '{1}'"),
			FText::FromString(ResolvedAudioPath),
			FText::FromString(PrimPathString)
		));
		return;
	}

	FString DesiredName = FPaths::GetBaseFilename(ResolvedAudioPath);
	FString PrefixedAudioHash = UsdUtils::GetAssetHashPrefix(UsdPrim, Context->bShareAssetsForIdenticalPrims)
								+ LexToString(FMD5Hash::HashFile(*ResolvedAudioPath));

	USoundWave* SoundWave = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<USoundWave>(
		PrefixedAudioHash,
		DesiredName,
		Context->ObjectFlags,
		[&ResolvedAudioPath](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse) -> UObject*
		{
			return UE::UsdMediaSpatialAudioTranslator::Private::CreateSoundWave(	//
				ResolvedAudioPath,
				SanitizedName,
				Outer,
				FlagsToUse
			);
		}
	);

	if (SoundWave)
	{
		Context->PrimLinkCache->LinkAssetToPrim(PrimPath, SoundWave);

		if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdAssetUserData>(SoundWave))
		{
			UserData->PrimPaths.AddUnique(PrimPathString);

			if (Context->MetadataOptions.bCollectMetadata)
			{
				UsdToUnreal::ConvertMetadata(
					UsdPrim,
					UserData,
					Context->MetadataOptions.BlockedPrefixFilters,
					Context->MetadataOptions.bInvertFilters,
					Context->MetadataOptions.bCollectFromEntireSubtrees
				);
			}
			else
			{
				UserData->StageIdentifierToMetadata.Remove(UsdToUnreal::ConvertString(Stage->GetRootLayer()->GetIdentifier()));
			}
		}
	}
}

USceneComponent* FUsdMediaSpatialAudioTranslator::CreateComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdMediaSpatialAudioTranslator::CreateComponents);

	USceneComponent* SceneComponent = nullptr;

	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		if (Context->bAllowParsingSounds)
		{
			const bool bNeedsActor = true;
			SceneComponent = CreateComponentsEx({UAudioComponent::StaticClass()}, bNeedsActor);
		}
	}
	else
	{
		SceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	UpdateComponents(SceneComponent);

	return SceneComponent;
}

void FUsdMediaSpatialAudioTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	// Note how we don't even set the audio on the component here: All of our audio playback is done via
	// the Sequencer. Essentially the audio actor/component are exclusively used for their transform, in case
	// the Sequencer is playing spatial audio.
	//
	// We exclusively use the Sequencer for playback and properties for a few reasons:
	//  - We must use the Sequencer for audio in the first place anyway, as there is no easy way of scrubbing/restarting
	//    audio playback directly via the audio component or Time attribute animation. As I understand it, the audio component
	//    is intentionally very "simple", and is meant to be driven by blueprint or just play its audio once at spawn and be
	//    disabled (e.g. for an explosion sound effect). It's not meant to be a full "media player";
	//  - It's much easier to manipulate the attributes like startTime/endTime, mediaOffset, volume and looping via
	//    the Sequencer from the user standpoint, as they're all right on the section and you can just click and drag keyframes,
	//    section boundaries, the entire section, etc.;
	//  - Putting all the attributes on the Sequencer section means we can reuse the same audio asset for different
	//    attribute configurations. For example, if we wanted to play the audio directly via the component, we'd have to store whether
	//    it is looping or not within the asset itself. This means that playing two versions of the audio simultaneously,
	//    one looping and one not, would have needed two separate copies of the same audio asset...
	//  - If we had done all of the above and *also* placed the audio and whatever properties we could on the component, it would
	//    have just been more confusing, as a user wouldn't really know what was actually driving the audio or not

	Super::UpdateComponents(SceneComponent);
}

bool FUsdMediaSpatialAudioTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	return false;
}

bool FUsdMediaSpatialAudioTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return false;
}

TSet<UE::FSdfPath> FUsdMediaSpatialAudioTranslator::CollectAuxiliaryPrims() const
{
	return {};
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
