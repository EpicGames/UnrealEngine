// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDSchemasModule.h"

#include "Objects/USDSchemaTranslator.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "AnalyticsEventAttribute.h"
#include "Modules/ModuleManager.h"

#if USE_USD_SDK
#include "Custom/MaterialXUSDShadeMaterialTranslator.h"
#include "USDGeomCameraTranslator.h"
#include "USDGeometryCacheTranslator.h"
#include "USDGeomMeshTranslator.h"
#include "USDGeomPointInstancerTranslator.h"
#include "USDGeomPrimitiveTranslator.h"
#include "USDGeomXformableTranslator.h"
#include "USDGroomTranslator.h"
#include "USDLuxLightTranslator.h"
#include "USDMediaSpatialAudioTranslator.h"
#include "USDShadeConversion.h"
#include "USDShadeMaterialTranslator.h"
#include "USDSkelSkeletonTranslator.h"
#include "USDVolVolumeTranslator.h"

#include "USDIncludesStart.h"
#include "pxr/usd/kind/registry.h"
#include "pxr/usd/usd/primRange.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

class FUsdSchemasModule : public IUsdSchemasModule
{
public:
	virtual void StartupModule() override
	{
#if USE_USD_SDK
		LLM_SCOPE_BYTAG(Usd);

		FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();

		// Register the default translators
		TranslatorHandles = {
			Registry.Register<FUsdGeomCameraTranslator>(TEXT("UsdGeomCamera")),
			Registry.Register<FUsdGeomMeshTranslator>(TEXT("UsdGeomMesh")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomCapsule")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomCapsule_1")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomCone")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomCube")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomCylinder")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomCylinder_1")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomPlane")),
			Registry.Register<FUsdGeomPrimitiveTranslator>(TEXT("UsdGeomSphere")),
			Registry.Register<FUsdGeomPointInstancerTranslator>(TEXT("UsdGeomPointInstancer")),
			Registry.Register<FUsdGeomXformableTranslator>(TEXT("UsdGeomXformable")),
			Registry.Register<FUsdShadeMaterialTranslator>(TEXT("UsdShadeMaterial")),
			Registry.Register<FUsdLuxLightTranslator>(TEXT("UsdLuxBoundableLightBase")),
			Registry.Register<FUsdLuxLightTranslator>(TEXT("UsdLuxNonboundableLightBase")),
			Registry.Register<FUsdVolVolumeTranslator>(TEXT("UsdVolVolume"))
		};

#if WITH_EDITOR
		UsdUnreal::MaterialUtils::RegisterRenderContext(UnrealIdentifiers::MaterialXRenderContext);
		TranslatorHandles.Add(Registry.Register<FMaterialXUsdShadeMaterialTranslator>(TEXT("UsdShadeMaterial")));

		// Creating skeletal meshes technically works in Standalone mode, but by checking for this we artificially block it
		// to not confuse users as to why it doesn't work at runtime. Not registering the actual translators lets the inner meshes get parsed as
		// static meshes, at least.
		if (GIsEditor)
		{
			TranslatorHandles.Append(
				{Registry.Register<FUsdSkelSkeletonTranslator>(TEXT("UsdSkelSkeleton")),
				 Registry.Register<FUsdGroomTranslator>(TEXT("UsdGeomXformable")),
				 // The GeometryCacheTranslator also works on UsdGeomXformable through the GroomTranslator
				 Registry.Register<FUsdGeometryCacheTranslator>(TEXT("UsdGeomMesh")),
				 // It doesn't seem possible to create SoundWave assets at runtime at the moment, for whatever reason
				 Registry.Register<FUsdMediaSpatialAudioTranslator>(TEXT("UsdMediaSpatialAudio"))}
			);
		}
#endif	  // WITH_EDITOR

		Registry.ResetExternalTranslatorCount();

#endif	  // #if USE_USD_SDK
	}

	virtual void ShutdownModule() override
	{
		FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();

		for (const FRegisteredSchemaTranslatorHandle& TranslatorHandle : TranslatorHandles)
		{
			Registry.Unregister(TranslatorHandle);
		}

#if USE_USD_SDK && WITH_EDITOR
		UsdUnreal::MaterialUtils::UnregisterRenderContext(UnrealIdentifiers::MaterialXRenderContext);
#endif	  // WITH_EDITOR
	}

	virtual FUsdSchemaTranslatorRegistry& GetTranslatorRegistry() override
	{
		return FUsdSchemaTranslatorRegistry::Get();
	}

protected:
	TArray<FRegisteredSchemaTranslatorHandle> TranslatorHandles;
};

void UsdUnreal::Analytics::CollectSchemaAnalytics(const UE::FUsdStage& Stage, const FString& EventName)
{
#if USE_USD_SDK
	TArray<FAnalyticsEventAttribute> Attributes;

	FUsdSchemaTranslatorRegistry& Registry = FUsdSchemaTranslatorRegistry::Get();
	int32 SchemaTranslatorCount = Registry.GetExternalSchemaTranslatorCount();
	if (SchemaTranslatorCount > 0)
	{
		Attributes.Emplace(TEXT("CustomSchemaTranslatorCount"), SchemaTranslatorCount);
	}

	UsdUtils::CollectSchemaAnalytics(Stage, Attributes);

	if (Attributes.Num() > 0)
	{
		IUsdClassesModule::SendAnalytics(MoveTemp(Attributes), FString::Printf(TEXT("%s.CustomSchemaCount"), *EventName));
	}
#endif	  // USE_USD_SDK
}

IMPLEMENT_MODULE_USD(FUsdSchemasModule, USDSchemas);
