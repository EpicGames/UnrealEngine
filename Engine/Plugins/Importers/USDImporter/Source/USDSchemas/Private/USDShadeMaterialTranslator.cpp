// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDShadeMaterialTranslator.h"

#include "MeshTranslationImpl.h"
#include "Objects/USDPrimLinkCache.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDProjectSettings.h"
#include "USDShadeConversion.h"
#include "USDTranslatorUtils.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"

#include "EditorFramework/AssetImportData.h"
#include "Engine/Level.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialShared.h"
#include "Misc/SecureHash.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/base/tf/token.h"
#include "pxr/usd/usdShade/material.h"
#include "pxr/usd/usdShade/tokens.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDShadeMaterialTranslator"

namespace UE::UsdShadeTranslator::Private
{
	void RecursiveUpgradeMaterialsAndTexturesToVT(
		const TSet<UTexture*>& TexturesToUpgrade,
		const TSharedRef<FUsdSchemaTranslationContext>& Context,
		TSet<UMaterialInterface*>& VisitedMaterials,
		TSet<UMaterialInterface*>& NewMaterials
	)
	{
		if (!Context->UsdAssetCache || !Context->PrimLinkCache)
		{
			return;
		}

		for (UTexture* Texture : TexturesToUpgrade)
		{
			if (Texture->VirtualTextureStreaming)
			{
				continue;
			}

			USD_LOG_INFO(TEXT("Upgrading texture '%s' to VT as it is used by a material that must be VT"), *Texture->GetName());
			FPropertyChangedEvent PropertyChangeEvent(
				UTexture::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexture, VirtualTextureStreaming))
			);
			Texture->Modify();
			Texture->VirtualTextureStreaming = true;

#if WITH_EDITOR
			Texture->PostEditChangeProperty(PropertyChangeEvent);
#endif	  // WITH_EDITOR

			// Now that our texture is VT, all materials that use the texture must be VT
			if (const TSet<UMaterialInterface*>* UserMaterials = Context->TextureToUserMaterials.Find(Texture))
			{
				for (UMaterialInterface* UserMaterial : *UserMaterials)
				{
					if (VisitedMaterials.Contains(UserMaterial))
					{
						continue;
					}

					if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(UserMaterial))
					{
						// Important to not use GetBaseMaterial() here because if our parent is the translucent we'll
						// get the reference UsdPreviewSurface instead, as that is also *its* reference
						UMaterialInterface* ReferenceMaterial = MaterialInstance->Parent.Get();
						UMaterialInterface* ReferenceMaterialVT = Cast<UMaterialInterface>(
							UsdUnreal::MaterialUtils::GetVTVersionOfReferencePreviewSurfaceMaterial(ReferenceMaterial).TryLoad()
						);
						if (ReferenceMaterial == ReferenceMaterialVT)
						{
							// Material is already VT, we're good
							continue;
						}

						// Visit it before we start recursing. We need this because we must convert textures to VT
						// before materials (or else we get a warning) but we'll only actually swap the reference material
						// at the end of this scope
						VisitedMaterials.Add(UserMaterial);

						// If we're going to update this material to VT, all of *its* textures need to be VT too
						TSet<UTexture*> OtherUsedTextures;
						for (const FTextureParameterValue& TextureValue : MaterialInstance->TextureParameterValues)
						{
							if (UTexture* OtherTexture = TextureValue.ParameterValue)
							{
								OtherUsedTextures.Add(OtherTexture);
							}
						}

						RecursiveUpgradeMaterialsAndTexturesToVT(OtherUsedTextures, Context, VisitedMaterials, NewMaterials);

						// We can't blindly recreate all component render states when a level is being added, because
						// we may end up first creating render states for some components, and UWorld::AddToWorld
						// calls FScene::AddPrimitive which expects the component to not have primitives yet
						FMaterialUpdateContext::EOptions::Type Options = FMaterialUpdateContext::EOptions::Default;
						if (Context->Level->bIsAssociatingLevel)
						{
							Options = (FMaterialUpdateContext::EOptions::Type)(Options & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
						}

						USD_LOG_INFO(
							TEXT("Upgrading material instance '%s' to having a VT reference as texture '%s' requires it"),
							*MaterialInstance->GetName(),
							*Texture->GetName()
						);

#if WITH_EDITOR
						UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(MaterialInstance);
						if (GIsEditor && MIC)
						{
							FMaterialUpdateContext UpdateContext(Options, GMaxRHIShaderPlatform);
							UpdateContext.AddMaterialInstance(MIC);
							MIC->PreEditChange(nullptr);
							MIC->SetParentEditorOnly(ReferenceMaterialVT);
							MIC->PostEditChange();
						}
						else
#endif	  // WITH_EDITOR
		  // Don't spell out MID directly, as at runtime we may be trying to upgrade a packaged MIC
							if (UMaterialInstance* MI = Cast<UMaterialInstance>(UserMaterial))
							{
								TArray<UE::FSdfPath> PrimsForAsset = Context->PrimLinkCache->GetPrimsForAsset(MI);
								const FString Hash = Context->UsdAssetCache->GetHashForAsset(MI);

								// For MID we can't swap the reference material, so we need to remove the old material from the cache,
								// create a brand new one, copy the overrides and then add that back in its place

								TStrongObjectPtr<UMaterialInstance> MIDPin{MI};
								FSoftObjectPath OldMIDPath = Context->UsdAssetCache->StopTrackingAsset(Hash);
								ensure(OldMIDPath == FSoftObjectPath{MI});

								bool bCreatedNew = false;
								UMaterialInstanceDynamic* NewMID = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<UMaterialInstanceDynamic>(
									Hash,
									MI->GetName(),
									MI->GetFlags() | RF_Transient,	  // We never want MIDs to become assets in the content browser
									[ReferenceMaterialVT](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
									{
										UMaterialInstanceDynamic* CreatedMID = UMaterialInstanceDynamic::Create(
											ReferenceMaterialVT,
											Outer,
											SanitizedName
										);
										CreatedMID->ClearFlags(CreatedMID->GetFlags());
										CreatedMID->SetFlags(FlagsToUse);
										return CreatedMID;
									},
									&bCreatedNew
								);
								if (!ensure(bCreatedNew && NewMID))
								{
									continue;
								}

								NewMID->CopyParameterOverrides(MI);

								UUsdMaterialAssetUserData* OldUserData = UserMaterial->GetAssetUserData<UUsdMaterialAssetUserData>();
								if (OldUserData)
								{
									UUsdMaterialAssetUserData* NewUserData = DuplicateObject(OldUserData, NewMID);
									NewMID->AddAssetUserData(NewUserData);
								}

								for (const UE::FSdfPath& PrimPath : PrimsForAsset)
								{
									Context->PrimLinkCache->LinkAssetToPrim(PrimPath, NewMID);
								}
								NewMaterials.Add(NewMID);

								MI->MarkAsGarbage();
							}
							else
							{
								// This should never happen
								ensure(false);
							}
					}
				}
			}
		}
	}

	void UpgradeMaterialsAndTexturesToVT(TSet<UTexture*> TexturesToUpgrade, TSharedRef<FUsdSchemaTranslationContext>& Context)
	{
		TSet<UMaterialInterface*> VisitedMaterials;
		TSet<UMaterialInterface*> NewMaterials;
		RecursiveUpgradeMaterialsAndTexturesToVT(TexturesToUpgrade, Context, VisitedMaterials, NewMaterials);

		// When we "visit" a MID we'll create a brand new instance of it and discard the old one, so let's drop the old ones
		// from Context->TextureToUserMaterials too
		for (UMaterialInterface* Material : VisitedMaterials)
		{
			if (UMaterialInstanceDynamic* OldMID = Cast<UMaterialInstanceDynamic>(Material))
			{
				for (TPair<UTexture*, TSet<UMaterialInterface*>>& Pair : Context->TextureToUserMaterials)
				{
					Pair.Value.Remove(Material);
				}
			}
		}

		// Additionally now we need to add those new MIDs we created back into Context->TextureToUserMaterials
		for (UMaterialInterface* Material : NewMaterials)
		{
			UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(Material);
			if (ensure(MID))
			{
				for (const FTextureParameterValue& TextureValue : MID->TextureParameterValues)
				{
					if (UTexture* Texture = TextureValue.ParameterValue)
					{
						Context->TextureToUserMaterials.FindOrAdd(Texture).Add(MID);
					}
				}
			}
		}
	}

	// We need to hash the reference material that we'll use, so that if this is changed we regenerate a new instance.
	// However, unlike for displayColor materials, we can't really know *which* reference material we'll end up using
	// until after we've already created it (which doesn't sound like it makes any sense but it's part of why we have those
	// VT and double-sided "upgrade" mechanisms).
	//
	// If all we want is a hash, the solution can be simple though: Hash them all. Yea we may end up unnecessarily regenerating
	// materials sometimes but changing the reference materials on the project settings should be rare.
	void HashPreviewSurfaceReferences(FSHA1& InOutHash)
	{
		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (!Settings)
		{
			return;
		}

		TArray<const FSoftObjectPath*> ReferenceMaterials = {
			&Settings->ReferencePreviewSurfaceMaterial,
			&Settings->ReferencePreviewSurfaceTranslucentMaterial,
			&Settings->ReferencePreviewSurfaceTwoSidedMaterial,
			&Settings->ReferencePreviewSurfaceTranslucentTwoSidedMaterial,
			&Settings->ReferencePreviewSurfaceVTMaterial,
			&Settings->ReferencePreviewSurfaceTranslucentVTMaterial,
			&Settings->ReferencePreviewSurfaceTwoSidedVTMaterial,
			&Settings->ReferencePreviewSurfaceTranslucentTwoSidedVTMaterial
		};

		for (const FSoftObjectPath* ReferencePath : ReferenceMaterials)
		{
			if (!ReferencePath)
			{
				continue;
			}

			FString ReferencePathStr = ReferencePath->ToString();
			InOutHash.UpdateWithString(*ReferencePathStr, ReferencePathStr.Len());
		}
	}
}	 // namespace UE::UsdShadeTranslator::Private

void FUsdShadeMaterialTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdShadeMaterialTranslator::CreateAssets);

	pxr::UsdShadeMaterial ShadeMaterial(GetPrim());

	if (!ShadeMaterial)
	{
		return;
	}

	if (Context->bTranslateOnlyUsedMaterials && Context->UsdInfoCache)
	{
		if (!Context->UsdInfoCache->IsMaterialUsed(PrimPath))
		{
			return;
		}
	}

	const pxr::TfToken RenderContextToken = Context->RenderContext.IsNone() ? pxr::UsdShadeTokens->universalRenderContext
																			: UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();

	// If this material has a valid surface output for the 'unreal' render context and we're using it, don't bother
	// generating any new UMaterialInterface asset because when resolving material assignments for this material
	// all consumers will just use the referenced UAsset anyway
	if (RenderContextToken == UnrealIdentifiers::Unreal)
	{
		TOptional<FString> UnrealMaterial = UsdUtils::GetUnrealSurfaceOutput(ShadeMaterial.GetPrim());
		if (UnrealMaterial.IsSet())
		{
			UObject* Object = FSoftObjectPath(UnrealMaterial.GetValue()).TryLoad();
			if (!Object)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("MissingUnrealMaterial", "Failed to find the Unreal material '{0}' referenced by material prim '{1}'."),
					FText::FromString(UnrealMaterial.GetValue()),
					FText::FromString(PrimPath.GetString())
				));
			}

			return;
		}
	}

	FString MaterialHash;
	{
		FSHAHash OutHash;

		FSHA1 SHA1;

		UsdUtils::HashShadeMaterial(ShadeMaterial, SHA1, RenderContextToken);
		UE::UsdShadeTranslator::Private::HashPreviewSurfaceReferences(SHA1);

		SHA1.Final();
		SHA1.GetHash(&OutHash.Hash[0]);
		MaterialHash = OutHash.ToString();
	}
	const FString PrefixedMaterialHash = UsdUtils::GetAssetHashPrefix(GetPrim(), Context->bShareAssetsForIdenticalPrims) + MaterialHash;

	const FString PrimPathString = PrimPath.GetString();
	const FString DesiredName = FPaths::GetBaseFilename(PrimPathString);

	const bool bIsMaterialTranslucent = UsdUtils::IsMaterialTranslucent(ShadeMaterial);

	UMaterialInterface* ConvertedMaterial = nullptr;

	bool bCreatedNew = false;
#if WITH_EDITOR
	if (GIsEditor)
	{
		UMaterialInstanceConstant* MIC = Context->UsdAssetCache->GetOrCreateCachedAsset<UMaterialInstanceConstant>(
			PrefixedMaterialHash,	 //
			DesiredName,
			Context->ObjectFlags,
			&bCreatedNew
		);
		ConvertedMaterial = MIC;

		if (bCreatedNew)
		{
			const bool bSuccess = UsdToUnreal::ConvertMaterial(
				ShadeMaterial,
				*MIC,
				Context->UsdAssetCache.Get(),
				*Context->RenderContext.ToString(),
				Context->bShareAssetsForIdenticalPrims
			);
			if (!bSuccess)
			{
				UsdUnreal::TranslatorUtils::AbandonFailedAsset(MIC, Context->UsdAssetCache.Get(), Context->PrimLinkCache);
				return;
			}

			TSet<UTexture*> VTTextures;
			TSet<UTexture*> NonVTTextures;
			for (const FTextureParameterValue& TextureValue : MIC->TextureParameterValues)
			{
				if (UTexture* Texture = TextureValue.ParameterValue)
				{
					if (Texture->VirtualTextureStreaming)
					{
						UsdUtils::NotifyIfVirtualTexturesNeeded(Texture);
						VTTextures.Add(Texture);
					}
					else
					{
						NonVTTextures.Add(Texture);
					}
				}
			}

			// Our VT material only has VT texture samplers, so *all* of its textures must be VT
			if (VTTextures.Num() && NonVTTextures.Num())
			{
				UE::UsdShadeTranslator::Private::UpgradeMaterialsAndTexturesToVT(NonVTTextures, Context);
			}

			EUsdReferenceMaterialProperties Properties = EUsdReferenceMaterialProperties::None;
			if (bIsMaterialTranslucent)
			{
				Properties |= EUsdReferenceMaterialProperties::Translucent;
			}
			if (VTTextures.Num() > 0)
			{
				Properties |= EUsdReferenceMaterialProperties::VT;
			}
			UMaterialInterface* ReferenceMaterial = Cast<UMaterialInterface>(
				UsdUnreal::MaterialUtils::GetReferencePreviewSurfaceMaterial(Properties).TryLoad()
			);

			if (ensure(ReferenceMaterial))
			{
				MIC->SetParentEditorOnly(ReferenceMaterial);

				// We can't blindly recreate all component render states when a level is being added, because we may end up first creating
				// render states for some components, and UWorld::AddToWorld calls FScene::AddPrimitive which expects the component to not have
				// primitives yet
				FMaterialUpdateContext::EOptions::Type Options = FMaterialUpdateContext::EOptions::Default;
				if (Context->Level && Context->Level->bIsAssociatingLevel)
				{
					Options = (FMaterialUpdateContext::EOptions::Type)(Options & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
				}

				FMaterialUpdateContext UpdateContext(Options, GMaxRHIShaderPlatform);
				UpdateContext.AddMaterialInstance(MIC);
				MIC->PreEditChange(nullptr);
				MIC->PostEditChange();

				for (UTexture* Texture : VTTextures.Union(NonVTTextures))
				{
					Context->TextureToUserMaterials.FindOrAdd(Texture).Add(MIC);
				}
			}
		}
	}
	else
#endif	  // WITH_EDITOR
	{
		// Note how we're requesting just a UMaterialInstance here, instead of spelling out the MID. This because
		// if we're a runtime we may have a cooked MIC assigned to this hash, and in that case we want to use it
		// instead of overwriting it with a MID. Our creation func will ensure we create a MID as a fallback anyway
		UMaterialInstance* MI = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<UMaterialInstance>(
			PrefixedMaterialHash,
			DesiredName,
			Context->ObjectFlags | RF_Transient,	// We never want MIDs to become assets in the content browser
			[bIsMaterialTranslucent](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
			{
				EUsdReferenceMaterialProperties Properties = EUsdReferenceMaterialProperties::None;
				if (bIsMaterialTranslucent)
				{
					Properties |= EUsdReferenceMaterialProperties::Translucent;
				}
				UMaterialInterface* ReferenceMaterial = Cast<UMaterialInterface>(
					UsdUnreal::MaterialUtils::GetReferencePreviewSurfaceMaterial(Properties).TryLoad()
				);

				UMaterialInstanceDynamic* CreatedMID = UMaterialInstanceDynamic::Create(ReferenceMaterial, Outer, SanitizedName);
				CreatedMID->ClearFlags(CreatedMID->GetFlags());
				CreatedMID->SetFlags(FlagsToUse);
				return CreatedMID;
			},
			&bCreatedNew
		);
		ConvertedMaterial = MI;

		if (bCreatedNew)
		{
			const bool bSuccess = UsdToUnreal::ConvertMaterial(
				ShadeMaterial,
				*MI,
				Context->UsdAssetCache.Get(),
				*Context->RenderContext.ToString(),
				Context->bShareAssetsForIdenticalPrims
			);
			if (!bSuccess)
			{
				UsdUnreal::TranslatorUtils::AbandonFailedAsset(MI, Context->UsdAssetCache.Get(), Context->PrimLinkCache);
				return;
			}

			TSet<UTexture*> VTTextures;
			TSet<UTexture*> NonVTTextures;
			for (const FTextureParameterValue& TextureValue : MI->TextureParameterValues)
			{
				if (UTexture* Texture = TextureValue.ParameterValue)
				{
					if (Texture->VirtualTextureStreaming)
					{
						VTTextures.Add(Texture);
					}
					else
					{
						NonVTTextures.Add(Texture);
					}
				}
			}

			// We must stash our material and textures *before* we call UpgradeMaterialsAndTexturesToVT, as that
			// is what will actually swap our reference with a VT one if needed
			if (Context->PrimLinkCache)
			{
				Context->PrimLinkCache->LinkAssetToPrim(PrimPath, MI);
			}
			for (UTexture* Texture : VTTextures.Union(NonVTTextures))
			{
				Context->TextureToUserMaterials.FindOrAdd(Texture).Add(MI);
			}

			// Our VT material only has VT texture samplers, so *all* of its textures must be VT
			if (VTTextures.Num() && NonVTTextures.Num())
			{
				UE::UsdShadeTranslator::Private::UpgradeMaterialsAndTexturesToVT(NonVTTextures, Context);
			}

			// We must go through the cache to fetch our result material here as UpgradeMaterialsAndTexturesToVT
			// may have created a new MID for this material with a VT reference
			ConvertedMaterial = Context->UsdAssetCache->GetCachedAsset<UMaterialInterface>(PrefixedMaterialHash);
		}
	}

	PostImportMaterial(PrefixedMaterialHash, ConvertedMaterial);
}

bool FUsdShadeMaterialTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	return false;
}

bool FUsdShadeMaterialTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return false;
}

void FUsdShadeMaterialTranslator::PostImportMaterial(const FString& PrefixedMaterialHash, UMaterialInterface* ImportedMaterial)
{
	if (!ImportedMaterial || !Context->PrimLinkCache || !Context->UsdAssetCache)
	{
		return;
	}

	if (UUsdMaterialAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdMaterialAssetUserData>(ImportedMaterial))
	{
		UserData->PrimPaths.AddUnique(PrimPath.GetString());

		if (Context->MetadataOptions.bCollectMetadata)
		{
			UsdToUnreal::ConvertMetadata(
				GetPrim(),
				UserData,
				Context->MetadataOptions.BlockedPrefixFilters,
				Context->MetadataOptions.bInvertFilters,
				Context->MetadataOptions.bCollectFromEntireSubtrees
			);
		}
		else
		{
			// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the AssetUserData
			UserData->StageIdentifierToMetadata.Remove(GetPrim().GetStage().GetRootLayer().GetIdentifier());
		}
	}

	// Note that this needs to run even if we found this material in the asset cache already, otherwise we won't
	// re-register the prim asset links when we reload a stage
	Context->PrimLinkCache->LinkAssetToPrim(PrimPath, ImportedMaterial);

	// Also link the textures to the same material prim. Our textures should all come from USDShadeConversion.cpp or
	// MaterialX or MDL translators, so they should already be tracked by the same asset cache the material is tracked by.
	// This is important because it lets the stage actor drop its references to old unused textures in the
	// asset cache if they aren't being used by any other material
	TSet<UObject*> Dependencies = IUsdClassesModule::GetAssetDependencies(ImportedMaterial);
	for (UObject* Object : Dependencies)
	{
		if (UTexture* Texture = Cast<UTexture>(Object))
		{
			// We may be reusing a material from the asset cache that has textures fully unrelated to USD, which
			// we shouldn't interact with
			const bool bIsTrackedByCache = Context->UsdAssetCache->IsAssetTrackedByCache(Texture->GetPathName());
			if (bIsTrackedByCache)
			{
				Context->UsdAssetCache->TouchAssetPath(Texture);
				Context->PrimLinkCache->LinkAssetToPrim(PrimPath, Texture);

				if (UUsdAssetUserData* TextureUserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Texture))
				{
					TextureUserData->PrimPaths.AddUnique(PrimPath.GetString());
				}
			}
		}
	}
}

TSet<UE::FSdfPath> FUsdShadeMaterialTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	TSet<UE::FSdfPath> Result;
	{
		TFunction<void(const pxr::UsdShadeInput&)> TraverseShadeInput;
		TraverseShadeInput = [&TraverseShadeInput, &Result](const pxr::UsdShadeInput& ShadeInput)
		{
			if (!ShadeInput)
			{
				return;
			}

			pxr::UsdShadeConnectableAPI Source;
			pxr::TfToken SourceName;
			pxr::UsdShadeAttributeType AttributeType;
			if (pxr::UsdShadeConnectableAPI::GetConnectedSource(ShadeInput.GetAttr(), &Source, &SourceName, &AttributeType))
			{
				pxr::UsdPrim ConnectedPrim = Source.GetPrim();
				UE::FSdfPath ConnectedPrimPath = UE::FSdfPath{ConnectedPrim.GetPrimPath()};

				if (!Result.Contains(ConnectedPrimPath))
				{
					Result.Add(ConnectedPrimPath);

					for (const pxr::UsdShadeInput& ChildInput : Source.GetInputs())
					{
						TraverseShadeInput(ChildInput);
					}
				}
			}
		};

		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim Prim = GetPrim();
		pxr::UsdShadeMaterial UsdShadeMaterial{Prim};
		if (!UsdShadeMaterial)
		{
			return {};
		}

		const pxr::TfToken RenderContextToken = Context->RenderContext.IsNone() ? pxr::UsdShadeTokens->universalRenderContext
																				: UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
		pxr::UsdShadeShader SurfaceShader = UsdShadeMaterial.ComputeSurfaceSource({RenderContextToken});
		if (!SurfaceShader)
		{
			return {};
		}

		Result.Add(UE::FSdfPath{SurfaceShader.GetPrim().GetPrimPath()});

		for (const pxr::UsdShadeInput& ShadeInput : SurfaceShader.GetInputs())
		{
			TraverseShadeInput(ShadeInput);
		}
	}
	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
