// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDSkelSkeletonTranslator.h"

#if USE_USD_SDK

#include "MeshTranslationImpl.h"
#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDGroomTranslatorUtils.h"
#include "USDIntegrationUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDSetUtils.h"
#include "USDSkeletalDataConversion.h"
#include "USDTranslatorUtils.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdSkelSkinningQuery.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "GroomComponent.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"

#if WITH_EDITOR
#include "AnimGraphNode_LiveLinkPose.h"
#include "AnimNode_LiveLinkPose.h"
#include "BlueprintCompilationManager.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "ObjectTools.h"
#include "PhysicsAssetUtils.h"
#endif	  // WITH_EDITOR

#include "USDIncludesStart.h"
#include "pxr/usd/usd/primRange.h"
#include "pxr/usd/usdGeom/mesh.h"
#include "pxr/usd/usdSkel/binding.h"
#include "pxr/usd/usdSkel/bindingAPI.h"
#include "pxr/usd/usdSkel/blendShapeQuery.h"
#include "pxr/usd/usdSkel/cache.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeletonQuery.h"
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "UsdSkelRoot"

static bool bGeneratePhysicsAssets = true;
static FAutoConsoleVariableRef CVarGeneratePhysicsAssets(
	TEXT("USD.GeneratePhysicsAssets"),
	bGeneratePhysicsAssets,
	TEXT("Whether to automatically generate and assign PhysicsAssets to generated SkeletalMeshes.")
);

namespace UsdSkelSkeletonTranslatorImpl
{
#if WITH_EDITOR
	bool ProcessMaterials(
		const pxr::UsdPrim& UsdPrim,
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& LODIndexToMaterialInfo,
		USkeletalMesh* SkeletalMesh,
		UUsdAssetCache3& AssetCache,
		FUsdPrimLinkCache& PrimLinkCache,
		float Time,
		EObjectFlags Flags,
		bool bSkeletalMeshHasMorphTargets,
		bool bShareAssetsForIdenticalPrims
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdSkelSkeletonTranslatorImpl::ProcessMaterials);

		if (!SkeletalMesh)
		{
			return false;
		}

		TArray<UMaterialInterface*> ExistingAssignments;
		for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
		{
			ExistingAssignments.Add(SkeletalMaterial.MaterialInterface);
		}

		UUsdMeshAssetUserData* UserData = SkeletalMesh->GetAssetUserData<UUsdMeshAssetUserData>();
		ensureMsgf(
			UserData,
			TEXT("Skeletal Mesh '%s' generated for prim '%s' should have an UUsdMeshAssetUserData at this point!"),
			*SkeletalMesh->GetPathName(),
			*UsdToUnreal::ConvertPath(UsdPrim.GetPrimPath())
		);

		TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolvedMaterials = MeshTranslationImpl::ResolveMaterialAssignmentInfo(
			UsdPrim,
			LODIndexToMaterialInfo,
			AssetCache,
			PrimLinkCache,
			Flags,
			bShareAssetsForIdenticalPrims
		);

		bool bMaterialsHaveChanged = false;

		const FSkeletalMeshModel* ImportedResource = SkeletalMesh->GetImportedModel();
		if (!ImportedResource)
		{
			return false;
		}

		const TIndirectArray<FSkeletalMeshLODModel>& LODModels = ImportedResource->LODModels;

		uint32 SkeletalMeshSlotIndex = 0;
		for (int32 LODIndex = 0; LODIndex < LODIndexToMaterialInfo.Num(); ++LODIndex)
		{
			const TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToMaterialInfo[LODIndex].Slots;

			// We need to fill this in with the mapping from LOD material slots (i.e. sections) to the skeletal mesh's material slots
			FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
			if (!LODInfo)
			{
				USD_LOG_ERROR(
					TEXT("When processing materials for SkeletalMesh '%s', encountered no LOD info for LOD index %d!"),
					*SkeletalMesh->GetName(),
					LODIndex
				);
				continue;
			}

			if (!LODModels.IsValidIndex(LODIndex))
			{
				return false;
			}
			const FSkeletalMeshLODModel& LODModel = LODModels[LODIndex];

			TMap<int32, int32> LODIndexToMeshIndex;

			for (int32 LODSlotIndex = 0; LODSlotIndex < LODSlots.Num(); ++LODSlotIndex, ++SkeletalMeshSlotIndex)
			{
				const UsdUtils::FUsdPrimMaterialSlot& Slot = LODSlots[LODSlotIndex];

				UMaterialInterface* Material = UMaterial::GetDefaultMaterial(MD_Surface);

				if (UMaterialInterface** FoundMaterial = ResolvedMaterials.Find(&Slot))
				{
					Material = *FoundMaterial;
				}
				else
				{
					USD_LOG_ERROR(
						TEXT("Failed to resolve material '%s' for slot '%d' of LOD '%d' for mesh '%s'"),
						*Slot.MaterialSource,
						LODSlotIndex,
						LODIndex,
						*UsdToUnreal::ConvertPath(UsdPrim.GetPath())
					);
					continue;
				}

				if (Material)
				{
					Material->SetMaterialUsage(MATUSAGE_SkeletalMesh);
					if (bSkeletalMeshHasMorphTargets)
					{
						Material->SetMaterialUsage(MATUSAGE_MorphTargets);
					}
				}

				FName MaterialSlotName = *LexToString(SkeletalMeshSlotIndex);

				// Already have a material at that skeletal mesh slot, need to reassign
				if (SkeletalMesh->GetMaterials().IsValidIndex(SkeletalMeshSlotIndex))
				{
					FSkeletalMaterial& ExistingMaterial = SkeletalMesh->GetMaterials()[SkeletalMeshSlotIndex];

					if (ExistingMaterial.MaterialInterface != Material || ExistingMaterial.MaterialSlotName != MaterialSlotName
						|| ExistingMaterial.ImportedMaterialSlotName != MaterialSlotName)
					{
						ExistingMaterial.MaterialInterface = Material;
						ExistingMaterial.MaterialSlotName = MaterialSlotName;
						ExistingMaterial.ImportedMaterialSlotName = MaterialSlotName;
						bMaterialsHaveChanged = true;
					}
				}
				// Add new material
				else
				{
					const bool bEnableShadowCasting = true;
					const bool bRecomputeTangents = false;
					SkeletalMesh->GetMaterials().Add(
						FSkeletalMaterial(Material, bEnableShadowCasting, bRecomputeTangents, MaterialSlotName, MaterialSlotName)
					);
					bMaterialsHaveChanged = true;
				}

				LODIndexToMeshIndex.Add(LODSlotIndex, SkeletalMeshSlotIndex);
			}

			// Our LOD slots from USD want to use LODSlotIndex (above) as a material index, but the SkeletalMesh
			// actual material slot order may be different as we just append all material assignments,
			// so we need to fill in LODMaterialMap which is internally used to do that mapping.
			//
			// Note that LODMaterialMap needs to match the actual list of sections on the skeletal mesh, and
			// we may end up with more (or less?) sections than we expect (e.g. if our skeleton is too large
			// the build process may create new "chunked" sections that also point at the same material slots).
			// Here we step through all sections for this LOD and add LODMaterialMap entries for the
			// relevant ones.
			TArray<int32>& LODMaterialMap = LODInfo->LODMaterialMap;
			LODMaterialMap.SetNumUninitialized(LODModel.Sections.Num());
			for (int32& Mapping : LODMaterialMap)
			{
				Mapping = INDEX_NONE;	 // Initialize map with INDEX_NONE (means no remapping for that index)
			}

			for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); ++SectionIndex)
			{
				const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

				if (int32* FoundMeshSlotIndex = LODIndexToMeshIndex.Find(Section.MaterialIndex))
				{
					LODMaterialMap[SectionIndex] = *FoundMeshSlotIndex;
				}
			}
		}

		return bMaterialsHaveChanged;
	}

	FSHAHash ComputeSHAHash(
		const TArray<FSkeletalMeshImportData>& LODIndexToSkeletalMeshImportData,
		TArray<SkeletalMeshImportData::FBone>& ImportedBones,
		UsdUtils::FBlendShapeMap* BlendShapes
	)
	{
		FSHA1 HashState;

		for (const FSkeletalMeshImportData& ImportData : LODIndexToSkeletalMeshImportData)
		{
			HashState.Update((uint8*)ImportData.Points.GetData(), ImportData.Points.Num() * ImportData.Points.GetTypeSize());
			HashState.Update((uint8*)ImportData.Wedges.GetData(), ImportData.Wedges.Num() * ImportData.Wedges.GetTypeSize());
			HashState.Update((uint8*)ImportData.Faces.GetData(), ImportData.Faces.Num() * ImportData.Faces.GetTypeSize());
			HashState.Update((uint8*)ImportData.Influences.GetData(), ImportData.Influences.Num() * ImportData.Influences.GetTypeSize());
		}

		// Hash the bones as well because it is possible for the mesh to be identical while only the bone configuration changed, and in that case we'd
		// need new skeleton and ref skeleton Maybe in the future (as a separate feature) we could split off the skeleton import so that it could vary
		// independently of the skeletal mesh
		for (const SkeletalMeshImportData::FBone& Bone : ImportedBones)
		{
			HashState.UpdateWithString(*Bone.Name, Bone.Name.Len());
			HashState.Update((uint8*)&Bone.Flags, sizeof(Bone.Flags));
			HashState.Update((uint8*)&Bone.NumChildren, sizeof(Bone.NumChildren));
			HashState.Update((uint8*)&Bone.ParentIndex, sizeof(Bone.ParentIndex));
			HashState.Update((uint8*)&Bone.BonePos, sizeof(Bone.BonePos));
		}

		if (BlendShapes)
		{
			for (const TPair<FString, UsdUtils::FUsdBlendShape>& Pair : *BlendShapes)
			{
				const UsdUtils::FUsdBlendShape& BlendShape = Pair.Value;
				HashState.UpdateWithString(*BlendShape.Name, BlendShape.Name.Len());

				HashState.Update((uint8*)BlendShape.Vertices.GetData(), BlendShape.Vertices.Num() * BlendShape.Vertices.GetTypeSize());

				for (const UsdUtils::FUsdBlendShapeInbetween& InBetween : BlendShape.Inbetweens)
				{
					HashState.UpdateWithString(*InBetween.Name, InBetween.Name.Len());
					HashState.Update((uint8*)&InBetween.InbetweenWeight, sizeof(InBetween.InbetweenWeight));
				}

				HashState.Update((uint8*)&BlendShape.bHasAuthoredTangents, sizeof(BlendShape.bHasAuthoredTangents));
			}
		}

		FSHAHash OutHash;

		HashState.Final();
		HashState.GetHash(&OutHash.Hash[0]);

		return OutHash;
	}

	FSHAHash ComputeSHAHash(
		const pxr::UsdSkelSkeletonQuery& InUsdSkeletonQuery,
		const pxr::UsdPrim& RootMotionPrim,
		const FString& SkeletalMeshHashString
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UsdSkelSkeletonTranslatorImpl::ComputeSHAHash_SkelQuery);

		FSHAHash OutHash;
		FSHA1 HashState;

		FScopedUsdAllocs Allocs;

		pxr::UsdSkelAnimQuery AnimQuery = InUsdSkeletonQuery.GetAnimQuery();
		if (!AnimQuery)
		{
			return OutHash;
		}

		pxr::UsdPrim UsdPrim = InUsdSkeletonQuery.GetPrim();
		if (!UsdPrim)
		{
			return OutHash;
		}

		pxr::UsdStageRefPtr Stage = UsdPrim.GetStage();
		if (!Stage)
		{
			return OutHash;
		}

		int32 InterpolationType = static_cast<int32>(Stage->GetInterpolationType());
		HashState.Update((uint8*)&InterpolationType, sizeof(int32));

		// Hash blend shape and joint order tokens
		TFunction<void(const pxr::VtArray<pxr::TfToken>&)> HashTokens = [&HashState](const pxr::VtArray<pxr::TfToken>& Tokens)
		{
			for (const pxr::TfToken& Token : Tokens)
			{
				const std::string& TokenString = Token.GetString();
				const char* TokenCStr = TokenString.c_str();
				HashState.Update((const uint8*)TokenCStr, sizeof(char) * TokenString.size());
			}
		};
		HashTokens(AnimQuery.GetJointOrder());
		HashTokens(AnimQuery.GetBlendShapeOrder());

		// Time samples for joint transforms
		std::vector<double> TimeData;
		AnimQuery.GetJointTransformTimeSamples(&TimeData);
		HashState.Update((uint8*)TimeData.data(), TimeData.size() * sizeof(double));

		// Joint transform values
		pxr::VtArray<pxr::GfMatrix4d> JointTransforms;
		for (double JointTimeSample : TimeData)
		{
			InUsdSkeletonQuery.ComputeJointLocalTransforms(&JointTransforms, JointTimeSample);
			HashState.Update((uint8*)JointTransforms.data(), JointTransforms.size() * sizeof(pxr::GfMatrix4d));
		}

		// restTransforms
		pxr::VtArray<pxr::GfMatrix4d> Transforms;
		const bool bAtRest = true;
		InUsdSkeletonQuery.ComputeJointLocalTransforms(&Transforms, pxr::UsdTimeCode::EarliestTime(), bAtRest);
		HashState.Update((uint8*)Transforms.data(), Transforms.size() * sizeof(pxr::GfMatrix4d));

		// bindTransforms
		InUsdSkeletonQuery.GetJointWorldBindTransforms(&Transforms);
		HashState.Update((uint8*)Transforms.data(), Transforms.size() * sizeof(pxr::GfMatrix4d));

		// Time samples for blend shape curves
		AnimQuery.GetBlendShapeWeightTimeSamples(&TimeData);
		HashState.Update((uint8*)TimeData.data(), TimeData.size() * sizeof(double));

		// Blend shape curve values
		pxr::VtArray<float> WeightsForSample;
		for (double CurveTimeSample : TimeData)
		{
			AnimQuery.ComputeBlendShapeWeights(&WeightsForSample, pxr::UsdTimeCode(CurveTimeSample));
			HashState.Update((uint8*)WeightsForSample.data(), WeightsForSample.size() * sizeof(float));
		}

		// If we're pulling root motion from anywhere, hash that too because if it changes we'll need to rebake
		// the AnimSequence asset
		if (pxr::UsdGeomXformable Xformable{RootMotionPrim})
		{
			// Hash non-animated transform too, because we'll put these directly on the components if
			// the SkelRoot/Skeleton is not animated, and we need our AnimSequence to combine nicely with
			// those
			pxr::GfMatrix4d Transform{};
			bool bResetsXformSack = false;
			Xformable.GetLocalTransformation(&Transform, &bResetsXformSack, pxr::UsdTimeCode::Default());
			HashState.Update((uint8*)Transform.data(), 16 * sizeof(double));

			std::vector<double> TimeSamples;
			Xformable.GetTimeSamples(&TimeSamples);

			for (double TimeSample : TimeSamples)
			{
				Xformable.GetLocalTransformation(&Transform, &bResetsXformSack, TimeSample);
				HashState.Update((uint8*)Transform.data(), 16 * sizeof(double));
			}

			HashState.Update((uint8*)&bResetsXformSack, sizeof(bool));
		}

		// An anim sequence matches a particular skeleton. If the skeleton is different, we'll likely
		// need a new AnimSequence, even if the SkelAnimation prim itself hashes the same. The same
		// applies to SkeletalMesh morph targets too: In USD a SkelAnimation has decoupled "blend
		// shape channel" curves that can animate blend shapes specific to each mesh, but in UE the
		// AnimSequence curves are specific to each morph target name, so if we have a different mesh,
		// we're probably better off ensuring we have a different AnimSequence too
		HashState.UpdateWithString(*SkeletalMeshHashString, SkeletalMeshHashString.Len());

		HashState.Final();
		HashState.GetHash(&OutHash.Hash[0]);

		return OutHash;
	}

	bool LoadAllSkeletalData(
		const pxr::UsdSkelBinding& InSkeletonBinding,
		pxr::UsdSkelCache& InSkelCache,
		TArray<FSkeletalMeshImportData>& OutLODIndexToSkeletalMeshImportData,
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo>& OutLODIndexToMaterialInfo,
		FUsdCombinedPrimMetadata& LODMetadata,
		TArray<SkeletalMeshImportData::FBone>& OutSkeletonBones,
		FName& OutSkeletonName,
		UsdUtils::FBlendShapeMap* OutBlendShapes,
		TSet<FString>& InOutUsedMorphTargetNames,
		bool bInInterpretLODs,
		const UsdToUnreal::FUsdMeshConversionOptions& Options,
		const FUsdMetadataImportOptions& MetadataOptions
	)
	{
		FScopedUsdAllocs UsdAllocs;

		const pxr::UsdSkelSkeleton& Skeleton = InSkeletonBinding.GetSkeleton();
		if (!Skeleton)
		{
			return false;
		}

		const pxr::UsdSkelSkeletonQuery& SkeletonQuery = InSkelCache.GetSkelQuery(Skeleton);
		if (!SkeletonQuery)
		{
			return false;
		}

		pxr::UsdPrim SkeletonPrim = Skeleton.GetPrim();
		pxr::SdfPath SkeletonPrimPath = SkeletonPrim.GetPrimPath();
		pxr::UsdPrim ClosestParentSkelRoot = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim);
		pxr::SdfPath SkelRootPrimPath = ClosestParentSkelRoot.GetPrimPath();

		pxr::UsdStageRefPtr Stage = SkeletonPrim.GetStage();
		const FUsdStageInfo StageInfo(Stage);

		// Import skeleton data
		{
			FSkeletalMeshImportData DummyImportData;
			const bool bSkeletonValid = UsdToUnreal::ConvertSkeleton(SkeletonQuery, DummyImportData);
			if (!bSkeletonValid)
			{
				return false;
			}
			OutSkeletonBones = MoveTemp(DummyImportData.RefBonesBinary);
			OutSkeletonName = *UsdToUnreal::ConvertString(SkeletonPrim.GetName());
		}

		// Note that the approach is to store skelroot + skinned mesh metadata onto the USkeletalMesh, and
		// purely skeleton metadata onto the USkeleton.
		// Here we collect metadata from the skelroot itself, as the process inside ConvertLOD will only collect
		// metadata from the skinned meshes
		if (MetadataOptions.bCollectMetadata)
		{
			// Here we're always setting this to false otherwise we'll also end up collecting metadata on the skeleton for
			// skel animation and other prims that weren't handled
			const bool bCollectMetadataFromSubtree = false;
			UsdToUnreal::ConvertMetadata(
				ClosestParentSkelRoot,
				LODMetadata,
				MetadataOptions.BlockedPrefixFilters,
				MetadataOptions.bInvertFilters,
				bCollectMetadataFromSubtree
			);
		}

		TMap<int32, FSkeletalMeshImportData> LODIndexToSkeletalMeshImportDataMap;
		TMap<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToMaterialInfoMap;
		UsdUtils::FCaseSensitiveStringSet ProcessedLODParentPaths;

		// Since we may need to switch variants to parse LODs, we could invalidate references to SkinningQuery objects, so we need
		// to keep track of these by path and construct one whenever we need them
		TArray<pxr::SdfPath> PathsToSkinnedPrims;
		for (const pxr::UsdSkelSkinningQuery& SkinningQuery : InSkeletonBinding.GetSkinningTargets())
		{
			// In USD, the skinning target need not be a mesh, but for Unreal we are only interested in skinning meshes
			if (pxr::UsdGeomMesh SkinningMesh = pxr::UsdGeomMesh(SkinningQuery.GetPrim()))
			{
				// Let's only care about prims with the SkelBindingAPI for now as we'll *need* joint influences and weights
				if (SkinningQuery.GetPrim().HasAPI<pxr::UsdSkelBindingAPI>())
				{
					PathsToSkinnedPrims.Add(SkinningMesh.GetPrim().GetPath());
				}
				else
				{
					// TODO: Does this even make any sense? To be a "skinned prim" it *must* have the SkelBindingAPI, no?
					USD_LOG_INFO(
						TEXT("Ignoring skinned prim '%s' when generating Skeletal Mesh for Skeleton '%s' as the prim doesn't have the SkelBindingAPI"
						),
						*UsdToUnreal::ConvertPath(SkinningQuery.GetPrim().GetPrimPath()),
						*UsdToUnreal::ConvertPath(SkeletonPrim.GetPrimPath())
					);
				}
			}
		}

		bool bConvertedMeshData = false;
		TFunction<bool(const pxr::UsdGeomMesh&, int32)> ConvertLOD = [&LODIndexToSkeletalMeshImportDataMap,
																	  &LODIndexToMaterialInfoMap,
																	  &LODMetadata,
																	  &InSkelCache,
																	  &Stage,
																	  &SkeletonPrimPath,
																	  &InOutUsedMorphTargetNames,
																	  OutBlendShapes,
																	  &StageInfo,
																	  Options,
																	  &MetadataOptions,
																	  &bConvertedMeshData](const pxr::UsdGeomMesh& LODMesh, int32 LODIndex)
		{
			pxr::UsdPrim LODMeshPrim = LODMesh.GetPrim();

			// Construct this and SkinningQuery every time so as to survive the prim reference invalidation caused by flipping LODs
			pxr::UsdSkelSkeletonQuery SkeletonQuery = InSkelCache.GetSkelQuery(pxr::UsdSkelSkeleton{Stage->GetPrimAtPath(SkeletonPrimPath)});
			if (!SkeletonQuery)
			{
				return true;	// Continue trying other LODs
			}

			pxr::UsdSkelSkinningQuery SkinningQuery = UsdUtils::CreateSkinningQuery(LODMeshPrim, SkeletonQuery);
			if (!SkinningQuery)
			{
				return true;
			}

			// Ignore prims from disabled purposes
			if (!EnumHasAllFlags(Options.PurposesToLoad, IUsdPrim::GetPurpose(LODMeshPrim)))
			{
				return true;
			}

			// If a skinned prim has an alt draw mode, let's prioritize showing the alt draw mode (that is, geommesh/geometrycache translators
			// will handle it) and skip skinning it
			EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(LODMeshPrim);
			if (DrawMode != EUsdDrawMode::Default)
			{
				return true;
			}

			if (LODMesh && LODMesh.ComputeVisibility() == pxr::UsdGeomTokens->invisible)
			{
				return true;
			}

			FSkeletalMeshImportData& LODImportData = LODIndexToSkeletalMeshImportDataMap.FindOrAdd(LODIndex);
			UsdUtils::FUsdPrimMaterialAssignmentInfo& LODMaterialInfo = LODIndexToMaterialInfoMap.FindOrAdd(
				Options.bMergeIdenticalMaterialSlots ? 0 : LODIndex
			);

			// BlendShape data is respective to point indices for each mesh in isolation, but we combine all points
			// into one FSkeletalMeshImportData per LOD, so we need to remap the indices using this
			uint32 NumPointsBeforeThisMesh = static_cast<uint32>(LODImportData.Points.Num());

			bool bSuccess = UsdToUnreal::ConvertSkinnedMesh(SkinningQuery, SkeletonQuery, LODImportData, LODMaterialInfo, Options);
			if (!bSuccess)
			{
				USD_LOG_WARNING(TEXT("Failed to convert skinned mesh '%s'"), *UsdToUnreal::ConvertPath(LODMeshPrim.GetPrimPath()));
				return true;
			}
			bConvertedMeshData = true;

			if (MetadataOptions.bCollectMetadata && MetadataOptions.bCollectFromEntireSubtrees)
			{
				// Collect metadata from this particular LOD mesh prim
				UsdToUnreal::ConvertMetadata(
					LODMeshPrim,
					LODMetadata,
					MetadataOptions.BlockedPrefixFilters,
					MetadataOptions.bInvertFilters,
					MetadataOptions.bCollectFromEntireSubtrees
				);
			}

			if (OutBlendShapes)
			{
				pxr::GfMatrix4d GeomBindTransform = SkinningQuery.GetGeomBindTransform(Options.TimeCode);

				pxr::SdfPath MeshPrimPath = LODMeshPrim.GetPrimPath();

				pxr::UsdSkelBindingAPI SkelBindingAPI{Stage->GetPrimAtPath(MeshPrimPath)};
				if (pxr::UsdSkelBlendShapeQuery BlendShapeQuery{SkelBindingAPI})
				{
					for (uint32 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeQuery.GetNumBlendShapes(); ++BlendShapeIndex)
					{
						UsdToUnreal::ConvertBlendShape(
							BlendShapeQuery.GetBlendShape(BlendShapeIndex),
							StageInfo,
							LODIndex,
							NumPointsBeforeThisMesh,
							InOutUsedMorphTargetNames,
							*OutBlendShapes,
							Options,
							&GeomBindTransform
						);
					}
				}
			}

			return true;
		};

		// Actually parse all mesh data
		for (const pxr::SdfPath& SkinnedPrimPath : PathsToSkinnedPrims)
		{
			pxr::UsdGeomMesh SkinnedMesh{Stage->GetPrimAtPath(SkinnedPrimPath)};
			if (!SkinnedMesh)
			{
				continue;
			}

			pxr::UsdPrim ParentPrim = SkinnedMesh.GetPrim().GetParent();
			FString ParentPrimPath = UsdToUnreal::ConvertPath(ParentPrim.GetPath());

			bool bInterpretedLODs = false;
			if (bInInterpretLODs && ParentPrim && !ProcessedLODParentPaths.Contains(ParentPrimPath))
			{
				// At the moment we only consider a single mesh per variant, so if multiple meshes tell us to process the same parent prim, we skip.
				// This check would also prevent us from getting in here in case we just have many meshes children of a same prim, outside
				// of a variant. In this case they don't fit the "one mesh per variant" pattern anyway, and we want to fallback to ignoring LODs
				ProcessedLODParentPaths.Add(ParentPrimPath);

				// WARNING: After this is called, references to objects that were inside any of the LOD Meshes will be invalidated!
				bInterpretedLODs = UsdUtils::IterateLODMeshes(ParentPrim, ConvertLOD);
			}

			if (!bInterpretedLODs)
			{
				// Refresh reference to this prim as it could have been inside a variant that was temporarily switched by IterateLODMeshes
				ConvertLOD(pxr::UsdGeomMesh{Stage->GetPrimAtPath(SkinnedPrimPath)}, 0);
			}
		}

		// Repopulate the skeleton cache because flipping through LODs can invalidate some stuff like skeleton references
		InSkelCache.Clear();
		InSkelCache.Populate(pxr::UsdSkelRoot{Stage->GetPrimAtPath(SkelRootPrimPath)}, pxr::UsdTraverseInstanceProxies());

		// Place the LODs in order as we can't have e.g. LOD0 and LOD2 without LOD1, and there's no reason downstream code needs to care about
		// what LOD number these data originally wanted to be
		TMap<int32, int32> OldLODIndexToNewLODIndex;
		LODIndexToSkeletalMeshImportDataMap.KeySort(TLess<int32>());
		OutLODIndexToSkeletalMeshImportData.Reset(LODIndexToSkeletalMeshImportDataMap.Num());
		OutLODIndexToMaterialInfo.Reset(LODIndexToMaterialInfoMap.Num());
		for (TPair<int32, FSkeletalMeshImportData>& Entry : LODIndexToSkeletalMeshImportDataMap)
		{
			FSkeletalMeshImportData& ImportData = Entry.Value;
			if (Entry.Value.Points.Num() == 0)
			{
				continue;
			}

			const int32 OldLODIndex = Entry.Key;
			const int32 NewLODIndex = OutLODIndexToSkeletalMeshImportData.Add(MoveTemp(ImportData));

			if (UsdUtils::FUsdPrimMaterialAssignmentInfo* FoundInfo = LODIndexToMaterialInfoMap.Find(OldLODIndex))
			{
				OutLODIndexToMaterialInfo.Add(*FoundInfo);
			}

			// Keep track of these to remap blendshapes
			OldLODIndexToNewLODIndex.Add(OldLODIndex, NewLODIndex);
		}
		if (OutBlendShapes)
		{
			for (auto& Pair : *OutBlendShapes)
			{
				UsdUtils::FUsdBlendShape& BlendShape = Pair.Value;

				TSet<int32> NewLODIndexUsers;
				NewLODIndexUsers.Reserve(BlendShape.LODIndicesThatUseThis.Num());

				for (int32 OldLODIndexUser : BlendShape.LODIndicesThatUseThis)
				{
					if (int32* FoundNewLODIndex = OldLODIndexToNewLODIndex.Find(OldLODIndexUser))
					{
						NewLODIndexUsers.Add(*FoundNewLODIndex);
					}
					else
					{
						USD_LOG_ERROR(TEXT("Failed to remap blend shape '%s's LOD index '%d'"), *BlendShape.Name, OldLODIndexUser);
					}
				}

				BlendShape.LODIndicesThatUseThis = MoveTemp(NewLODIndexUsers);
			}
		}

		return bConvertedMeshData;
	}

	/** Warning: This function will temporarily switch the active LOD variant if one exists, so it's *not* thread safe! */
	void SetMaterialOverrides(
		const pxr::UsdSkelBinding& SkeletonBinding,
		const TArray<UMaterialInterface*>& ExistingAssignments,
		UMeshComponent& MeshComponent,
		FUsdSchemaTranslationContext& Context
	)
	{
		FScopedUsdAllocs Allocs;

		const pxr::UsdSkelSkeleton& Skeleton = SkeletonBinding.GetSkeleton();
		if (!Skeleton)
		{
			return;
		}
		pxr::UsdPrim SkeletonPrim = Skeleton.GetPrim();
		pxr::SdfPath SkeletonPrimPath = SkeletonPrim.GetPath();

		pxr::UsdStageRefPtr Stage = SkeletonPrim.GetStage();

		pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
		if (!Context.RenderContext.IsNone())
		{
			RenderContextToken = UnrealToUsd::ConvertToken(*Context.RenderContext.ToString()).Get();
		}

		pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		if (!Context.MaterialPurpose.IsNone())
		{
			MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context.MaterialPurpose.ToString()).Get();
		}

		TMap<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToMaterialInfoMap;
		TMap<int32, TSet<UsdUtils::FUsdPrimMaterialSlot>> CombinedSlotsForLODIndex;

		TFunction<bool(const pxr::UsdGeomMesh&, int32)> IterateLODsLambda = [&LODIndexToMaterialInfoMap,
																			 &CombinedSlotsForLODIndex,
																			 &Context,
																			 RenderContextToken,
																			 MaterialPurposeToken](const pxr::UsdGeomMesh& LODMesh, int32 LODIndex)
		{
			if (LODMesh && LODMesh.ComputeVisibility() == pxr::UsdGeomTokens->invisible)
			{
				return true;
			}

			// Ignore prims with disabled purposes: We need to match the material slot ordering that was used
			// to generate the mesh in the first place, so this is important
			if (!EnumHasAllFlags(Context.PurposesToLoad, IUsdPrim::GetPurpose(LODMesh.GetPrim())))
			{
				return true;
			}

			// When merging slots, we share the same material info across all LODs
			int32 LODIndexToUse = Context.bMergeIdenticalMaterialSlots ? 0 : LODIndex;

			TArray<UsdUtils::FUsdPrimMaterialSlot>& CombinedLODSlots = LODIndexToMaterialInfoMap.FindOrAdd(LODIndexToUse).Slots;
			TSet<UsdUtils::FUsdPrimMaterialSlot>& CombinedLODSlotsSet = CombinedSlotsForLODIndex.FindOrAdd(LODIndexToUse);

			const bool bProvideMaterialIndices = false;	   // We have no use for material indices and it can be slow to retrieve, as it will iterate
														   // all faces
			UsdUtils::FUsdPrimMaterialAssignmentInfo LocalInfo = UsdUtils::GetPrimMaterialAssignments(
				LODMesh.GetPrim(),
				pxr::UsdTimeCode(Context.Time),
				bProvideMaterialIndices,
				RenderContextToken,
				MaterialPurposeToken
			);

			// Combine material slots in the same order that UsdToUnreal::ConvertSkinnedMesh does
			for (UsdUtils::FUsdPrimMaterialSlot& LocalSlot : LocalInfo.Slots)
			{
				if (!CombinedLODSlotsSet.Contains(LocalSlot))
				{
					CombinedLODSlots.Add(LocalSlot);
					CombinedLODSlotsSet.Add(LocalSlot);
				}
			}

			return true;
		};

		UsdUtils::FCaseSensitiveStringSet ProcessedLODParentPaths;

		// Because we combine all skinning target meshes into a single skeletal mesh, we'll have to reconstruct the combined
		// material assignment info that this SkelRoot wants in order to compare with the existing assignments.
		for (const pxr::UsdSkelSkinningQuery& SkinningQuery : SkeletonBinding.GetSkinningTargets())
		{
			pxr::UsdPrim MeshPrim = SkinningQuery.GetPrim();
			pxr::UsdGeomMesh Mesh{MeshPrim};
			if (!Mesh)
			{
				continue;
			}

			// GetSkinningTargets can also return prims without the skel binding API, but we don't want to collect
			// material bindings from those as they are not going to be globbed into the SkeletalMesh
			if (!MeshPrim.HasAPI<pxr::UsdSkelBindingAPI>())
			{
				continue;
			}

			pxr::SdfPath MeshPrimPath = MeshPrim.GetPath();

			pxr::UsdPrim ParentPrim = MeshPrim.GetParent();
			FString ParentPrimPath = UsdToUnreal::ConvertPath(ParentPrim.GetPath());

			bool bInterpretedLODs = false;
			if (Context.bAllowInterpretingLODs && UsdUtils::IsGeomMeshALOD(MeshPrim) && !ProcessedLODParentPaths.Contains(ParentPrimPath))
			{
				ProcessedLODParentPaths.Add(ParentPrimPath);

				bInterpretedLODs = UsdUtils::IterateLODMeshes(ParentPrim, IterateLODsLambda);
			}

			if (!bInterpretedLODs)
			{
				// Refresh reference to this prim as it could have been inside a variant that was temporarily switched by IterateLODMeshes
				IterateLODsLambda(pxr::UsdGeomMesh{Stage->GetPrimAtPath(MeshPrimPath)}, 0);
			}
		}

		// Refresh reference to Skeleton prim because variant switching potentially invalidated it
		pxr::UsdPrim ValidSkeletonPrim = Stage->GetPrimAtPath(SkeletonPrimPath);

		// Place the LODs in order as we can't have e.g. LOD0 and LOD2 without LOD1, and there's no reason downstream code needs to care about
		// what LOD number these data originally wanted to be
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToAssignments;
		LODIndexToMaterialInfoMap.KeySort(TLess<int32>());
		for (TPair<int32, UsdUtils::FUsdPrimMaterialAssignmentInfo>& Entry : LODIndexToMaterialInfoMap)
		{
			LODIndexToAssignments.Add(MoveTemp(Entry.Value));
		}

		// Stash our mesh PrimvarToUVIndex into the assignment info, as that's where ResolveMaterialAssignmentInfo will look for it
		UUsdMeshAssetUserData* UserData = nullptr;
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(&MeshComponent))
		{
			if (USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				UserData = Mesh->GetAssetUserData<UUsdMeshAssetUserData>();
			}
		}
		if (UserData)
		{
			if (LODIndexToAssignments.Num() > 0)
			{
				LODIndexToAssignments[0].PrimvarToUVIndex = UserData->PrimvarToUVIndex;
			}
		}
		else
		{
			ensureMsgf(
				UserData,
				TEXT("Expected component '%s''s SkeletalMesh to have an instance of UUsdMeshAssetUserData at this point!"),
				*MeshComponent.GetPathName()
			);
		}

		TMap<const UsdUtils::FUsdPrimMaterialSlot*, UMaterialInterface*> ResolvedMaterials = MeshTranslationImpl::ResolveMaterialAssignmentInfo(
			ValidSkeletonPrim,
			LODIndexToAssignments,
			*Context.UsdAssetCache,
			*Context.PrimLinkCache,
			Context.ObjectFlags,
			Context.bShareAssetsForIdenticalPrims
		);

		// Compare resolved materials with existing assignments, and create overrides if we need to
		uint32 SkeletalMeshSlotIndex = 0;
		for (int32 LODIndex = 0; LODIndex < LODIndexToAssignments.Num(); ++LODIndex)
		{
			const TArray<UsdUtils::FUsdPrimMaterialSlot>& LODSlots = LODIndexToAssignments[LODIndex].Slots;
			for (int32 LODSlotIndex = 0; LODSlotIndex < LODSlots.Num(); ++LODSlotIndex, ++SkeletalMeshSlotIndex)
			{
				const UsdUtils::FUsdPrimMaterialSlot& Slot = LODSlots[LODSlotIndex];

				UMaterialInterface* Material = nullptr;
				if (UMaterialInterface** FoundMaterial = ResolvedMaterials.Find(&Slot))
				{
					Material = *FoundMaterial;
				}
				else
				{
					USD_LOG_ERROR(
						TEXT("Lost track of resolved material for slot '%d' of LOD '%d' for skeletal mesh '%s'"),
						LODSlotIndex,
						LODIndex,
						*UsdToUnreal::ConvertPath(ValidSkeletonPrim.GetPath())
					);
					continue;
				}

				UMaterialInterface* ExistingMaterial = ExistingAssignments.IsValidIndex(SkeletalMeshSlotIndex)
														   ? ExistingAssignments[SkeletalMeshSlotIndex]
														   : nullptr;
				if (!ExistingMaterial || ExistingMaterial == Material)
				{
					continue;
				}
				else
				{
					MeshComponent.SetMaterial(SkeletalMeshSlotIndex, Material);
				}
			}
		}
	}

	bool HasLODSkinningTargets(const pxr::UsdSkelBinding& SkelBinding)
	{
		FScopedUsdAllocs Allocs;

		for (const pxr::UsdSkelSkinningQuery& SkinningQuery : SkelBinding.GetSkinningTargets())
		{
			if (UsdUtils::IsGeomMeshALOD(SkinningQuery.GetPrim()))
			{
				return true;
			}
		}

		return false;
	}

	UAnimBlueprint* CreateAnimBlueprint(
		const FUsdSchemaTranslationContext& Context,
		const pxr::UsdPrim& SkeletonPrim,
		bool bDelayRecompilation = false,	  // Whether we should recompile within this function or not
		bool* bOutNeedsRecompile = nullptr	  // Whether this function wants the returned AnimBP to be recompiled
	)
	{
		if (!SkeletonPrim || !Context.PrimLinkCache || !Context.UsdAssetCache)
		{
			return nullptr;
		}

		FString PrimName = UsdToUnreal::ConvertString(SkeletonPrim.GetName());

		USkeletalMesh* SkeletalMesh = Context.PrimLinkCache->GetSingleAssetForPrim<USkeletalMesh>(UE::FSdfPath{SkeletonPrim.GetPath()});
		if (!SkeletalMesh)
		{
			return nullptr;
		}

		USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
		if (!Skeleton)
		{
			return nullptr;
		}

		// Fetch relevant attributes from prim, since we know it has the schema
		FString AnimBPPath;
		{
			FScopedUsdAllocs Allocs;

			if (pxr::UsdAttribute Attr = SkeletonPrim.GetAttribute(UnrealIdentifiers::UnrealAnimBlueprintPath))
			{
				std::string PathString;
				if (Attr.Get(&PathString))
				{
					AnimBPPath = UsdToUnreal::ConvertString(PathString);
				}
			}
			// Temporarily check the SkelRoot for the same attribute for backwards compatibility
			else if (pxr::UsdPrim ClosestParentSkelRoot = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim))
			{
				if (pxr::UsdAttribute ParentAttr = ClosestParentSkelRoot.GetAttribute(UnrealIdentifiers::UnrealAnimBlueprintPath))
				{
					std::string PathString;
					if (ParentAttr.Get(&PathString))
					{
						AnimBPPath = UsdToUnreal::ConvertString(PathString);
					}
				}
			}
		}
		if (AnimBPPath.IsEmpty())
		{
			return nullptr;
		}

		bool bNeedRecompile = false;

		UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(FSoftObjectPath(AnimBPPath).TryLoad());
		if (!AnimBP)
		{
			return nullptr;
		}

		// Create transient AnimBP based on our template, so that we can assign it a proper skeleton
		const static FString DefaultAnimBPPath = TEXT("/USDImporter/Blueprint/DefaultLiveLinkAnimBP.DefaultLiveLinkAnimBP");
		if (DefaultAnimBPPath == AnimBPPath)
		{
			const FString PrimPath = UsdToUnreal::ConvertPath(SkeletonPrim.GetPrimPath());

			// Let's try to never reuse AnimBP between prims (as we want to be able to switch subject names
			// independently and we likely won't have more than a handful of these anyway).
			// We should have at least something deterministic though so that we don't repeatedly recreate assets for the same prim.
			FSHAHash Hash;
			FSHA1 SHA1;
			// Each stage actor has a separate info cache that is assigned to the context
			SHA1.Update(reinterpret_cast<const uint8*>(Context.UsdInfoCache), sizeof(Context.UsdInfoCache));
			SHA1.UpdateWithString(*PrimPath, PrimPath.Len());
			SHA1.Final();
			SHA1.GetHash(&Hash.Hash[0]);
			const FString PrefixedAnimBPHash = UsdUtils::GetAssetHashPrefix(SkeletonPrim, Context.bShareAssetsForIdenticalPrims) + Hash.ToString();

			bool bReusedAnimBP = false;
			if (UAnimBlueprint* CachedAnimBP = Context.UsdAssetCache->GetCachedAsset<UAnimBlueprint>(PrefixedAnimBPHash))
			{
				if (CachedAnimBP->TargetSkeleton == Skeleton)
				{
					AnimBP = CachedAnimBP;
					bReusedAnimBP = true;
				}
			}

			// We have to generate a new transient AnimBP
			if (!bReusedAnimBP)
			{
				// Duplicate and never reuse these so that they can be assigned independent subject names if desired.
				// Its not as if scenes will have thousands of these anyway.
				bool bCreatedAsset = false;
				const EObjectFlags DesiredFlags = Skeleton->GetFlags();
				const FString& DesiredAnimBPName = PrimName + TEXT("_DefaultAnimBlueprint");
				AnimBP = Context.UsdAssetCache->GetOrCreateCustomCachedAsset<UAnimBlueprint>(
					PrefixedAnimBPHash,
					DesiredAnimBPName,
					DesiredFlags,
					[AnimBP](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
					{
						UObject* Duplicated = DuplicateObject(AnimBP, Outer, SanitizedName);
						Duplicated->ClearFlags(Duplicated->GetFlags());
						Duplicated->SetFlags(FlagsToUse);
						return Duplicated;
					},
					&bCreatedAsset
				);

				if (AnimBP && bCreatedAsset)
				{
					AnimBP->TargetSkeleton = Skeleton;
					AnimBP->bIsTemplate = false;

					bNeedRecompile = true;
				}
			}
		}
		// Path is pointing to an existing, persistent AnimBP
		else
		{
			// Force skeletons to be compatible if they aren't (we need both ways!)
			if (!Skeleton->IsCompatibleForEditor(AnimBP->TargetSkeleton))
			{
				AnimBP->TargetSkeleton->AddCompatibleSkeleton(Skeleton);
				Skeleton->AddCompatibleSkeleton(AnimBP->TargetSkeleton);

				if (AnimBP->TargetSkeleton->GetReferenceSkeleton().GetRefBoneInfo() != Skeleton->GetReferenceSkeleton().GetRefBoneInfo())
				{
					USD_LOG_WARNING(
						TEXT(
							"Forcing AnimBlueprint '%s's Skeleton '%s' to be compatible with the Skeleton generated for prim '%s', but they may be different!"
						),
						*AnimBP->GetPathName(),
						*AnimBP->TargetSkeleton->GetPathName(),
						*PrimName
					);
				}
			}
		}

		if (bNeedRecompile && !bDelayRecompilation)
		{
			FCompilerResultsLog Results;
			FBPCompileRequest Request(AnimBP, EBlueprintCompileOptions::None, &Results);
			FBlueprintCompilationManager::CompileSynchronously(Request);
			bNeedRecompile = false;
		}

		if (bOutNeedsRecompile)
		{
			*bOutNeedsRecompile = bNeedRecompile;
		}

		return AnimBP;
	}

	void UpdateLiveLinkProperties(const FUsdSchemaTranslationContext& Context, USkeletalMeshComponent* Component, const pxr::UsdPrim& SkeletonPrim)
	{
		if (!Component || !Component->GetSkeletalMeshAsset() || !SkeletonPrim)
		{
			return;
		}

		FString PrimName = UsdToUnreal::ConvertString(SkeletonPrim.GetName());

		USkeleton* Skeleton = Component->GetSkeletalMeshAsset()->GetSkeleton();
		if (!Skeleton)
		{
			return;
		}

		UAnimBlueprint* ExistingAnimBP = nullptr;
		if (Component->AnimClass)
		{
			ExistingAnimBP = Cast<UAnimBlueprint>(Component->AnimClass->ClassGeneratedBy);
		}

		// Fetch relevant attributes from prim, since we know it has the schema
		TUsdStore<pxr::UsdSkelRoot> ClosestSkelRootParent;
		TFunction<FString(const pxr::TfToken&)> GetAttrValue = [&SkeletonPrim, &ClosestSkelRootParent](const pxr::TfToken& AttrName) -> FString
		{
			FScopedUsdAllocs Allocs;

			if (pxr::UsdAttribute Attr = SkeletonPrim.GetAttribute(AttrName))
			{
				std::string SubjectNameString;
				if (Attr.Get(&SubjectNameString))
				{
					return UsdToUnreal::ConvertString(SubjectNameString);
				}
			}
			// Temporarily check the SkelRoot for the same attribute for backwards compatibility
			else
			{
				if (!ClosestSkelRootParent.Get())
				{
					ClosestSkelRootParent = pxr::UsdSkelRoot{UsdUtils::GetClosestParentSkelRoot(SkeletonPrim)};
				}

				if (ClosestSkelRootParent.Get())
				{
					if (pxr::UsdAttribute ParentAttr = ClosestSkelRootParent.Get().GetPrim().GetAttribute(AttrName))
					{
						std::string SubjectNameString;
						if (ParentAttr.Get(&SubjectNameString))
						{
							return UsdToUnreal::ConvertString(SubjectNameString);
						}
					}
				}
			}

			return {};
		};
		FString SubjectName = GetAttrValue(UnrealIdentifiers::UnrealLiveLinkSubjectName);
		FString AnimBPPath = GetAttrValue(UnrealIdentifiers::UnrealAnimBlueprintPath);

		UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(FSoftObjectPath(AnimBPPath).TryLoad());

		// Check if we need to change the AnimBP
		bool bNeedRecompile = false;
		if (ExistingAnimBP != AnimBP)
		{
			const bool bDelayRecompilation = true;
			AnimBP = UsdSkelSkeletonTranslatorImpl::CreateAnimBlueprint(Context, SkeletonPrim, bDelayRecompilation, &bNeedRecompile);
		}

		// Apply subject name to live link pose AnimBlueprint node
		// Reference: UAnimationBlueprintLibrary::AddNodeAssetOverride
		if (AnimBP)
		{
			TArray<UBlueprint*> BlueprintHierarchy;
			AnimBP->GetBlueprintHierarchyFromClass(AnimBP->GetAnimBlueprintGeneratedClass(), BlueprintHierarchy);

			TArray<UAnimGraphNode_LiveLinkPose*> LiveLinkNodes;

			for (int32 BlueprintIndex = 0; BlueprintIndex < BlueprintHierarchy.Num(); ++BlueprintIndex)
			{
				UBlueprint* CurrentBlueprint = BlueprintHierarchy[BlueprintIndex];

				TArray<UEdGraph*> Graphs;
				CurrentBlueprint->GetAllGraphs(Graphs);

				for (UEdGraph* Graph : Graphs)
				{
					for (UEdGraphNode* Node : Graph->Nodes)
					{
						if (UAnimGraphNode_LiveLinkPose* AnimNode = Cast<UAnimGraphNode_LiveLinkPose>(Node))
						{
							LiveLinkNodes.Add(AnimNode);
						}
					}
				}
			}

			if (LiveLinkNodes.Num() > 1 && !ExistingAnimBP)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"MoreThanOneLiveLinkPose",
						"Found more than one LiveLinkPose blueprint node on AnimBlueprint '{0}'s graphs. Note that all of those nodes will have their LiveLink SubjectName's updated to '{1}', as described for prim '{2}'!"
					),
					FText::FromString(AnimBP->GetPathName()),
					FText::FromString(SubjectName),
					FText::FromString(UsdToUnreal::ConvertPath(SkeletonPrim.GetPrimPath()))
				));
			}

			for (UAnimGraphNode_LiveLinkPose* Node : LiveLinkNodes)
			{
				const UEdGraphSchema* Schema = Node->GetSchema();
				if (!Schema)
				{
					continue;
				}

				UEdGraphPin* SubjectNamePin = nullptr;
				for (UEdGraphPin* Pin : Node->Pins)
				{
					if (Pin->GetName() == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_LiveLinkPose, LiveLinkSubjectName))
					{
						SubjectNamePin = Pin;
						break;
					}
				}

				// The subject name pin is already connected to something...
				if (SubjectNamePin->LinkedTo.Num() != 0)
				{
					if (!ExistingAnimBP)
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"PinAlreadyConnected",
								"Failed to update a LiveLinkPose node's 'Subject Name' to '{0}' on AnimBlueprint '{1}', because the pin is already connected to some other node. Disconnect it if you want it to be updated automatically."
							),
							FText::FromString(SubjectName),
							FText::FromString(AnimBP->GetPathName())
						));
					}

					continue;
				}

				if (SubjectNamePin)
				{
					// The pin type is FLiveLinkSubjectName, so we must create an instance of it and serialize it using
					// UScriptStruct::ExportText to generate a proper default value string
					FLiveLinkSubjectName Dummy;
					Dummy.Name = *SubjectName;

					FString ValueString;
					const void* Defaults = nullptr;
					UObject* OwnerObject = nullptr;
					const int32 PortFlags = EPropertyPortFlags::PPF_None;
					UObject* ExportRootScope = nullptr;
					FLiveLinkSubjectName::StaticStruct()->ExportText(ValueString, &Dummy, Defaults, OwnerObject, PortFlags, ExportRootScope);

					if (!Schema->DoesDefaultValueMatch(*SubjectNamePin, ValueString))
					{
						SubjectNamePin->Modify();
						Schema->TrySetDefaultValue(*SubjectNamePin, ValueString);

						FBlueprintEditorUtils::MarkBlueprintAsModified(AnimBP);
						bNeedRecompile = true;
					}
				}
			}
		}

		if (bNeedRecompile)
		{
			FCompilerResultsLog Results;
			FBPCompileRequest Request(AnimBP, EBlueprintCompileOptions::None, &Results);
			FBlueprintCompilationManager::CompileSynchronously(Request);

			// We need to force the component to update its anim after we regenerate the blueprint class
			Component->ClearAnimScriptInstance();
			Component->InitAnim(true);
		}

		if (AnimBP != ExistingAnimBP)
		{
			// This can internally change AnimationMode, but lets revert it to what it was so that we can control it from
			// that single place in ::UpdateComponents
			EAnimationMode::Type OldMode = Component->GetAnimationMode();
			Component->SetAnimInstanceClass(AnimBP->GeneratedClass);
			Component->SetAnimationMode(OldMode);
		}
	}

	class FSkelSkeletonCreateAssetsTaskChain : public FUsdSchemaTranslatorTaskChain
	{
	public:
		explicit FSkelSkeletonCreateAssetsTaskChain(
			const TSharedRef<FUsdSchemaTranslationContext>& InContext,
			const UE::FSdfPath& InSkeletonPrimPath
		);

	protected:
		// Inputs
		UE::FSdfPath SkeletonPrimPath;
		TSharedRef<FUsdSchemaTranslationContext> Context;

		// Outputs
		TArray<FSkeletalMeshImportData> LODIndexToSkeletalMeshImportData;
		TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToMaterialInfo;
		FUsdCombinedPrimMetadata LODMetadata;
		TArray<SkeletalMeshImportData::FBone> SkeletonBones;
		FName SkeletonName;
		UsdUtils::FBlendShapeMap NewBlendShapes;

		// Note that we want this to be case insensitive so that our UMorphTarget FNames are unique not only due to case differences
		TSet<FString> UsedMorphTargetNames;
		TUsdStore<pxr::UsdSkelBinding> SkeletonBinding;
		TUsdStore<pxr::UsdSkelCache> SkelCache;
		TUsdStore<pxr::UsdSkelRoot> ClosestParentSkelRoot;
		TUsdStore<pxr::UsdSkelSkeletonQuery> SkeletonQuery;
		FString PrefixedSkelMeshHash;

		// Don't keep a live reference to the prim because other translators may mutate the stage in an ExclusiveSync translation step, invalidating
		// the reference
		UE::FUsdPrim GetSkeletonPrim() const
		{
			return Context->Stage.GetPrimAtPath(SkeletonPrimPath);
		}

		void SetupTasks();
	};

	FSkelSkeletonCreateAssetsTaskChain::FSkelSkeletonCreateAssetsTaskChain(
		const TSharedRef<FUsdSchemaTranslationContext>& InContext,
		const UE::FSdfPath& InPrimPath
	)
		: SkeletonPrimPath(InPrimPath)
		, Context(InContext)
	{
		// Collect our skel binding info
		{
			FScopedUsdAllocs Allocs;

			pxr::UsdPrim SkeletonPrim = GetSkeletonPrim();

			bool bSuccess = false;

			ClosestParentSkelRoot = pxr::UsdSkelRoot{UsdUtils::GetClosestParentSkelRoot(SkeletonPrim)};
			SkelCache.Get().Clear();
			if (ClosestParentSkelRoot.Get() && SkelCache.Get().Populate(ClosestParentSkelRoot.Get(), pxr::UsdTraverseInstanceProxies()))
			{
				bSuccess = UsdUtils::GetSkelQueries(
					ClosestParentSkelRoot.Get(),
					pxr::UsdSkelSkeleton{SkeletonPrim},
					SkeletonBinding.Get(),
					SkeletonQuery.Get(),
					&SkelCache.Get()
				);
			}

			if (!bSuccess)
			{
				USD_LOG_WARNING(
					TEXT("Failed to find skeleton binding info for skeleton at path '%s'. Is it within a SkelRoot prim?"),
					*InPrimPath.GetString()
				);
				return;
			}
		}

		SetupTasks();
	}

	// Right now parsing LODs involves flipping through variants, which invalidates some prims and references.
	// The SkelSkeletonTranslator is especially vulnerable to this because the SkeletonBinding contains
	// skinning queries that are all invalidated when we flip through variants, and the task chain holds on
	// to the same SkeletonBinding throughout the entire chain...
	// Here we'll refresh those references if needed.
	// TODO: Find better way of handling LODs that doesn't require this mechanism.
	void RefreshSkelReferencesIfNeeded(
		const pxr::UsdSkelRoot& InSkelRootPrim,
		const pxr::UsdSkelSkeleton& InSkeletonPrim,
		pxr::UsdSkelCache& InOutSkelCache,
		pxr::UsdSkelBinding& InOutSkelBinding,
		pxr::UsdSkelSkeletonQuery& InOutSkeletonQuery
	)
	{
		// If we still have valid skinning queries we know we still have valid references, as those are the first to break
		for (const pxr::UsdSkelSkinningQuery& SkinningQuery : InOutSkelBinding.GetSkinningTargets())
		{
			if (pxr::UsdGeomMesh SkinningMesh = pxr::UsdGeomMesh(SkinningQuery.GetPrim()))
			{
				return;
			}
		}

		InOutSkelCache.Clear();
		InOutSkelCache.Populate(InSkelRootPrim, pxr::UsdTraverseInstanceProxies());
		ensure(UsdUtils::GetSkelQueries(InSkelRootPrim, InSkeletonPrim, InOutSkelBinding, InOutSkeletonQuery, &InOutSkelCache));
	}

	void FSkelSkeletonCreateAssetsTaskChain::SetupTasks()
	{
		// To parse all LODs we need to actively switch variant sets to other variants (triggering prim loading/unloading and notices),
		// which could cause race conditions if other async translation tasks are trying to access those prims
		const bool bTryLODParsing = Context->bAllowInterpretingLODs && HasLODSkinningTargets(SkeletonBinding.Get());
		ESchemaTranslationLaunchPolicy LaunchPolicy = bTryLODParsing ? ESchemaTranslationLaunchPolicy::ExclusiveSync
																	 : ESchemaTranslationLaunchPolicy::Async;

		// Create SkeletalMeshImportData (Async or ExclusiveSync)
		Do(LaunchPolicy,
		   [this, bTryLODParsing]()
		   {
			   RefreshSkelReferencesIfNeeded(
				   ClosestParentSkelRoot.Get(),
				   pxr::UsdSkelSkeleton{GetSkeletonPrim()},
				   SkelCache.Get(),
				   SkeletonBinding.Get(),
				   SkeletonQuery.Get()
			   );

			   // No point in importing blend shapes if the import context doesn't want them
			   UsdUtils::FBlendShapeMap* OutBlendShapes = Context->BlendShapesByPath ? &NewBlendShapes : nullptr;

			   pxr::TfToken RenderContextToken = pxr::UsdShadeTokens->universalRenderContext;
			   if (!Context->RenderContext.IsNone())
			   {
				   RenderContextToken = UnrealToUsd::ConvertToken(*Context->RenderContext.ToString()).Get();
			   }

			   pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
			   if (!Context->MaterialPurpose.IsNone())
			   {
				   MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
			   }

			   UsdToUnreal::FUsdMeshConversionOptions Options;
			   Options.TimeCode = Context->Time;
			   Options.RenderContext = RenderContextToken;
			   Options.MaterialPurpose = MaterialPurposeToken;
			   Options.SubdivisionLevel = Context->SubdivisionLevel;
			   Options.PurposesToLoad = Context->PurposesToLoad;
			   Options.bMergeIdenticalMaterialSlots = Context->bMergeIdenticalMaterialSlots;

			   const bool bContinueTaskChain = UsdSkelSkeletonTranslatorImpl::LoadAllSkeletalData(
				   SkeletonBinding.Get(),
				   SkelCache.Get(),
				   LODIndexToSkeletalMeshImportData,
				   LODIndexToMaterialInfo,
				   LODMetadata,
				   SkeletonBones,
				   SkeletonName,
				   OutBlendShapes,
				   UsedMorphTargetNames,
				   bTryLODParsing,
				   Options,
				   Context->MetadataOptions
			   );

			   return bContinueTaskChain;
		   });

		// Create USkeletalMesh (Main thread)
		Then(
			ESchemaTranslationLaunchPolicy::Sync,
			[this]()
			{
				if (!Context->PrimLinkCache || !Context->UsdAssetCache)
				{
					return false;
				}

				// We may have invalidated references with the previous task if it parsed LODs, so refresh them if needed.
				// We'll assume that it's unlikely that those would be invalidated past this point though, as only the previous
				// task is capable of invalidating them, and it is an ExclusiveSync task
				RefreshSkelReferencesIfNeeded(
					ClosestParentSkelRoot.Get(),
					pxr::UsdSkelSkeleton{GetSkeletonPrim()},
					SkelCache.Get(),
					SkeletonBinding.Get(),
					SkeletonQuery.Get()
				);

				UsdUtils::FBlendShapeMap* BlendShapes = Context->BlendShapesByPath ? &NewBlendShapes : nullptr;

				FSHAHash SkeletalMeshHash = UsdSkelSkeletonTranslatorImpl::ComputeSHAHash(
					LODIndexToSkeletalMeshImportData,
					SkeletonBones,
					BlendShapes
				);
				PrefixedSkelMeshHash = UsdUtils::GetAssetHashPrefix(GetSkeletonPrim(), Context->bShareAssetsForIdenticalPrims)
									   + SkeletalMeshHash.ToString();

				const FString DesiredSkeletalMeshName = UsdToUnreal::ConvertString(ClosestParentSkelRoot.Get().GetPrim().GetName());

				// Even though we're translating SkeletalMeshes from Skeleton prims now, keep using the SkelRoot
				// prim name as the SkeletalMesh asset name. This because in the general case the entire SkelRoot will
				// represent a character and be named after it, and the Skeleton will be named just "Skeleton" or "Skel"
				bool bIsNew = false;
				USkeletalMesh* SkeletalMesh = Context->UsdAssetCache->GetOrCreateCachedAsset<USkeletalMesh>(
					PrefixedSkelMeshHash,
					DesiredSkeletalMeshName,
					Context->ObjectFlags,
					&bIsNew
				);

				if (bIsNew)
				{
					const FString PrefixedSkeletonHash = PrefixedSkelMeshHash + TEXT("_Skeleton");
					const FString DesiredSkeletonName = SkeletonName.ToString();
					bool bSkeletonIsNew = false;
					USkeleton* Skeleton = Context->UsdAssetCache->GetOrCreateCachedAsset<USkeleton>(
						PrefixedSkeletonHash,
						DesiredSkeletonName,
						Context->ObjectFlags,
						&bSkeletonIsNew
					);

					Skeleton->SetPreviewMesh(SkeletalMesh);
					SkeletalMesh->SetSkeleton(Skeleton);

					bool bSuccess = UsdToUnreal::ConvertSkeletalImportData(
						LODIndexToSkeletalMeshImportData,
						SkeletonBones,
						NewBlendShapes,
						SkeletalMesh	// Already has Skeleton
					);

					if (!bSuccess)
					{
						USD_LOG_WARNING(
							TEXT("Failed to create SkeletalMesh for prim '%s'"),
							*UsdToUnreal::ConvertPath(ClosestParentSkelRoot.Get().GetPrim().GetPrimPath())
						);
						UsdUnreal::TranslatorUtils::AbandonFailedAsset(SkeletalMesh, Context->UsdAssetCache.Get(), Context->PrimLinkCache);

						if (bSkeletonIsNew)
						{
							USD_LOG_WARNING(
								TEXT("Failed to create Skeleton '%s' for prim '%s'"),
								*SkeletonName.ToString(),
								*UsdToUnreal::ConvertPath(ClosestParentSkelRoot.Get().GetPrim().GetPrimPath())
							);
							UsdUnreal::TranslatorUtils::AbandonFailedAsset(Skeleton, Context->UsdAssetCache.Get(), Context->PrimLinkCache);
						}
					}
				}

				if (SkeletalMesh)
				{
					// Handle the SkeletalMesh AssetUserData
					if (UUsdMeshAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdMeshAssetUserData>(SkeletalMesh))
					{
						if (LODIndexToMaterialInfo.Num() > 0)
						{
							// We use the same primvar mapping for all LODs
							UserData->PrimvarToUVIndex = LODIndexToMaterialInfo[0].PrimvarToUVIndex;
						}
						UserData->PrimPaths.AddUnique(SkeletonPrimPath.GetString());

						// For the skel task chain we always collect skeletal mesh metadata when first parsing the prims directly, as it
						// allows us to do it while we're flipping through LOD variants, if any
						if (Context->MetadataOptions.bCollectMetadata)
						{
							UserData->StageIdentifierToMetadata.Add(GetSkeletonPrim().GetStage().GetRootLayer().GetIdentifier(), LODMetadata);
						}
						else
						{
							// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the AssetUserData
							UserData->StageIdentifierToMetadata.Remove(GetSkeletonPrim().GetStage().GetRootLayer().GetIdentifier());
						}

						MeshTranslationImpl::RecordSourcePrimsForMaterialSlots(LODIndexToMaterialInfo, UserData);
					}

					// Handle the Skeleton AssetUserData
					if (USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
					{
						if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData(Skeleton))
						{
							UserData->PrimPaths.AddUnique(SkeletonPrimPath.GetString());

							if (Context->MetadataOptions.bCollectMetadata)
							{
								// Since we never collapse, we'll spawn assets components for any child prim that happens to be inside
								// the skeleton itself, and the skeleton type doesn't have any relevant "child prim" type (like for
								// Mesh prims and UsdGeomSubsets), so we're probably safe in never collecting metadata from the skeleton
								// prim subtree
								const bool bCollectFromEntireSubtrees = false;
								UsdToUnreal::ConvertMetadata(
									GetSkeletonPrim(),
									UserData,
									Context->MetadataOptions.BlockedPrefixFilters,
									Context->MetadataOptions.bInvertFilters,
									bCollectFromEntireSubtrees
								);
							}
							else
							{
								// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the
								// AssetUserData
								UserData->StageIdentifierToMetadata.Remove(GetSkeletonPrim().GetStage().GetRootLayer().GetIdentifier());
							}
						}
					}

					if (bIsNew)
					{
						const bool bMaterialsHaveChanged = UsdSkelSkeletonTranslatorImpl::ProcessMaterials(
							GetSkeletonPrim(),
							LODIndexToMaterialInfo,
							SkeletalMesh,
							*Context->UsdAssetCache,
							*Context->PrimLinkCache,
							Context->Time,
							Context->ObjectFlags,
							NewBlendShapes.Num() > 0,
							Context->bShareAssetsForIdenticalPrims
						);

						if (bMaterialsHaveChanged)
						{
							const bool bRebuildAll = true;
							SkeletalMesh->UpdateUVChannelData(bRebuildAll);
						}
					}

					if (bGeneratePhysicsAssets)
					{
						UPhysicsAsset* PhysicsAsset = SkeletalMesh->GetPhysicsAsset();

#if WITH_EDITOR
						if (!PhysicsAsset)
						{
							const FString PhysicsAssetHash = PrefixedSkelMeshHash + TEXT("_PhysicsAsset");
							const FString DesiredPhysicsAssetName = FPaths::GetBaseFilename(TEXT("PHYS_") + SkeletalMesh->GetName());
							bool bCreatedPhysicsAsset = false;
							PhysicsAsset = Context->UsdAssetCache->GetOrCreateCachedAsset<UPhysicsAsset>(
								PhysicsAssetHash,
								DesiredPhysicsAssetName,
								Context->ObjectFlags,
								&bCreatedPhysicsAsset
							);

							if (bCreatedPhysicsAsset)
							{
								FPhysAssetCreateParams NewBodyData;
								FText CreationErrorMessage;

								bool bSuccess = false;
								if (SkeletalMesh && SkeletalMesh->GetResourceForRendering())
								{
									bSuccess = FPhysicsAssetUtils::CreateFromSkeletalMesh(
										PhysicsAsset,
										SkeletalMesh,
										NewBodyData,
										CreationErrorMessage
									);
								}
								if (!bSuccess)
								{
									USD_LOG_WARNING(TEXT("Failed to create PhysicsAsset for skeletal mesh '%s'"), *SkeletalMesh->GetPathName());
									UsdUnreal::TranslatorUtils::AbandonFailedAsset(
										PhysicsAsset,
										Context->UsdAssetCache.Get(),
										Context->PrimLinkCache
									);
								}
							}
						}
#endif	  // WITH_EDITOR

						if (PhysicsAsset)
						{
							Context->PrimLinkCache->LinkAssetToPrim(SkeletonPrimPath, PhysicsAsset);
							Context->UsdAssetCache->TouchAssetPath(PhysicsAsset);
						}
					}
					else
					{
						// Actively clear this so that if we toggle the cvar and reload we'll clear our physics assets
						SkeletalMesh->SetPhysicsAsset(nullptr);
					}

					Context->PrimLinkCache->LinkAssetToPrim(SkeletonPrimPath, SkeletalMesh);

					// Track our Skeleton by the source skeleton prim path
					if (USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
					{
						Context->PrimLinkCache->LinkAssetToPrim(SkeletonPrimPath, Skeleton);
						Context->UsdAssetCache->TouchAssetPath(Skeleton);
					}

					// We may be reusing a skeletal mesh we got in the cache, but we always need the BlendShapesByPath stored on the
					// actor to be up-to-date with the Skeletal Mesh that is actually being displayed
					if (Context->BlendShapesByPath)
					{
						Context->BlendShapesByPath->Append(NewBlendShapes);
					}
				}

				// Continuing even if the mesh is not new as we currently don't add the SkelAnimation info to the mesh hash, so the animations
				// may have changed
				return true;
			}
		);

		// Create AnimBP asset if we need to
		Then(
			ESchemaTranslationLaunchPolicy::Sync,
			[this]()
			{
				UE::FUsdPrim Prim = GetSkeletonPrim();

				// We need to *also* create the AnimBP within CreateAssets so that we will still generate it even if
				// we never call Create/UpdateComponents (e.g. importing without importing actors).
				bool bPrimHasLiveLinkSchema = UsdUtils::PrimHasSchema(Prim, UnrealIdentifiers::LiveLinkAPI);
				if (!bPrimHasLiveLinkSchema)
				{
					// Temporarily check the closest skelroot prim for the schema for backwards compatibility
					bPrimHasLiveLinkSchema = UsdUtils::PrimHasSchema(ClosestParentSkelRoot.Get().GetPrim(), UnrealIdentifiers::LiveLinkAPI);
				}

				if (bPrimHasLiveLinkSchema)
				{
					TSharedPtr<FUsdSchemaTranslationContext> ContextPtr = Context;

					// When importing, we can't delay creating the AnimBP to the next frame as that will be after the
					// import. We don't need to worry about deadlocks though, because we will never *import* as a
					// response to a USD event: It's always an intentional function call, where the Python GIL is
					// properly handled
					if (ContextPtr && ContextPtr->bIsImporting)
					{
						UsdSkelSkeletonTranslatorImpl::CreateAnimBlueprint(*ContextPtr.Get(), Prim);
					}
					else
					{
						// HACK. c.f. the large comment on the analogous timer inside
						// FUsdSkelSkeletonTranslator::UpdateComponents
						FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
							[ContextPtr, Prim](float Time)
							{
								if (ContextPtr)
								{
									UsdSkelSkeletonTranslatorImpl::CreateAnimBlueprint(*ContextPtr.Get(), Prim);
								}

								// Returning false means this is a one-off, and won't repeat
								return false;
							}
						));
					}
				}

				return true;
			}
		);

		// Create UAnimSequences (requires a completed USkeleton. Main thread as some steps of the animation compression require it)
		Then(
			ESchemaTranslationLaunchPolicy::Sync,
			[this]()
			{
				if (!Context->bAllowParsingSkeletalAnimations || !Context->PrimLinkCache)
				{
					return false;
				}

				USkeletalMesh* SkeletalMesh = Context->PrimLinkCache->GetSingleAssetForPrim<USkeletalMesh>(SkeletonPrimPath);
				if (!SkeletalMesh)
				{
					return false;
				}

				FScopedUsdAllocs Allocs;

				const pxr::UsdSkelSkeleton& Skeleton = SkeletonBinding.Get().GetSkeleton();
				const pxr::UsdSkelAnimQuery& UsdAnimQuery = SkeletonQuery.Get().GetAnimQuery();
				if (!UsdAnimQuery)
				{
					return false;
				}

				pxr::UsdPrim SkelAnimationPrim = UsdAnimQuery.GetPrim();
				if (!SkelAnimationPrim)
				{
					return false;
				}
				FString SkelAnimationPrimPath = UsdToUnreal::ConvertPath(SkelAnimationPrim.GetPath());

				std::vector<double> JointTimeSamples;
				std::vector<double> BlendShapeTimeSamples;
				if ((!UsdAnimQuery.GetJointTransformTimeSamples(&JointTimeSamples) || JointTimeSamples.size() == 0)
					&& (NewBlendShapes.Num() == 0
						|| (!UsdAnimQuery.GetBlendShapeWeightTimeSamples(&BlendShapeTimeSamples) || BlendShapeTimeSamples.size() == 0)))
				{
					return false;
				}

				pxr::UsdPrim RootMotionPrim;
				switch (Context->RootMotionHandling)
				{
					case EUsdRootMotionHandling::UseMotionFromSkelRoot:
					{
						if (pxr::UsdPrim Root = UsdUtils::GetClosestParentSkelRoot(Skeleton.GetPrim()))
						{
							RootMotionPrim = Root.GetPrim();
						}
						break;
					}
					case EUsdRootMotionHandling::UseMotionFromSkeleton:
					{
						RootMotionPrim = Skeleton.GetPrim();
						break;
					}
					default:
					case EUsdRootMotionHandling::NoAdditionalRootMotion:
					{
						break;
					}
				}

				FSHAHash Hash = UsdSkelSkeletonTranslatorImpl::ComputeSHAHash(SkeletonQuery.Get(), RootMotionPrim, PrefixedSkelMeshHash);
				FString PrefixedSkelAnimHash = UsdUtils::GetAssetHashPrefix(SkelAnimationPrim, Context->bShareAssetsForIdenticalPrims)
											   + Hash.ToString();

				const FString DesiredName = UsdToUnreal::ConvertToken(SkelAnimationPrim.GetName());

				bool bAnimSequenceIsNew = false;
				UAnimSequence* AnimSequence = Context->UsdAssetCache->GetOrCreateCachedAsset<UAnimSequence>(
					PrefixedSkelAnimHash,
					DesiredName,
					Context->ObjectFlags,
					&bAnimSequenceIsNew
				);

				TOptional<float> StageStartOffsetSeconds;
				TOptional<float> StageScaleSeconds;
				if (bAnimSequenceIsNew || AnimSequence->GetSkeleton() != SkeletalMesh->GetSkeleton())
				{
					FScopedUnrealAllocs UEAllocs;

					// This is read back in the USDImporter, so that if we ever import this AnimSequence we will always also import the
					// SkeletalMesh for it
					AnimSequence->SetPreviewMesh(SkeletalMesh);
					AnimSequence->SetSkeleton(SkeletalMesh->GetSkeleton());

					TUsdStore<pxr::VtArray<pxr::UsdSkelSkinningQuery>> SkinningTargets = SkeletonBinding.Get().GetSkinningTargets();
					StageStartOffsetSeconds = 0.0f;
					StageScaleSeconds = 1.0f;
					bool bSuccess = UsdToUnreal::ConvertSkelAnim(
						SkeletonQuery.Get(),
						&SkinningTargets.Get(),
						&NewBlendShapes,
						Context->bAllowInterpretingLODs,
						RootMotionPrim,
						AnimSequence,
						&StageStartOffsetSeconds.GetValue(),
						&StageScaleSeconds.GetValue()
					);

					if (!bSuccess
						|| (AnimSequence->GetDataModel()->GetNumBoneTracks() == 0 && AnimSequence->GetDataModel()->GetNumberOfFloatCurves() == 0))
					{
						USD_LOG_WARNING(TEXT("Failed to create AnimSequence for prim '%s'"), *SkelAnimationPrimPath);
						UsdUnreal::TranslatorUtils::AbandonFailedAsset(AnimSequence, Context->UsdAssetCache.Get(), Context->PrimLinkCache);
						AnimSequence = nullptr;
					}
				}

				if (UUsdAnimSequenceAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdAnimSequenceAssetUserData>(
						AnimSequence
					))
				{
					UserData->PrimPaths.AddUnique(SkelAnimationPrimPath);

					// It should be fine that we won't fetch/set these again in case we're reusing an AnimSequence from the asset cache
					// because they are influenced by the animation start/end time codes, which should always affect the asset hash.
					// This means that if the correct value for this were to change, we'd end up generating a new AnimSequence and computing 
					// it anyway
					if (StageStartOffsetSeconds.IsSet())
					{
						UserData->StageStartOffsetSeconds = StageStartOffsetSeconds.GetValue();
					}
					if (StageScaleSeconds.IsSet())
					{
						UserData->StageScaleSeconds = StageScaleSeconds.GetValue();
					}

					if (Context->MetadataOptions.bCollectMetadata)
					{
						// Since we never collapse, we'll spawn assets components for any child prim that happens to be inside
						// the skeleton itself, and the SkelAnimation type doesn't have any relevant "child prim" type (like for
						// Mesh prims and UsdGeomSubsets), so we're probably safe in never collecting metadata from the SkelAnimation
						// prim subtree
						const bool bCollectFromEntireSubtrees = false;
						UsdToUnreal::ConvertMetadata(
							SkelAnimationPrim,
							UserData,
							Context->MetadataOptions.BlockedPrefixFilters,
							Context->MetadataOptions.bInvertFilters,
							bCollectFromEntireSubtrees
						);
					}
					else
					{
						// Strip the metadata from this prim, so that if we uncheck "Collect Metadata" it actually disappears on the
						// AssetUserData
						UserData->StageIdentifierToMetadata.Remove(
							UsdToUnreal::ConvertString(SkelAnimationPrim.GetStage()->GetRootLayer()->GetIdentifier())
						);
					}
				}

				if (AnimSequence && Context->PrimLinkCache)
				{
					Context->PrimLinkCache->LinkAssetToPrim(SkeletonPrimPath, AnimSequence);
				}

				return true;
			}
		);
	}

#endif	  // WITH_EDITOR
}	 // namespace UsdSkelSkeletonTranslatorImpl

void FUsdSkelSkeletonTranslator::CreateAssets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdSkelSkeletonTranslator::CreateAssets);

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

#if WITH_EDITOR
	Context->TranslatorTasks.Add(MakeShared<UsdSkelSkeletonTranslatorImpl::FSkelSkeletonCreateAssetsTaskChain>(Context, PrimPath));
#endif	  // WITH_EDITOR
}

USceneComponent* FUsdSkelSkeletonTranslator::CreateComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdSkelSkeletonTranslator::CreateComponents);

	USceneComponent* SceneComponent = FUsdGeomXformableTranslator::CreateComponents();

#if WITH_EDITOR
	UE::FUsdPrim SkeletonPrim = GetPrim();

	pxr::UsdPrim ClosestParentSkelRoot = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim);
	if (!ClosestParentSkelRoot)
	{
		USD_LOG_WARNING(
			TEXT("Ignoring skeleton '%s' when creating components as it is not contained within a SkelRoot prim scope"),
			*PrimPath.GetString()
		);
		return SceneComponent;
	}

	// Check if the prim has the GroomBinding schema and setup the component and assets necessary to bind the groom to the SkeletalMesh
	if (Context->bAllowParsingGroomAssets && Context->UsdAssetCache && Context->PrimLinkCache)
	{
		UE::FUsdPrim PrimWithSchema;
		if (UsdUtils::PrimHasSchema(SkeletonPrim, UnrealIdentifiers::GroomBindingAPI))
		{
			PrimWithSchema = SkeletonPrim;
		}
		// Temporarily also allow this to be on the closest skelroot for backwards compatibility
		else if (UsdUtils::PrimHasSchema(ClosestParentSkelRoot, UnrealIdentifiers::GroomBindingAPI))
		{
			PrimWithSchema = ClosestParentSkelRoot;
		}

		if (PrimWithSchema)
		{
			UsdGroomTranslatorUtils::CreateGroomBindingAsset(
				PrimWithSchema,
				*Context->UsdAssetCache,
				*Context->PrimLinkCache,
				Context->ObjectFlags,
				Context->bShareAssetsForIdenticalPrims
			);

			// For the groom binding to work, the GroomComponent must be a child of the SceneComponent
			// so the Context ParentComponent is set to the SceneComponent temporarily
			TGuardValue<USceneComponent*> ParentComponentGuard{Context->ParentComponent, SceneComponent};
			const bool bNeedsActor = false;
			UGroomComponent* GroomComponent = Cast<UGroomComponent>(
				CreateComponentsEx(TSubclassOf<USceneComponent>(UGroomComponent::StaticClass()), bNeedsActor)
			);
			if (GroomComponent)
			{
				UpdateComponents(SceneComponent);
			}
		}
	}
#endif	  // WITH_EDITOR

	return SceneComponent;
}

void FUsdSkelSkeletonTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
	USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(SceneComponent);
	if (!SkeletalMeshComponent || !Context->PrimLinkCache)
	{
		return;
	}

	UE::FUsdPrim SkeletonPrim = GetPrim();
	UE::FUsdPrim ClosestSkelRootPrim = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim);
	if (!ClosestSkelRootPrim)
	{
		USD_LOG_WARNING(
			TEXT("Ignoring skeleton '%s' when updating components as it is not contained within a SkelRoot prim scope"),
			*PrimPath.GetString()
		);
		return;
	}

	UE::FUsdPrim PrimWithLiveLinkSchema;
	if (UsdUtils::PrimHasSchema(SkeletonPrim, UnrealIdentifiers::LiveLinkAPI))
	{
		PrimWithLiveLinkSchema = SkeletonPrim;
	}
	const bool bPrimHasLiveLinkSchema = static_cast<bool>(PrimWithLiveLinkSchema);

	bool bPrimHasLiveLinkEnabled = bPrimHasLiveLinkSchema;
	if (bPrimHasLiveLinkSchema)
	{
		FScopedUsdAllocs Allocs;

		if (pxr::UsdAttribute Attr = pxr::UsdPrim{PrimWithLiveLinkSchema}.GetAttribute(UnrealIdentifiers::UnrealLiveLinkEnabled))
		{
			bPrimHasLiveLinkEnabled = Attr.Get(&bPrimHasLiveLinkEnabled) && bPrimHasLiveLinkEnabled;
		}
	}

	SkeletalMeshComponent->Modify();

	SkeletalMeshComponent->SetAnimationMode(
		bPrimHasLiveLinkEnabled			 ? EAnimationMode::AnimationBlueprint
		: Context->bSequencerIsAnimating ? EAnimationMode::AnimationCustomMode
										 : EAnimationMode::AnimationSingleNode
	);

	UE::FUsdPrim SkelAnimPrim;
	TUsdStore<pxr::UsdSkelBinding> SkeletonBindingForPrim;
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdSkelSkeletonQuery SkelQuery;
		const bool bSuccess = UsdUtils::GetSkelQueries(
			pxr::UsdSkelRoot{ClosestSkelRootPrim},
			pxr::UsdSkelSkeleton{SkeletonPrim},
			SkeletonBindingForPrim.Get(),
			SkelQuery
		);

		pxr::UsdSkelAnimQuery AnimQuery = SkelQuery.GetAnimQuery();
		if (bSuccess && AnimQuery)
		{
			SkelAnimPrim = AnimQuery.GetPrim();
		}
	}

	if (SkelAnimPrim)
	{
		UAnimSequence* TargetAnimSequence = Context->PrimLinkCache->GetSingleAssetForPrim<UAnimSequence>(PrimPath);
		if (TargetAnimSequence != SkeletalMeshComponent->AnimationData.AnimToPlay)
		{
			SkeletalMeshComponent->AnimationData.AnimToPlay = TargetAnimSequence;
			SkeletalMeshComponent->AnimationData.bSavedLooping = false;
			SkeletalMeshComponent->AnimationData.bSavedPlaying = false;
			SkeletalMeshComponent->SetAnimation(TargetAnimSequence);
		}
	}

	Super::UpdateComponents(SceneComponent);

	// We always want this, but need to be registered for this to work (Super::UpdateComponents should register us)
	const bool bNewUpdateState = true;
	SkeletalMeshComponent->SetUpdateAnimationInEditor(bNewUpdateState);

#if WITH_EDITOR
	// Re-set the skeletal mesh if we created a new one (maybe the hash changed, a skinned UsdGeomMesh was hidden, etc.)
	USkeletalMesh* TargetSkeletalMesh = Context->PrimLinkCache->GetSingleAssetForPrim<USkeletalMesh>(PrimPath);
	if (SkeletalMeshComponent->GetSkeletalMeshAsset() != TargetSkeletalMesh)
	{
		SkeletalMeshComponent->SetSkeletalMesh(TargetSkeletalMesh);

		// Handle material overrides
		if (TargetSkeletalMesh)
		{
			TArray<UMaterialInterface*> ExistingAssignments;
			for (FSkeletalMaterial& SkeletalMaterial : TargetSkeletalMesh->GetMaterials())
			{
				ExistingAssignments.Add(SkeletalMaterial.MaterialInterface);
			}

			UsdSkelSkeletonTranslatorImpl::SetMaterialOverrides(SkeletonBindingForPrim.Get(), ExistingAssignments, *SkeletalMeshComponent, *Context);
		}
	}

	if (bPrimHasLiveLinkSchema)
	{
		TSharedPtr<FUsdSchemaTranslationContext> ContextPtr = Context;
		TStrongObjectPtr<USkeletalMeshComponent> PinnedSkelMeshComponent{SkeletalMeshComponent};

		if (ContextPtr && ContextPtr->bIsImporting)
		{
			UsdSkelSkeletonTranslatorImpl::UpdateLiveLinkProperties(*ContextPtr.Get(), PinnedSkelMeshComponent.Get(), SkeletonPrim);
		}
		else
		{
			// HACK: This is a temporary work-around for a GIL deadlock. See UE-156489 and UE-156370 for more details, but
			// the long-story short of it is that at this point we may have a callstack that originates from Python,
			// triggers a USD stage notice and causes C++ code to run as our stage actor is listening to them. If we
			// cause GC to run right now (which the DuplicateObject inside UpdateLiveLinkProperties will) we may cause
			// a deadlock, as the game thread still holds the GIL and a background reference collector thread may want
			// to acquire it too. What this does is run this part of the UpdateComponents on tick, that has a callstack
			// that doesn't originate from Python, and so doesn't have the GIL locked
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[ContextPtr, PinnedSkelMeshComponent, SkeletonPrim](float Time)
				{
					if (ContextPtr)
					{
						UsdSkelSkeletonTranslatorImpl::UpdateLiveLinkProperties(*ContextPtr.Get(), PinnedSkelMeshComponent.Get(), SkeletonPrim);
					}

					// Returning false means this is a one-off, and won't repeat
					return false;
				}
			));
		}
	}

	// Update the animation state
	// Don't try animating ourselves if the sequencer is animating as it will just overwrite the animation state on next
	// tick anyway, and all this would do is lead to flickering and other issues
	if (!Context->bSequencerIsAnimating && SkeletalMeshComponent->GetSkeletalMeshAsset() && !bPrimHasLiveLinkEnabled)
	{
		if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(SkeletalMeshComponent->AnimationData.AnimToPlay.Get()))
		{
			double StageStartOffsetSeconds = 0.0f;
			double StageScaleSeconds = 1.0f;
			if (UUsdAnimSequenceAssetUserData* UserData = AnimSequence->GetAssetUserData<UUsdAnimSequenceAssetUserData>())
			{
				StageStartOffsetSeconds = UserData->StageStartOffsetSeconds;
				StageScaleSeconds = UserData->StageScaleSeconds;
			}

			// Always change the mode here because the sequencer will change it back to AnimationCustomMode when animating
			SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);

			// Always use the sequence's framerate here because we need to sample the UAnimSequence with seconds, and that
			// asset may have been created when the stage had a different framesPerSecond (and was reused by the assets cache)
			// Use the import framerate here because we will need to change the sampling framerate of the sequence in order to get it
			// to match the target duration in seconds and the number of source frames.
			const double AnimSequenceTime = Context->Time / AnimSequence->ImportFileFramerate;
			SkeletalMeshComponent->SetPosition((AnimSequenceTime - StageStartOffsetSeconds) / StageScaleSeconds);

			SkeletalMeshComponent->TickAnimation(0.f, false);
			SkeletalMeshComponent->RefreshBoneTransforms();
			SkeletalMeshComponent->RefreshFollowerComponents();
			SkeletalMeshComponent->UpdateComponentToWorld();
			SkeletalMeshComponent->FinalizeBoneTransform();
			SkeletalMeshComponent->MarkRenderTransformDirty();
			SkeletalMeshComponent->MarkRenderDynamicDataDirty();
		}
	}

	// If the prim has a GroomBinding schema, apply the target groom to its associated GroomComponent
	if (UsdUtils::PrimHasSchema(SkeletonPrim, UnrealIdentifiers::GroomBindingAPI))
	{
		UsdGroomTranslatorUtils::SetGroomFromPrim(SkeletonPrim, *Context->PrimLinkCache, SceneComponent);
	}
#endif	  // WITH_EDITOR
}

bool FUsdSkelSkeletonTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// There's no real reason to collapse: We're going to find all of our skinned meshes directly wherever they are, and other translators
	// are going to skip skinned stuff anyway. If the user happens to put an e.g. Light or Camera prim inside of the Skeleton for
	// some reason there's no reason we can't just handle it as normal
	return false;
}

bool FUsdSkelSkeletonTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	// Definetely cannot be collapsed into another StaticMesh. We could maybe be collapsed into another SkeletalMesh if we merged the
	// skeletons together, but that seems like a lot of complicated work and likely undesirable anyway: Users likely want to see their
	// skeleton prims generate dedicated USkeletons, every time
	return false;
}

TSet<UE::FSdfPath> FUsdSkelSkeletonTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	TSet<UE::FSdfPath> Result;
	{
		FScopedUsdAllocs UsdAllocs;

		pxr::UsdPrim SkeletonPrim = GetPrim();
		pxr::UsdPrim ClosestParentSkelRoot = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim);

		pxr::TfType ImageableSchema = pxr::TfType::Find<pxr::UsdGeomImageable>();
		pxr::TfType MeshSchema = pxr::TfType::Find<pxr::UsdGeomMesh>();

		pxr::UsdPrimRange PrimRange{ClosestParentSkelRoot, pxr::UsdTraverseInstanceProxies()};
		for (pxr::UsdPrimRange::iterator PrimRangeIt = PrimRange.begin(); PrimRangeIt != PrimRange.end(); ++PrimRangeIt)
		{
			if (PrimRangeIt->IsA(MeshSchema))
			{
				Result.Add(UE::FSdfPath{PrimRangeIt->GetPrimPath()});

				if (pxr::UsdSkelBindingAPI SkelBindingAPI{*PrimRangeIt})
				{
					// Collect blend shapes, which don't have to be within the Mesh or SkelRoot prim at all
					if (pxr::UsdSkelBlendShapeQuery BlendShapeQuery{SkelBindingAPI})
					{
						for (uint32 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeQuery.GetNumBlendShapes(); ++BlendShapeIndex)
						{
							if (pxr::UsdSkelBlendShape BlendShape = BlendShapeQuery.GetBlendShape(BlendShapeIndex))
							{
								Result.Add(UE::FSdfPath{BlendShape.GetPrim().GetPrimPath()});
							}
						}
					}
				}
			}
			// All meshes, xforms, skeleton are imageables. Registering any imageable is also a good idea because their
			// visibility could affect child Mesh prims, and so the combined skeletal mesh.
			else if (PrimRangeIt->IsA(ImageableSchema))
			{
				Result.Add(UE::FSdfPath{PrimRangeIt->GetPrimPath()});
			}
		}

		// Collect the bound SkelAnimation, that doesn't have to be within the actual SkelRoot
		if (ClosestParentSkelRoot)
		{
			pxr::UsdSkelBinding SkelBinding;
			pxr::UsdSkelSkeletonQuery SkelQuery;
			const bool bSuccess = UsdUtils::GetSkelQueries(
				pxr::UsdSkelRoot{ClosestParentSkelRoot},
				pxr::UsdSkelSkeleton{SkeletonPrim},
				SkelBinding,
				SkelQuery
			);

			pxr::UsdSkelAnimQuery AnimQuery = SkelQuery.GetAnimQuery();
			if (bSuccess && AnimQuery)
			{
				if (pxr::UsdPrim SkelAnimationPrim = AnimQuery.GetPrim())
				{
					Result.Add(UE::FSdfPath{SkelAnimationPrim.GetPrimPath()});
				}
			}
		}
	}
	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
