// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDTranslatorUtils.h"

#include "USDAssetCache3.h"
#include "Objects/USDPrimLinkCache.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/Ticker.h"

void UsdUnreal::TranslatorUtils::AbandonFailedAsset(UObject* Asset, UUsdAssetCache3* AssetCache, FUsdPrimLinkCache* PrimLinkCache)
{
	if (!Asset)
	{
		return;
	}

	if (AssetCache)
	{
		const FString Hash = AssetCache->GetHashForAsset(Asset);
		if (!Hash.IsEmpty())
		{
			AssetCache->StopTrackingAsset(Hash);
		}
	}

	if (PrimLinkCache)
	{
		PrimLinkCache->RemoveAllAssetPrimLinks(Asset);
	}

	// We can't call MarkPackageDirty() from an async thread, and sometimes we call AbandonFailedAsset() from schema translator task chains
	TWeakObjectPtr<UObject> AssetPtr{Asset};
	ExecuteOnGameThread(
		UE_SOURCE_LOCATION,
		[AssetPtr]()
		{
			if (UObject* Asset = AssetPtr.Get())
			{
				Asset->ClearFlags(RF_Standalone | RF_Public);

				// These come from the internals of ObjectTools::DeleteSingleObject
				Asset->MarkPackageDirty();
#if WITH_EDITOR
				FAssetRegistryModule::AssetDeleted(Asset);
#endif	  // WITH_EDITOR
				Asset->MarkAsGarbage();
			}
		}
	);
}
