// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

class UUsdAssetCache3;
class FUsdPrimLinkCache;

namespace UsdUnreal::TranslatorUtils
{
	// Properly deletes the asset and removes it from the asset and info caches, if provided
	void AbandonFailedAsset(UObject* Asset, UUsdAssetCache3* AssetCache, FUsdPrimLinkCache* PrimLinkCache);
}
