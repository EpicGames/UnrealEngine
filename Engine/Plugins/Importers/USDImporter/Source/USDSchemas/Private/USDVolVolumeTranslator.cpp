// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDVolVolumeTranslator.h"

#include "Objects/USDPrimLinkCache.h"
#include "USDAssetImportData.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDIntegrationUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDProjectSettings.h"
#include "USDShadeConversion.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "Components/HeterogeneousVolumeComponent.h"
#include "Engine/Level.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialShared.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"

#if WITH_EDITOR
#include "AssetImportTask.h"
#include "OpenVDBImportOptions.h"
#include "SparseVolumeTextureFactory.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include <pxr/usd/usd/prim.h>
#include <pxr/usd/usd/stage.h>
#include <pxr/usd/usdShade/material.h>
#include <pxr/usd/usdShade/materialBindingAPI.h>
#include <pxr/usd/usdVol/openVDBAsset.h>
#include <pxr/usd/usdVol/volume.h>
#include "USDIncludesEnd.h"

#define LOCTEXT_NAMESPACE "USDVolVolumeTranslator"

namespace UE::UsdVolVolumeTranslator::Private
{
#if WITH_EDITOR
	static const int32 SparseVolumeTextureChannelCount = 8;

	static_assert((int)UsdUtils::ESparseVolumeAttributesFormat::Unorm8 == (int)::ESparseVolumeAttributesFormat::Unorm8);
	static_assert((int)UsdUtils::ESparseVolumeAttributesFormat::Float16 == (int)::ESparseVolumeAttributesFormat::Float16);
	static_assert((int)UsdUtils::ESparseVolumeAttributesFormat::Float32 == (int)::ESparseVolumeAttributesFormat::Float32);

	struct FSparseVolumeTextureInfo
	{
		const UsdUtils::FVolumePrimInfo* InnerInfo = nullptr;
		USparseVolumeTexture* SparseVolumeTexture = nullptr;
		FString PrefixedAssetHash;
	};

	// Here we must stash into InOutPreviewData.ImportOptions the desired channel mapping for this SVT given all the GridNameToChannelNames
	// mappings we pulled out of the prims if they had any instances of our SparseVolumeTextureAPI schema
	void SetVDBImportOptions(const UsdUtils::FVolumePrimInfo& ParsedTexture, FOpenVDBPreviewData& InOutPreviewData)
	{
		// Tweak the collected filenames for other frames: The OpenVDB importer will scan for similar filenames
		// in the same folder as the main file, but through USD we expect the user to manually pick file paths
		// for each time sample (which may or may not come from the same folder, or be in any particular order)
		InOutPreviewData.SequenceFilenames = ParsedTexture.TimeSamplePaths;
		InOutPreviewData.DefaultImportOptions.bIsSequence = ParsedTexture.TimeSamplePaths.Num() > 1;

		// Apply manually specified channel formats, if any
		check(InOutPreviewData.DefaultImportOptions.Attributes.Num() == 2);
		if (ParsedTexture.AttributesAFormat.IsSet())
		{
			InOutPreviewData.DefaultImportOptions.Attributes[0].Format = static_cast<ESparseVolumeAttributesFormat>(
				ParsedTexture.AttributesAFormat.GetValue()
			);
		}
		if (ParsedTexture.AttributesBFormat.IsSet())
		{
			InOutPreviewData.DefaultImportOptions.Attributes[1].Format = static_cast<ESparseVolumeAttributesFormat>(
				ParsedTexture.AttributesBFormat.GetValue()
			);
		}

		static const TMap<FString, int32> ChannelIndexMap = {
			{TEXT("AttributesA.R"), 0},
			{TEXT("AttributesA.G"), 1},
			{TEXT("AttributesA.B"), 2},
			{TEXT("AttributesA.A"), 3},
			{TEXT("AttributesB.R"), 4},
			{TEXT("AttributesB.G"), 5},
			{TEXT("AttributesB.B"), 6},
			{TEXT("AttributesB.A"), 7}
		};

		static const TMap<FString, int32> ComponentIndexMap = {
			{TEXT("X"), 0},
			{TEXT("Y"), 1},
			{TEXT("Z"), 2},
			{TEXT("W"), 3},
			{TEXT("R"), 0},
			{TEXT("G"), 1},
			{TEXT("B"), 2},
			{TEXT("A"), 3}
		};

		// We'll use this to make sure we only try assigning one thing to each available attribute channel
		FString UsedChannels[SparseVolumeTextureChannelCount];

		TMap<FString, const FOpenVDBGridInfo*> GridInfoByName;
		for (const FOpenVDBGridInfo& GridInfo : InOutPreviewData.GridInfo)
		{
			GridInfoByName.Add(GridInfo.Name, &GridInfo);
		}

		FString AvailableGrids;
		static FString AvailableComponentNames;
		static FString AvailableChannelNames;

		// We'll collect our new mapping here and only apply to InOutPreviewData if we have a valid mapping,
		// so that we don't wipe it clean if we don't have anything valid to add back anyway
		TStaticArray<FOpenVDBSparseVolumeAttributesDesc, 2> NewChannelMapping;
		bool bHadValidMapping = false;

		for (const TPair<FString, TMap<FString, FString>>& GridToMapping : ParsedTexture.GridNameToChannelComponentMapping)
		{
			const FString& GridName = GridToMapping.Key;
			if (const FOpenVDBGridInfo* FoundGridInfo = GridInfoByName.FindRef(GridName))
			{
				int32 GridIndex = static_cast<int32>(FoundGridInfo->Index);

				for (const TPair<FString, FString>& ChannelToComponent : GridToMapping.Value)
				{
					// Get and validate the desired component as an index (e.g. whether this mapping refers to 'velocity.X' (index 0) or 'velocity.Y'
					// (index 1), etc.)
					const FString& DesiredComponent = ChannelToComponent.Value;
					int32 ComponentIndex = INDEX_NONE;
					if (const int32* FoundComponentIndex = ComponentIndexMap.Find(DesiredComponent))
					{
						ComponentIndex = *FoundComponentIndex;

						check(ComponentIndex < SparseVolumeTextureChannelCount);
						if (ComponentIndex < 0 || static_cast<uint32>(ComponentIndex) >= FoundGridInfo->NumComponents)
						{
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT(
									"InvalidComponentNameIndex",
									"Invalid component name '{0}' for grid '{1}' in VDB file '{2}', as that particular grid only has {3} components."
								),
								FText::FromString(DesiredComponent),
								FText::FromString(GridName),
								FText::FromString(ParsedTexture.SourceVDBFilePath),
								FoundGridInfo->NumComponents
							));
							ComponentIndex = INDEX_NONE;
						}
					}
					else
					{
						if (AvailableComponentNames.IsEmpty())
						{
							TArray<FString> ComponentNames;
							ComponentIndexMap.GetKeys(ComponentNames);

							AvailableComponentNames = TEXT("'") + FString::Join(ComponentNames, TEXT("', '")) + TEXT("'");
						}

						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"InvalidComponentName",
								"Desired component name '{0}' for grid '{1}' in VDB file '{2}' is not a valid component name. Avaliable component names: {3}"
							),
							FText::FromString(DesiredComponent),
							FText::FromString(GridName),
							FText::FromString(ParsedTexture.SourceVDBFilePath),
							FText::FromString(AvailableComponentNames)
						));
					}

					// Get and validate desired channel (e.g. whether this mapping means to put something on 'AttributesA.R' or 'AttributesB.A', etc.)
					const FString& DesiredChannel = ChannelToComponent.Key;
					int32 ChannelIndex = INDEX_NONE;
					if (const int32* FoundChannelIndex = ChannelIndexMap.Find(DesiredChannel))
					{
						ChannelIndex = *FoundChannelIndex;

						const bool bIsSet = !UsedChannels[ChannelIndex].IsEmpty();
						const FString GridAndComponent = GridName + TEXT(".") + DesiredComponent;

						if (bIsSet && UsedChannels[ChannelIndex] != GridAndComponent)
						{
							USD_LOG_USERWARNING(FText::Format(
								LOCTEXT(
									"AttributeChannelAlreadyUsed",
									"Cannot use attribute channel '{0}' for grid '{1}' in VDB file '{2}', as the channel is already being used for the grid and component '{3}'"
								),
								FText::FromString(DesiredChannel),
								FText::FromString(GridName),
								FText::FromString(ParsedTexture.SourceVDBFilePath),
								FText::FromString(UsedChannels[ChannelIndex])
							));
							ChannelIndex = INDEX_NONE;
						}
						else
						{
							ChannelIndex = *FoundChannelIndex;
							UsedChannels[ChannelIndex] = GridAndComponent;
						}
					}
					else
					{
						if (AvailableChannelNames.IsEmpty())
						{
							TArray<FString> ChannelNames;
							ChannelIndexMap.GetKeys(ChannelNames);

							AvailableChannelNames = TEXT("'") + FString::Join(ChannelNames, TEXT("', '")) + TEXT("'");
						}

						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"InvalidAttributeChannel",
								"Desired attribute channel '{0}' for grid '{1}' in VDB file '{2}' is not a valid channel name. Avaliable channel names: {3}"
							),
							FText::FromString(DesiredChannel),
							FText::FromString(GridName),
							FText::FromString(ParsedTexture.SourceVDBFilePath),
							FText::FromString(AvailableChannelNames)
						));
					}

					// Finally actually assign the desired grid/component mapping
					if (ComponentIndex != INDEX_NONE && GridIndex != INDEX_NONE && ChannelIndex != INDEX_NONE)
					{
						// We track ChannelIndex from 0 through 7, but it's actually two groups of 4
						int32 AttributeGroupIndex = ChannelIndex / 4;
						int32 AttributeChannelIndex = ChannelIndex % 4;

						FOpenVDBSparseVolumeComponentMapping& ComponentMapping = NewChannelMapping[AttributeGroupIndex]
																					 .Mappings[AttributeChannelIndex];
						ComponentMapping.SourceGridIndex = GridIndex;
						ComponentMapping.SourceComponentIndex = ComponentIndex;

						bHadValidMapping = true;
					}
				}
			}
			else
			{
				if (AvailableGrids.IsEmpty())
				{
					TArray<FString> GridNames;
					GridInfoByName.GetKeys(GridNames);

					if (GridNames.Num() > 0)
					{
						AvailableGrids = TEXT("'") + FString::Join(GridNames, TEXT("', '")) + TEXT("'");
					}
				}

				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT("InvalidGridName", "Faild to find grid named '{0}' inside VDB file '{1}'. Avaliable grid names: {2}"),
					FText::FromString(GridName),
					FText::FromString(ParsedTexture.SourceVDBFilePath),
					FText::FromString(AvailableGrids)
				));
			}
		}

		if (bHadValidMapping)
		{
			InOutPreviewData.DefaultImportOptions.Attributes[0].Mappings = NewChannelMapping[0].Mappings;
			InOutPreviewData.DefaultImportOptions.Attributes[1].Mappings = NewChannelMapping[1].Mappings;
		}
	}

	void HashForSparseVolumeTexture(const FOpenVDBPreviewData& PreviewData, FSHA1& InOutHash)
	{
		InOutHash.Update(
			reinterpret_cast<const uint8*>(PreviewData.LoadedFile.GetData()),
			PreviewData.LoadedFile.Num() * PreviewData.LoadedFile.GetTypeSize()
		);

		// Hash other files
		{
			FMD5 MD5;

			if (PreviewData.SequenceFilenames.Num() > 1)
			{
				// Skip first one as that should always be the "main" file, that we just hashed on PreviewData.LoadedFile above.
				// Note: This could become a performance issue if we have many large frames
				for (int32 Index = 1; Index < PreviewData.SequenceFilenames.Num(); ++Index)
				{
					const FString& FrameFilePath = PreviewData.SequenceFilenames[Index];

					// Copied from FMD5Hash::HashFileFromArchive as it doesn't expose its FMD5
					if (TUniquePtr<FArchive> Ar{IFileManager::Get().CreateFileReader(*FrameFilePath)})
					{
						TArray<uint8> LocalScratch;
						LocalScratch.SetNumUninitialized(1024 * 64);

						const int64 Size = Ar->TotalSize();
						int64 Position = 0;

						while (Position < Size)
						{
							const auto ReadNum = FMath::Min(Size - Position, (int64)LocalScratch.Num());
							Ar->Serialize(LocalScratch.GetData(), ReadNum);
							MD5.Update(LocalScratch.GetData(), ReadNum);

							Position += ReadNum;
						}
					}
				}

				FMD5Hash Hash;
				Hash.Set(MD5);
				InOutHash.Update(Hash.GetBytes(), Hash.GetSize());
			}
		}

		// The only other thing that affects the SVT asset hash is the grid to attribute channel mapping.
		// i.e. if we have another Volume prim with entirely different field names but that ends up with the same grid names
		// mapped to the same attribute channels, we want to reuse the generated SVT asset

		InOutHash.Update(
			reinterpret_cast<const uint8*>(&PreviewData.DefaultImportOptions.bIsSequence),
			sizeof(PreviewData.DefaultImportOptions.bIsSequence)
		);

		const TStaticArray<FOpenVDBSparseVolumeAttributesDesc, 2>& AttributeMapping = PreviewData.DefaultImportOptions.Attributes;
		for (int32 AttributesIndex = 0; AttributesIndex < AttributeMapping.Num(); ++AttributesIndex)
		{
			const FOpenVDBSparseVolumeAttributesDesc& AttributesDesc = AttributeMapping[AttributesIndex];

			InOutHash.Update(reinterpret_cast<const uint8*>(&AttributesDesc.Format), sizeof(AttributesDesc.Format));

			const TStaticArray<FOpenVDBSparseVolumeComponentMapping, 4>& ChannelMapping = AttributesDesc.Mappings;
			for (int32 ChannelIndex = 0; ChannelIndex < ChannelMapping.Num(); ++ChannelIndex)
			{
				const FOpenVDBSparseVolumeComponentMapping& Map = ChannelMapping[ChannelIndex];
				InOutHash.Update(reinterpret_cast<const uint8*>(&Map.SourceGridIndex), sizeof(Map.SourceGridIndex));
				InOutHash.Update(reinterpret_cast<const uint8*>(&Map.SourceComponentIndex), sizeof(Map.SourceComponentIndex));
			}
		}
	}

	void HashForVolumetricMaterial(
		const UMaterialInterface* ReferenceMaterial,
		const TMap<FString, FSparseVolumeTextureInfo*>& MaterialParameterToTexture,
		FSHA1& InOutHash
	)
	{
		FString ReferenceMaterialPathName = ReferenceMaterial->GetPathName();
		InOutHash.UpdateWithString(*ReferenceMaterialPathName, ReferenceMaterialPathName.Len());

		TArray<TPair<FString, FSparseVolumeTextureInfo*>> MaterialParameterPairs;
		MaterialParameterPairs.Reserve(MaterialParameterToTexture.Num());
		for (const TPair<FString, FSparseVolumeTextureInfo*>& Pair : MaterialParameterToTexture)
		{
			MaterialParameterPairs.Add(Pair);
		}

		// Make sure we hash our SVTs deterministically, whether they have a specific material assignment due
		// to the schema or not
		MaterialParameterPairs.Sort(
			[](const TPair<FString, FSparseVolumeTextureInfo*>& LHS, const TPair<FString, FSparseVolumeTextureInfo*>& RHS) -> bool
			{
				if (LHS.Key == RHS.Key)
				{
					return LHS.Value->PrefixedAssetHash < RHS.Value->PrefixedAssetHash;
				}
				return LHS.Key < RHS.Key;
			}
		);

		for (const TPair<FString, FSparseVolumeTextureInfo*>& Pair : MaterialParameterPairs)
		{
			InOutHash.UpdateWithString(*Pair.Key, Pair.Key.Len());
			InOutHash.UpdateWithString(*Pair.Value->PrefixedAssetHash, Pair.Value->PrefixedAssetHash.Len());
		}
	}

	// This collects a mapping describing which Sparse Volume Texture (SVT) should be assigned to each SVT material parameter
	// of the ReferenceMaterial.
	// It will prefer checking the VolumePrim for a custom schema where that is manually described, then it will fall back
	// to trying to map Volume prim field names to material parameter names, and finally will just distribute the SVTs over all
	// available parameters in alphabetical order
	TMap<FString, FSparseVolumeTextureInfo*> CollectMaterialParameterTextureAssignment(
		const pxr::UsdPrim& VolumePrim,
		const UMaterial* ReferenceMaterial,
		const TMap<FString, FSparseVolumeTextureInfo>& FilePathHashToTextureInfo
	)
	{
		TMap<FString, FSparseVolumeTextureInfo*> ResultParameterToInfo;

		if (!VolumePrim || !ReferenceMaterial || FilePathHashToTextureInfo.Num() == 0)
		{
			return ResultParameterToInfo;
		}

		FScopedUsdAllocs Allocs;

		// Collect which field was mapped to which .VDB asset (and so SVT asset)
		// A field can only be mapped to a single .VDB, but multiple fields can be mapped to the same .VDB
		TMap<FString, FSparseVolumeTextureInfo*> FieldNameToInfo;
		FieldNameToInfo.Reserve(FilePathHashToTextureInfo.Num());
		for (const TPair<FString, FSparseVolumeTextureInfo>& FilePathToInfo : FilePathHashToTextureInfo)
		{
			for (const FString& FieldName : FilePathToInfo.Value.InnerInfo->VolumeFieldNames)
			{
				// We won't be modifying UsdUtils::FVolumePrimInfo in this function anyway, it's just so that we can
				// return non-const pointers
				FieldNameToInfo.Add(FieldName, const_cast<FSparseVolumeTextureInfo*>(&FilePathToInfo.Value));
			}
		}

		// Collect material parameter assignments manually specified via the custom schema, if any
		TMultiMap<FString, FString> MaterialParameterToFieldName = UsdUtils::GetVolumeMaterialParameterToFieldNameMap(VolumePrim);
		for (const TPair<FString, FString>& Pair : MaterialParameterToFieldName)
		{
			const FString& MaterialParameter = Pair.Key;
			const FString& FieldName = Pair.Value;

			if (FSparseVolumeTextureInfo* FoundParsedTexture = FieldNameToInfo.FindRef(FieldName))
			{
				if (const FSparseVolumeTextureInfo* JobAtParam = ResultParameterToInfo.FindRef(MaterialParameter))
				{
					if (JobAtParam->SparseVolumeTexture != FoundParsedTexture->SparseVolumeTexture)
					{
						USD_LOG_USERWARNING(FText::Format(
							LOCTEXT(
								"MultipleSVTsOnSameParameter",
								"Trying to assign different Sparse Volume Textures to the same material parameter '{0}' on the material instantiated for Volume prim '{1}' and field name '{2}'! Only a single texture can be assigned to a material parameter at a time."
							),
							FText::FromString(MaterialParameter),
							FText::FromString(UsdToUnreal::ConvertPath(VolumePrim.GetPrimPath())),
							FText::FromString(FieldName)
						));
					}
				}
				else
				{
					// We won't be modifying anything in here, it's just so that we can return non-const pointers
					ResultParameterToInfo.Add(MaterialParameter, FoundParsedTexture);
				}
			}
		}

		// Collect available parameter names on this material instance
		TArray<FString> SparseVolumeTextureParameterNames = UsdUtils::GetSparseVolumeTextureParameterNames(ReferenceMaterial);

		// Validate that all parameters exist on the material, or else emit a warning
		for (const TPair<FString, FSparseVolumeTextureInfo*>& AssignmentPair : ResultParameterToInfo)
		{
			if (!SparseVolumeTextureParameterNames.Contains(AssignmentPair.Key))
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"MissingMaterialParameter",
						"Failed to assign Sparse Volume Texture '{0}' to material '{1}' as the desired material parameter '{2}' doesn't exist on it"
					),
					FText::FromString(AssignmentPair.Value->SparseVolumeTexture->GetPathName()),
					FText::FromString(ReferenceMaterial->GetPathName()),
					FText::FromString(AssignmentPair.Key)
				));
			}
		}

		bool bHadManualAssignment = MaterialParameterToFieldName.Num() > 0;

		// No manual material parameter assignment specified via custom schema: First let's assume that the field names match material parameter names
		bool bHadParameterNameMatch = false;
		if (!bHadManualAssignment)
		{
			ensure(ResultParameterToInfo.Num() == 0);

			TMap<FString, FString> CaseInsensitiveToSensitiveParameterNames;
			CaseInsensitiveToSensitiveParameterNames.Reserve(SparseVolumeTextureParameterNames.Num());
			for (const FString& ParameterName : SparseVolumeTextureParameterNames)
			{
				CaseInsensitiveToSensitiveParameterNames.Add(ParameterName.ToLower(), ParameterName);
			}

			for (const TPair<FString, FSparseVolumeTextureInfo*>& TextureJobPair : FieldNameToInfo)
			{
				const FString& FieldName = TextureJobPair.Key;

				if (const FString* CaseSensitiveParameterName = CaseInsensitiveToSensitiveParameterNames.Find(FieldName.ToLower()))
				{
					ResultParameterToInfo.Add(*CaseSensitiveParameterName, TextureJobPair.Value);
					bHadParameterNameMatch = true;
				}
			}
		}

		// Nothing yet, let's fall back to just distributing the SVTs across the available material parameter slots in alphabetical order
		if (!bHadManualAssignment && !bHadParameterNameMatch)
		{
			ensure(ResultParameterToInfo.Num() == 0);

			// If there aren't enough parameters, let the user know
			if ((SparseVolumeTextureParameterNames.Num() < FilePathHashToTextureInfo.Num()) && FilePathHashToTextureInfo.Num() > 0)
			{
				USD_LOG_USERWARNING(FText::Format(
					LOCTEXT(
						"PossibleUnassignedSVTs",
						"Material '{0}' used for prim '{1}' doesn't have enough Sparse Volume Texture params to fit all of its {2} parsed textures! Some may be left unassigned."
					),
					FText::FromString(ReferenceMaterial->GetPathName()),
					FText::FromString(UsdToUnreal::ConvertPath(VolumePrim.GetPrimPath())),
					FilePathHashToTextureInfo.Num()
				));
			}

			TArray<FString> SortedFieldNames;
			FieldNameToInfo.GetKeys(SortedFieldNames);
			SortedFieldNames.Sort();

			SparseVolumeTextureParameterNames.Sort();

			for (int32 Index = 0; Index < SortedFieldNames.Num() && Index < SparseVolumeTextureParameterNames.Num(); ++Index)
			{
				const FString& FieldName = SortedFieldNames[Index];
				const FString& ParameterName = SparseVolumeTextureParameterNames[Index];

				FSparseVolumeTextureInfo* FoundTextureJob = FieldNameToInfo.FindRef(FieldName);
				if (ensure(FoundTextureJob))
				{
					ResultParameterToInfo.Add(ParameterName, FoundTextureJob);
				}
			}
		}

		return ResultParameterToInfo;
	}

	void AssignMaterialParameters(UMaterialInstance* MaterialInstance, const TMap<FString, FSparseVolumeTextureInfo*>& ParameterToTexture)
	{
		// Now that we finally have the parameter assignment for each SVT, assign them to the materials
		for (const TPair<FString, FSparseVolumeTextureInfo*>& Pair : ParameterToTexture)
		{
			if (UMaterialInstanceConstant* Constant = Cast<UMaterialInstanceConstant>(MaterialInstance))
			{
				FMaterialParameterInfo Info;
				Info.Name = *Pair.Key;
				Constant->SetSparseVolumeTextureParameterValueEditorOnly(Info, Pair.Value->SparseVolumeTexture);
			}
			else if (UMaterialInstanceDynamic* Dynamic = Cast<UMaterialInstanceDynamic>(MaterialInstance))
			{
				Dynamic->SetSparseVolumeTextureParameterValue(*Pair.Key, Pair.Value->SparseVolumeTexture);
			}
		}
	}
#endif	  // WITH_EDITOR
}	 // namespace UE::UsdVolVolumeTranslator::Private

void FUsdVolVolumeTranslator::CreateAssets()
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdVolVolumeTranslator::CreateAssets);

	using namespace UE::UsdVolVolumeTranslator::Private;

	if (!Context->UsdAssetCache || !Context->PrimLinkCache)
	{
		return;
	}

	// Don't bother generating assets if we're going to just draw some bounds for this prim instead
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		CreateAlternativeDrawModeAssets(DrawMode);
		return;
	}

	if (!Context->bAllowParsingSparseVolumeTextures)
	{
		return;
	}

	const FString VolumePrimPathString = PrimPath.GetString();
	pxr::UsdPrim VolumePrim = GetPrim();
	pxr::UsdVolVolume Volume{VolumePrim};
	if (!Volume)
	{
		return;
	}
	pxr::UsdStageRefPtr Stage = VolumePrim.GetStage();

	FString VolumePrimHashPrefix = UsdUtils::GetAssetHashPrefix(GetPrim(), Context->bShareAssetsForIdenticalPrims);

	// Bake our stage coordinate system transform into the volume, just like we do for geometry.
	// We do this here because OpenVDB doesn't really specify a fixed coordinate system, and so we interpret it as just meaning
	// "units" in general. This means we must go through a metersPerUnit / upAxis conversion to get it into UE's world space
	const FUsdStageInfo StageInfo{Stage};
	FTransform StageInfoTransform = UsdUtils::GetUsdToUESpaceTransform(StageInfo);

	// Collect info from requested files
	const TMap<FString, UsdUtils::FVolumePrimInfo> FilePathHashToVolumeInfo = UsdUtils::GetVolumeInfoByFilePathHash(VolumePrim);

	// Move the info into another struct so that we can tack on the generated SVT and prefixed asset hash
	TMap<FString, FSparseVolumeTextureInfo> FilePathHashToSparseVolumeInfo;
	FilePathHashToSparseVolumeInfo.Reserve(FilePathHashToVolumeInfo.Num());
	for (const TPair<FString, UsdUtils::FVolumePrimInfo>& Pair : FilePathHashToVolumeInfo)
	{
		FSparseVolumeTextureInfo& SparseVolumeTextureInfo = FilePathHashToSparseVolumeInfo.FindOrAdd(Pair.Key);
		SparseVolumeTextureInfo.InnerInfo = &Pair.Value;
	}

	// Create SVT assets from the info structs
	for (TPair<FString, FSparseVolumeTextureInfo>& FilePathHashToInfo : FilePathHashToSparseVolumeInfo)
	{
		const FString& VDBFilePath = FilePathHashToInfo.Value.InnerInfo->SourceVDBFilePath;
		FSparseVolumeTextureInfo& SparseVolumeTextureInfo = FilePathHashToInfo.Value;

		if (!FPaths::FileExists(VDBFilePath))
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("MissingVDBFile", "Failed to find a VDB file at path '{0}' when parsing Volume prim '{1}'"),
				FText::FromString(VDBFilePath),
				FText::FromString(VolumePrimPathString)
			));
			continue;
		}

		// Here we're going to pick how to map between the grids from the .vdb files (each one is a separate volumetric texture,
		// like "density" or "temperature", etc.) into the SVT texture's 8 attribute channels (AttributesA.RGBA and AttributesB.RGBA),
		// and also pick the attribute channel data types.
		//
		// By default we'll defer to LoadOpenVDBPreviewData which has some heuristics based on the grid names and data types.
		// In practice these .vdb files should only have 1-3 grids each with some common names so the heuristics should hopefully
		// be fine for a sensible result.
		//
		// Users can also add a custom schema to the OpenVDBAsset prims in order to manually control how to map the grids to the SVT
		// attributes, in a similar way to how blendshapes are mapped. We'll check for those in SetVDBImportOptions
		TStrongObjectPtr<UOpenVDBImportOptionsObject> ImportOptions{NewObject<UOpenVDBImportOptionsObject>()};
		LoadOpenVDBPreviewData(VDBFilePath, &ImportOptions->PreviewData);
		SetVDBImportOptions(*SparseVolumeTextureInfo.InnerInfo, ImportOptions->PreviewData);

		// Collect a hash for this VDB asset
		FSHAHash VDBAndAssignmentHash;
		{
			FSHA1 SHA1;
			HashForSparseVolumeTexture(ImportOptions->PreviewData, SHA1);

			SHA1.Final();
			SHA1.GetHash(&VDBAndAssignmentHash.Hash[0]);
		}
		SparseVolumeTextureInfo.PrefixedAssetHash = VolumePrimHashPrefix + VDBAndAssignmentHash.ToString();

		// File path instead of prim path in case we have  multiple .vdb files in the same Volume prim
		const FString& DesiredName = FPaths::GetBaseFilename(VDBFilePath);

		bool bCreatedNew = false;
		USparseVolumeTexture* SparseVolumeTexture = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<USparseVolumeTexture>(
			SparseVolumeTextureInfo.PrefixedAssetHash,
			DesiredName,
			Context->ObjectFlags,
			[&VDBFilePath, &ImportOptions, &StageInfoTransform](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
			{
				TStrongObjectPtr<USparseVolumeTextureFactory> SparseVolumeTextureFactory{NewObject<USparseVolumeTextureFactory>()};

				// We use the asset import task to indicate it's an automated import, and also to transmit our import options
				TStrongObjectPtr<UAssetImportTask> AssetImportTask{NewObject<UAssetImportTask>()};
				AssetImportTask->Filename = VDBFilePath;
				AssetImportTask->bAutomated = true;
				AssetImportTask->bSave = false;
				AssetImportTask->Options = ImportOptions.Get();
				AssetImportTask->Factory = SparseVolumeTextureFactory.Get();
				SparseVolumeTextureFactory->SetAssetImportTask(AssetImportTask.Get());

				// Call FactoryCreateFile directly here or else the usual AssetToolsModule.Get().ImportAssetTasks()
				// workflow would end up creating a package for every asset, which we don't care about since
				// the asset cache will do that anyway
				const TCHAR* Parms = nullptr;
				bool bOperationCanceled = false;
				return Cast<USparseVolumeTexture>(SparseVolumeTextureFactory->ImportWithTransform(
					USparseVolumeTexture::StaticClass(),
					Outer,
					SanitizedName,
					FlagsToUse,
					VDBFilePath,
					Parms,
					bOperationCanceled,
					StageInfoTransform
				));
			},
			&bCreatedNew
		);
		if (!SparseVolumeTexture)
		{
			USD_LOG_WARNING(TEXT("Failed to generate Sparse Volume Texture from OpenVDB file '%s'"), *VDBFilePath);
			return;
		}

		if (bCreatedNew && SparseVolumeTexture)
		{
			SparseVolumeTexture->PostEditChange();

			if (UStreamableSparseVolumeTexture* StreamableTexture = Cast<UStreamableSparseVolumeTexture>(SparseVolumeTexture))
			{
				// Set an asset import data into the texture as it won't do that on its own, and we would otherwise
				// lose the source .vdb file information downstream
				UUsdAssetImportData* ImportData = NewObject<UUsdAssetImportData>(SparseVolumeTexture);
				ImportData->UpdateFilenameOnly(VDBFilePath);

				StreamableTexture->AssetImportData = ImportData;
			}
		}

		if (SparseVolumeTexture)
		{
			Context->PrimLinkCache->LinkAssetToPrim(PrimPath, SparseVolumeTexture);

			if (UUsdSparseVolumeTextureAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<
					UUsdSparseVolumeTextureAssetUserData>(SparseVolumeTexture))
			{
				UserData->PrimPaths.AddUnique(VolumePrimPathString);
				UserData->SourceOpenVDBAssetPrimPaths = SparseVolumeTextureInfo.InnerInfo->SourceOpenVDBAssetPrimPaths;
				UserData->TimeSamplePaths = SparseVolumeTextureInfo.InnerInfo->TimeSamplePaths;
				UserData->TimeSamplePathIndices = SparseVolumeTextureInfo.InnerInfo->TimeSamplePathIndices;
				UserData->TimeSamplePathTimeCodes = SparseVolumeTextureInfo.InnerInfo->TimeSamplePathTimeCodes;

				if (Context->MetadataOptions.bCollectMetadata)
				{
					UsdToUnreal::ConvertMetadata(
						VolumePrim,
						UserData,
						Context->MetadataOptions.BlockedPrefixFilters,
						Context->MetadataOptions.bInvertFilters,
						Context->MetadataOptions.bCollectFromEntireSubtrees
					);
				}
				else
				{
					UserData->StageIdentifierToMetadata.Remove(UsdToUnreal::ConvertString(Stage->GetRootLayer()->GetIdentifier()));
				}
			}
		}

		SparseVolumeTextureInfo.SparseVolumeTexture = SparseVolumeTexture;
	}

	// Create SVT materials
	// Sparse volume textures are really 3D textures, and our actor essentially has a 3D cube mesh and will
	// draw these textures on the level. There's one step missing: The material to use.
	//
	// By default we'll spawn an instance of a reference material that we ship, that is basically just a simple
	// volume domain material with "add" blend mode, that connects AttributesA.R to the "extinction" material output,
	// and AttributesB.RGB into "albedo".
	//
	// The default material should be enough to get "something to show up", but realistically for the correct look the
	// user would need to set up a custom material. Even more so because these .vdb files can contain grids that are
	// meant to be drawn as level sets, or float values that are meant to go through look-up tables, usually don't have
	// any color, etc.
	//
	// Volume prims are Gprims however, and can have material bindings, which is what we'll fetch below. If this happens
	// to be an UnrealMaterial, we'll try to use it as the SVT material instead of our default. We'll only need to find
	// the correct material parameter to put our SVT assets in, and once again we can use the custom schema to let the
	// user specify the correct material parameter name for each SVT, in case there are more than one option.

	UMaterialInterface* ReferenceMaterial = nullptr;

	// Check to see if the Volume prim has a material binding to an UnrealMaterial we can use
	static FName UnrealRenderContext = *UsdToUnreal::ConvertToken(UnrealIdentifiers::Unreal);
	if (Context->RenderContext == UnrealRenderContext)
	{
		pxr::TfToken MaterialPurposeToken = pxr::UsdShadeTokens->allPurpose;
		if (!Context->MaterialPurpose.IsNone())
		{
			MaterialPurposeToken = UnrealToUsd::ConvertToken(*Context->MaterialPurpose.ToString()).Get();
		}

		pxr::UsdShadeMaterialBindingAPI BindingAPI{VolumePrim};
		if (pxr::UsdShadeMaterial ShadeMaterial = BindingAPI.ComputeBoundMaterial(MaterialPurposeToken))
		{
			if (TOptional<FString> UnrealMaterial = UsdUtils::GetUnrealSurfaceOutput(ShadeMaterial.GetPrim()))
			{
				ReferenceMaterial = Cast<UMaterialInterface>(FSoftObjectPath{UnrealMaterial.GetValue()}.TryLoad());
			}
		}
	}
	// Fall back to the default SVT material instead
	const UUsdProjectSettings* ProjectSettings = nullptr;
	if (!ReferenceMaterial)
	{
		ProjectSettings = GetDefault<UUsdProjectSettings>();
		if (!ProjectSettings)
		{
			return;
		}

		ReferenceMaterial = Cast<UMaterialInterface>(ProjectSettings->ReferenceDefaultSVTMaterial.TryLoad());
	}

	const UMaterial* Material = nullptr;
	if (ReferenceMaterial)
	{
		Material = ReferenceMaterial->GetMaterial();

		// Warn in case the used material can't be used for SVTs
		if (Material && Material->MaterialDomain != EMaterialDomain::MD_Volume)
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"WrongMaterialDomain",
					"The material '{0}' used for the Volume prim '{1}' may not be capable of using Sparse Volume Textures as it does not have the Volume material domain."
				),
				FText::FromString(ReferenceMaterial->GetPathName()),
				FText::FromString(VolumePrimPathString)
			));
		}
	}
	if (!Material)
	{
		return;
	}

	TMap<FString, FSparseVolumeTextureInfo*> MaterialParameterToTexture = CollectMaterialParameterTextureAssignment(
		VolumePrim,
		Material,
		FilePathHashToSparseVolumeInfo
	);

	FSHAHash MaterialHash;
	{
		FSHA1 SHA1;
		HashForVolumetricMaterial(ReferenceMaterial, MaterialParameterToTexture, SHA1);
		if (ProjectSettings)
		{
			FString ReferencePathString = ProjectSettings->ReferenceDefaultSVTMaterial.ToString();
			SHA1.UpdateWithString(*ReferencePathString, ReferencePathString.Len());
		}
		SHA1.Final();
		SHA1.GetHash(&MaterialHash.Hash[0]);
	}
	const FString PrefixedMaterialHash = VolumePrimHashPrefix + MaterialHash.ToString();

	const FString DesiredName = FPaths::GetBaseFilename(VolumePrimPathString);

	bool bIsNew = false;
	UMaterialInstance* MaterialInstance = nullptr;
	if (GIsEditor)
	{
		// Create an UMaterialInstanceConstant

		UMaterialInstanceConstant* MIC = Context->UsdAssetCache->GetOrCreateCachedAsset<UMaterialInstanceConstant>(
			PrefixedMaterialHash,
			DesiredName,
			Context->ObjectFlags,
			&bIsNew
		);

		FMaterialUpdateContext::EOptions::Type Options = FMaterialUpdateContext::EOptions::Default;
		if (Context->Level && Context->Level->bIsAssociatingLevel)
		{
			Options = (FMaterialUpdateContext::EOptions::Type)(Options & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
		}
		FMaterialUpdateContext UpdateContext(Options, GMaxRHIShaderPlatform);
		UpdateContext.AddMaterialInstance(MIC);
		MIC->SetParentEditorOnly(ReferenceMaterial);
		MIC->PreEditChange(nullptr);
		MIC->PostEditChange();

		MaterialInstance = MIC;
	}
	else
	{
		// Create a material instance for the volume component.
		// SparseVolumeTextures can't be created at runtime so this branch should never really be taken for now, but anyway...
		// Note: Some code in FNiagaraBakerRenderer::RenderSparseVolumeTexture suggests that this workflow wouldn't really work
		// because the HeterogeneousVolumeComponent always creates its own MID from the material we give it, and creating a MID
		// from another MID doesn't really work

		bool bCreatedAsset = false;
		UMaterialInstance* MI = Context->UsdAssetCache->GetOrCreateCustomCachedAsset<UMaterialInstance>(
			PrefixedMaterialHash,
			DesiredName,
			Context->ObjectFlags | RF_Transient,	// We never want MIDs to become assets in the content browser
			[ReferenceMaterial](UPackage* Outer, FName SanitizedName, EObjectFlags FlagsToUse)
			{
				UMaterialInstanceDynamic* NewMID = UMaterialInstanceDynamic::Create(ReferenceMaterial, Outer, SanitizedName);
				NewMID->ClearFlags(NewMID->GetFlags());
				NewMID->SetFlags(FlagsToUse);
				return NewMID;
			},
			&bCreatedAsset
		);

		MI->PreEditChange(nullptr);
		MI->PostEditChange();
	}

	// Create new material instance
	if (bIsNew && MaterialInstance)
	{
		AssignMaterialParameters(MaterialInstance, MaterialParameterToTexture);
	}

	if (MaterialInstance)
	{
		Context->PrimLinkCache->LinkAssetToPrim(PrimPath, MaterialInstance);

		if (UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetOrCreateAssetUserData<UUsdAssetUserData>(MaterialInstance))
		{
			UserData->PrimPaths.AddUnique(VolumePrimPathString);

			if (Context->MetadataOptions.bCollectMetadata)
			{
				UsdToUnreal::ConvertMetadata(
					VolumePrim,
					UserData,
					Context->MetadataOptions.BlockedPrefixFilters,
					Context->MetadataOptions.bInvertFilters,
					Context->MetadataOptions.bCollectFromEntireSubtrees
				);
			}
			else
			{
				UserData->StageIdentifierToMetadata.Remove(UsdToUnreal::ConvertString(Stage->GetRootLayer()->GetIdentifier()));
			}
		}
	}
#endif	  // WITH_EDITOR
}

USceneComponent* FUsdVolVolumeTranslator::CreateComponents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdVolVolumeTranslator::CreateComponents);

	USceneComponent* SceneComponent = nullptr;

#if WITH_EDITOR
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode == EUsdDrawMode::Default)
	{
		if (Context->bAllowParsingSparseVolumeTextures)
		{
			const bool bNeedsActor = true;
			SceneComponent = CreateComponentsEx({UHeterogeneousVolumeComponent::StaticClass()}, bNeedsActor);
		}
	}
	else
	{
		SceneComponent = CreateAlternativeDrawModeComponents(DrawMode);
	}

	UpdateComponents(SceneComponent);
#endif	  // WITH_EDITOR

	return SceneComponent;
}

void FUsdVolVolumeTranslator::UpdateComponents(USceneComponent* SceneComponent)
{
#if WITH_EDITOR
	// Set volumetric material onto the spawned component
	if (UHeterogeneousVolumeComponent* VolumeComponent = Cast<UHeterogeneousVolumeComponent>(SceneComponent))
	{
		const int32 ElementIndex = 0;
		UMaterialInterface* CurrentMaterial = VolumeComponent->GetMaterial(ElementIndex);

		if (UMaterialInstance* MaterialForPrim = Context->PrimLinkCache->GetSingleAssetForPrim<UMaterialInstance>(PrimPath))
		{
			if (MaterialForPrim != CurrentMaterial)
			{
				// We need to call PostLoad here or else it won't render the material properly (reference:
				// SNiagaraVolumeTextureViewport::Construct)
				VolumeComponent->SetMaterial(ElementIndex, MaterialForPrim);
				VolumeComponent->PostLoad();

				CurrentMaterial = MaterialForPrim;
			}
		}

		// Animate first SVT parameter if we have an animated one
		if (CurrentMaterial && !Context->bSequencerIsAnimating)
		{
			TArray<FMaterialParameterInfo> ParameterInfo;
			TArray<FGuid> ParameterIds;
			CurrentMaterial->GetAllSparseVolumeTextureParameterInfo(ParameterInfo, ParameterIds);

			if (ParameterInfo.Num() > 0)
			{
				const FMaterialParameterInfo& Info = ParameterInfo[0];
				USparseVolumeTexture* SparseVolumeTexture = nullptr;
				if (CurrentMaterial->GetSparseVolumeTextureParameterValue(Info, SparseVolumeTexture) && SparseVolumeTexture)
				{
					if (SparseVolumeTexture->GetNumFrames() > 1)
					{
						if (UUsdSparseVolumeTextureAssetUserData* UserData = Cast<UUsdSparseVolumeTextureAssetUserData>(
								UsdUnreal::ObjectUtils::GetAssetUserData(SparseVolumeTexture)
							))
						{
							UE::FUsdPrim VolumePrim = GetPrim();
							UE::FUsdStage Stage = VolumePrim.GetStage();

							UE::FUsdPrim PrimForOffsetCalculation = VolumePrim;
							if (UserData->SourceOpenVDBAssetPrimPaths.Num() > 0)
							{
								const FString& FirstAssetPrimPath = UserData->SourceOpenVDBAssetPrimPaths[0];
								UE::FUsdPrim FirstAssetPrim = Stage.GetPrimAtPath(UE::FSdfPath{*FirstAssetPrimPath});
								if (FirstAssetPrim)
								{
									PrimForOffsetCalculation = FirstAssetPrim;
								}
							}

							UE::FSdfLayerOffset CombinedOffset = UsdUtils::GetPrimToStageOffset(PrimForOffsetCalculation);
							const double LayerTimeCode = ((Context->Time - CombinedOffset.Offset) / CombinedOffset.Scale);

							// The SVTs will have all the volume frames packed next to each other with no time information,
							// and are indexed by a "frame index" where 0 is the first frame and N-1 is the last frame.
							// These is also no linear interpolation: The frame index is basically floor()'d and the integer
							// value is used as the index into the Frames array
							int32 TargetIndex = 0;
							for (; TargetIndex + 1 < UserData->TimeSamplePathTimeCodes.Num(); ++TargetIndex)
							{
								if (UserData->TimeSamplePathTimeCodes[TargetIndex + 1] > LayerTimeCode)
								{
									break;
								}
							}
							TargetIndex = FMath::Clamp(TargetIndex, 0, UserData->TimeSamplePathTimeCodes.Num() - 1);

							// At this point TargetIndex points at the index of the biggest timeCode that is
							// still <= LayerTimeCode. We may have an index mapping though, like when the bRemoveDuplicates
							// cvar is true

							if (UserData->TimeSamplePathIndices.IsValidIndex(TargetIndex))
							{
								TargetIndex = UserData->TimeSamplePathIndices[TargetIndex];
							}

							// Now TargetIndex should be pointing at the index of the desired frame within the SVT
							VolumeComponent->SetFrame(static_cast<float>(TargetIndex));
						}
					}
				}
			}
		}
	}
#endif	  // WITH_EDITOR

	Super::UpdateComponents(SceneComponent);
}

bool FUsdVolVolumeTranslator::CollapsesChildren(ECollapsingType CollapsingType) const
{
	// If we have a custom draw mode, it means we should draw bounds/cards/etc. instead
	// of our entire subtree, which is basically the same thing as collapsing
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(GetPrim());
	if (DrawMode != EUsdDrawMode::Default)
	{
		return true;
	}

	return false;
}

bool FUsdVolVolumeTranslator::CanBeCollapsed(ECollapsingType CollapsingType) const
{
	return false;
}

TSet<UE::FSdfPath> FUsdVolVolumeTranslator::CollectAuxiliaryPrims() const
{
	if (!Context->bIsBuildingInfoCache)
	{
		return Context->UsdInfoCache->GetAuxiliaryPrims(PrimPath);
	}

	TSet<UE::FSdfPath> Result;
#if WITH_EDITOR
	{
		FScopedUsdAllocs UsdAllocs;

		if (pxr::UsdVolVolume Volume{GetPrim()})
		{
			const std::map<pxr::TfToken, pxr::SdfPath>& FieldMap = Volume.GetFieldPaths();
			for (std::map<pxr::TfToken, pxr::SdfPath>::const_iterator Iter = FieldMap.cbegin(); Iter != FieldMap.end(); ++Iter)
			{
				const pxr::SdfPath& AssetPrimPath = Iter->second;

				Result.Add(UE::FSdfPath{*UsdToUnreal::ConvertPath(AssetPrimPath)});
			}
		}
	}
#endif	  // WITH_EDITOR
	return Result;
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
