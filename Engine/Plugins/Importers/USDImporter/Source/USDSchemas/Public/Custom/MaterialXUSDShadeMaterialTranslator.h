// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDShadeMaterialTranslator.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/pxr.h"
#include "USDIncludesEnd.h"

class FMaterialXUsdShadeMaterialTranslator : public FUsdShadeMaterialTranslator
{
	using Super = FUsdShadeMaterialTranslator;

public:
	using FUsdShadeMaterialTranslator::FUsdShadeMaterialTranslator;

	USDSCHEMAS_API virtual void CreateAssets() override;
};

#endif	  // #if USE_USD_SDK
