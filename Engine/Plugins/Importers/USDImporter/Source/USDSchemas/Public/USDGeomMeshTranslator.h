// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDGeomXformableTranslator.h"
#include "USDMetadata.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "MeshDescription.h"

#define UE_API USDSCHEMAS_API

#if USE_USD_SDK

class UStaticMesh;
class FStaticMeshComponentRecreateRenderStateContext;

namespace UsdUtils
{
	struct FUsdPrimMaterialAssignmentInfo;
}

class FBaseBuildStaticMeshTaskChain : public FUsdSchemaTranslatorTaskChain
{
public:
	explicit FBaseBuildStaticMeshTaskChain(
		const TSharedRef<FUsdSchemaTranslationContext>& InContext,
		const UE::FSdfPath& InPrimPath
	);

protected:
	// Inputs
	UE::FSdfPath PrimPath;
	TSharedRef<FUsdSchemaTranslationContext> Context;
	TArray<FMeshDescription> LODIndexToMeshDescription;

	// Outputs
	UStaticMesh* StaticMesh = nullptr;

	// Required to prevent StaticMesh from being used for drawing while it is being rebuilt
	TSharedPtr<FStaticMeshComponentRecreateRenderStateContext> RecreateRenderStateContextPtr;

protected:
	UE::FUsdPrim GetPrim() const
	{
		return Context->Stage.GetPrimAtPath(PrimPath);
	}

	virtual void SetupTasks();
};

class FBuildStaticMeshTaskChain : public FBaseBuildStaticMeshTaskChain
{
public:
	explicit FBuildStaticMeshTaskChain(
		const TSharedRef<FUsdSchemaTranslationContext>& InContext,
		const UE::FSdfPath& InPrimPath,
		const TOptional<UE::FSdfPath>& AlternativePrimToLinkAssetsTo = {}
	);

protected:
	// Inputs
	TArray<UsdUtils::FUsdPrimMaterialAssignmentInfo> LODIndexToMaterialInfo;
	// We collect metadata early (during LOD parsing) so that we don't have to flip through
	// LODs multiple times
	FUsdCombinedPrimMetadata LODMetadata;

	// Outputs
	TOptional<UE::FSdfPath> AlternativePrimToLinkAssetsTo;
	bool bCollectedMetadata = false;

protected:
	virtual void SetupTasks() override;
};

class FGeomMeshCreateAssetsTaskChain : public FBuildStaticMeshTaskChain
{
public:
	explicit FGeomMeshCreateAssetsTaskChain(
		const TSharedRef<FUsdSchemaTranslationContext>& InContext,
		const UE::FSdfPath& PrimPath,
		const TOptional<UE::FSdfPath>& AlternativePrimToLinkAssetsTo = {},
		const FTransform& AdditionalTransform = FTransform::Identity
	);

protected:
	// Inputs
	FTransform AdditionalTransform;

protected:
	virtual void SetupTasks() override;
};

class FUsdGeomMeshTranslator : public FUsdGeomXformableTranslator
{
public:
	using Super = FUsdGeomXformableTranslator;

	using FUsdGeomXformableTranslator::FUsdGeomXformableTranslator;

	FUsdGeomMeshTranslator(const FUsdGeomMeshTranslator& Other) = delete;
	FUsdGeomMeshTranslator& operator=(const FUsdGeomMeshTranslator& Other) = delete;

	UE_API virtual void CreateAssets() override;
	UE_API virtual USceneComponent* CreateComponents() override;
	UE_API virtual void UpdateComponents(USceneComponent* SceneComponent) override;

	UE_API virtual bool CollapsesChildren(ECollapsingType CollapsingType) const override;
	UE_API virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const override;

	UE_API virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const override;

protected:
	// Because of how the GroomTranslator derives the GeometryCacheTranslator that derives this, we may end up in situations where this
	// translator is not invoked on Mesh prims directly, and so we just yield back to the Xformable translator
	UE_API bool IsMeshPrim() const;

	// Returns true if the Mesh prim we're meant to translate is inside a SkelRoot and has the SkelBindingAPI.
	// We handle skeletal data now via an UsdSkelSkeletonTranslator for each Skeleton prim we encounter (the SkelRootTranslator has been
	// deprecated). The SkeletonTranslator can find the skinned meshes for its skeleton on its own, regardless of
	// where they are, but the fact that we don't have a SkelRootTranslator collapsing anymore means we'll get FUsdGeomMeshTranslator
	// invokations on skinned mesh prims that were already handled by a SkeletonTranslator somewhere else. This check lets us avoid
	// those meshes.
	// The "bCheckForComponent" parameter exists because we may want to always skip spawning assets for a skinnable prim (as a
	// SkeletonTranslator may have already handled the mesh data), but still want to spawn a component in some cases
	// (e.g. in the edge case that the skinned mesh prim has a transform and non-skinned children that inherit it)
	UE_API bool ShouldSkipSkinnablePrim(bool bCheckForComponent = false) const;
};

#endif	  // #if USE_USD_SDK

#undef UE_API
