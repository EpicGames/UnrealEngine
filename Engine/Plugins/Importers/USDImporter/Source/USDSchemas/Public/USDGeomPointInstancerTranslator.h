// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDGeomMeshTranslator.h"
#include "USDGeomXformableTranslator.h"

#define UE_API USDSCHEMAS_API

#if USE_USD_SDK

struct FUsdSchemaTranslationContext;

class FUsdGeomPointInstancerCreateAssetsTaskChain : public FBuildStaticMeshTaskChain
{
public:
	explicit FUsdGeomPointInstancerCreateAssetsTaskChain(
		const TSharedRef<FUsdSchemaTranslationContext>& InContext,
		const UE::FSdfPath& InPrimPath,
		bool bIgnoreTopLevelTransform,
		bool bIgnoreTopLevelVisibility,
		const TOptional<UE::FSdfPath>& AlternativePrimToLinkAssetsTo = {}
	);

protected:
	virtual void SetupTasks() override;

private:
	bool bIgnoreTopLevelTransform = false;
	bool bIgnoreTopLevelVisibility = false;
};

class FUsdGeomPointInstancerTranslator : public FUsdGeomXformableTranslator
{
public:
	using Super = FUsdGeomXformableTranslator;
	using FUsdGeomXformableTranslator::FUsdGeomXformableTranslator;

	UE_API virtual void CreateAssets() override;
	UE_API virtual USceneComponent* CreateComponents() override;
	UE_API virtual void UpdateComponents(USceneComponent* SceneComponent) override;

	// Point instancers will always collapse (i.e. "be in charge of") their entire prim subtree.
	// If we have a nested point instancer situation, the parentmost point instancer will be in charge of collapsing everything
	// below it, so returning true to "CanBeCollapsed" in that case won't really help anything.
	// We only return true from CanBeCollapsed if e.g. a generic Xform prim should be able to collapse us too (like when we're
	// collapsing even simple, not-nested point instancers).
	UE_API virtual bool CollapsesChildren(ECollapsingType CollapsingType) const override;
	UE_API virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const override;

	UE_API virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const override;
};

#endif	  // #if USE_USD_SDK

#undef UE_API
