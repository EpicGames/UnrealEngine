// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Objects/USDSchemaTranslator.h"

#include "Misc/Optional.h"

#define UE_API USDSCHEMAS_API

#if USE_USD_SDK

class USceneComponent;
enum class EUsdDrawMode : int32;

class FUsdGeomXformableTranslator : public FUsdSchemaTranslator
{
public:
	using FUsdSchemaTranslator::FUsdSchemaTranslator;

	UE_API explicit FUsdGeomXformableTranslator(
		TSubclassOf<USceneComponent> InComponentTypeOverride,
		TSharedRef<FUsdSchemaTranslationContext> InContext,
		const UE::FUsdTyped& InSchema
	);

	UE_API virtual void CreateAssets() override;
	UE_API virtual USceneComponent* CreateComponents() override;
	UE_API virtual void UpdateComponents(USceneComponent* SceneComponent) override;

	UE_API virtual bool CollapsesChildren(ECollapsingType CollapsingType) const override;
	UE_API virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const override;

	UE_API virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const override;

	// If the optional parameters are not set, we'll figure them out automatically.
	UE_API USceneComponent* CreateComponentsEx(TOptional<TSubclassOf<USceneComponent>> ComponentType, TOptional<bool> bNeedsActor);

protected:
	// Creates actors, components and assets in case we have an alt draw mode like bounds or cards.
	// In theory *any* prim can have these, but we're placing these on FUsdGeomXformableTranslator as we're assuming only Xformables
	// (something drawable in the first place) can realistically use an alternative draw mode (i.e. we're not going to do much
	// placing these on a Material prim, that can't even have an Xform)
	UE_API USceneComponent* CreateAlternativeDrawModeComponents(EUsdDrawMode DrawMode);
	UE_API void UpdateAlternativeDrawModeComponents(USceneComponent* SceneComponent, EUsdDrawMode DrawMode);
	UE_API void CreateAlternativeDrawModeAssets(EUsdDrawMode DrawMode);

private:
	TOptional<TSubclassOf<USceneComponent>> ComponentTypeOverride;
};

#endif	  // #if USE_USD_SDK

#undef UE_API
