// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if USE_USD_SDK && WITH_EDITOR

#include "USDGeomMeshTranslator.h"

class FUsdGeometryCacheTranslator : public FUsdGeomMeshTranslator
{
public:
	using Super = FUsdGeomMeshTranslator;

	using FUsdGeomMeshTranslator::FUsdGeomMeshTranslator;

	FUsdGeometryCacheTranslator(const FUsdGeometryCacheTranslator& Other) = delete;
	FUsdGeometryCacheTranslator& operator=(const FUsdGeometryCacheTranslator& Other) = delete;

	USDSCHEMAS_API virtual void CreateAssets() override;
	USDSCHEMAS_API virtual USceneComponent* CreateComponents() override;
	USDSCHEMAS_API virtual void UpdateComponents(USceneComponent* SceneComponent) override;

	USDSCHEMAS_API virtual bool CollapsesChildren(ECollapsingType CollapsingType) const override;
	USDSCHEMAS_API virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const override;

	USDSCHEMAS_API virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const override;

private:
	USDSCHEMAS_API bool IsPotentialGeometryCacheRoot() const;
};

#endif	  // #if USE_USD_SDK
