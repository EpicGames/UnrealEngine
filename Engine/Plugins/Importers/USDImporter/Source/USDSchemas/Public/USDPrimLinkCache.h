// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
UE_DEPRECATED_HEADER(5.6, "Use Objects/USDPrimLinkCache.h from the USDUtilities module instead")

#include "Objects/USDPrimLinkCache.h"
