// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/ForwardDeclarations.h"

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FUsdRenderContextRegistry;
class FUsdSchemaTranslatorRegistry;

class IUsdSchemasModule : public IModuleInterface
{
public:
	UE_DEPRECATED(5.6, "Use FUsdSchemaTranslatorRegistry::Get() from Objects/USDSchemaTranslator.h file in the USDUtilities module (USDCore plugin).")
	virtual FUsdSchemaTranslatorRegistry& GetTranslatorRegistry() = 0;
};

namespace UsdUnreal::Analytics
{
	/** Collects analytics about custom schemas, unsupported native schemas, and the count of custom registered schema translators */
	USDSCHEMAS_API void CollectSchemaAnalytics(const UE::FUsdStage& Stage, const FString& EventName);
}
