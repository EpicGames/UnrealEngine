// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Objects/USDSchemaTranslator.h"

#define UE_API USDSCHEMAS_API

#if USE_USD_SDK

class FUsdShadeMaterialTranslator : public FUsdSchemaTranslator
{
public:
	using FUsdSchemaTranslator::FUsdSchemaTranslator;

	UE_API virtual void CreateAssets() override;

	UE_API virtual bool CollapsesChildren(ECollapsingType CollapsingType) const override;
	UE_API virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const override;

	UE_API virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const override;

protected:
	UE_API virtual void PostImportMaterial(const FString& PrefixedMaterialHash, UMaterialInterface* ImportedMaterial);
};

#endif	  // #if USE_USD_SDK

#undef UE_API
