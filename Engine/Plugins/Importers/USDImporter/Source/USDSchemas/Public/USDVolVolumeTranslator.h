// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDGeomXformableTranslator.h"

#define UE_API USDSCHEMAS_API

#if USE_USD_SDK

class USceneComponent;

class FUsdVolVolumeTranslator : public FUsdGeomXformableTranslator
{
	using Super = FUsdGeomXformableTranslator;

public:

	using FUsdGeomXformableTranslator::FUsdGeomXformableTranslator;

	UE_API virtual void CreateAssets() override;
	UE_API virtual USceneComponent* CreateComponents() override;
	UE_API virtual void UpdateComponents(USceneComponent* SceneComponent) override;

	UE_API virtual bool CollapsesChildren(ECollapsingType CollapsingType) const override;
	UE_API virtual bool CanBeCollapsed(ECollapsingType CollapsingType) const override;

	UE_API virtual TSet<UE::FSdfPath> CollectAuxiliaryPrims() const override;
};

#endif	  // #if USE_USD_SDK

#undef UE_API
