// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDSchemas : ModuleRules
	{
		public USDSchemas(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Boost",
					"CinematicCamera",
					"Core",
					"CoreUObject",
					"Engine",
					"GeometryCache",
					"GeometryCore", // for FitKDOP
					"HairStrandsCore",
					"InterchangeCore",
					"InterchangeEngine",
					"InterchangeFactoryNodes",
					"InterchangeNodes",
					"InterchangePipelines",
					"LiveLinkAnimationCore",
					"LiveLinkComponents",
					"LiveLinkInterface",
					"MeshDescription",
					"RenderCore",
					"RHI", // For FMaterialUpdateContext and the right way of updating material instance constants
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"UnrealUSDWrapper",
					"USDClasses",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"USDUtilities", // Public because we moved some headers here so that we can use them from Interchange
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"AudioEditor", // For USoundFactory, which we use for parsing UsdMediaSpatialAudio
						"BlueprintGraph",
						"GeometryCacheUSD",
						"HairStrandsEditor",
						"Kismet",
						"LiveLinkGraphNode",
						"MaterialEditor",
						"MeshUtilities",
						"PhysicsUtilities", // For generating UPhysicsAssets for SkeletalMeshes and ConvexDecompTool
						"PropertyEditor",
						"SparseVolumeTexture",
						"UnrealEd",
					}
				);

				if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) ||
					Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
					Target.Platform == UnrealTargetPlatform.Mac)
				{
					PrivateDependencyModuleNames.AddRange(
						new string[]
						{
							"MaterialX"
						}
					);
				}
			}

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
