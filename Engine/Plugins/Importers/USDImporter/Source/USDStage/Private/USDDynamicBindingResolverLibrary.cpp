// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDDynamicBindingResolverLibrary.h"

#include "USDStageActor.h"

#include "EngineUtils.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneDynamicBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDDynamicBindingResolverLibrary)

FMovieSceneDynamicBindingResolveResult UUsdDynamicBindingResolverLibrary::ResolveWithStageActor(
	UObject* WorldContextObject,
	const FMovieSceneDynamicBindingResolveParams& Params,
	const FString& StageActorIDNameFilter,
	const FString& RootLayerFilter,
	const FString& PrimPath
)
{
	FMovieSceneDynamicBindingResolveResult Result;

	bool bIsActorBinding = true;

	ULevelSequence* LevelSequence = Cast<ULevelSequence>(Params.RootSequence.Get());
	const FGuid& BindingGuid = Params.ObjectBindingID;
	if (LevelSequence && BindingGuid.IsValid())
	{
		// Check if we're looking for an actor or component
		// Note that this is currently not used as we only ever setup our dynamic bindings for actors
		if (UMovieScene* MovieScene = LevelSequence->GetMovieScene())
		{
			if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(BindingGuid))
			{
				if (Possessable->GetParent().IsValid())
				{
					bIsActorBinding = false;
				}
			}
		}

		const bool bNoNameFilter = StageActorIDNameFilter.IsEmpty();
		const bool bNoRootLayerFilter = RootLayerFilter.IsEmpty();

		UWorld* World = WorldContextObject->GetWorld();
		for (TActorIterator<AUsdStageActor> ActorIt(World, AUsdStageActor::StaticClass()); ActorIt; ++ActorIt)
		{
			if (AUsdStageActor* Actor = *ActorIt)
			{
				const bool bNameOK = bNoNameFilter || Actor->GetName() == StageActorIDNameFilter;
				const bool bRootLayerOK = bNoRootLayerFilter || Actor->RootLayer.FilePath == RootLayerFilter;

				if (bNameOK && bRootLayerOK)
				{
					if (USceneComponent* FoundComponent = Actor->GetGeneratedComponent(PrimPath))
					{
						if (bIsActorBinding)
						{
							Result.Objects.Add(FoundComponent->GetOwner());
							Result.Object = FoundComponent->GetOwner(); // Object deprecated 5.7
						}
						else
						{
							Result.Objects.Add(FoundComponent);
							Result.Object = FoundComponent; // Object deprecated 5.7
						}
						break;
					}
				}
			}
		}
	}

	return Result;
}
