// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDInfoCacheObject.h"

#include "Objects/USDInfoCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDInfoCacheObject)

UUsdInfoCache::UUsdInfoCache()
{
	Impl = MakePimpl<FUsdInfoCache>();
}

void UUsdInfoCache::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUsdInfoCache::Serialize);

	if (FUsdInfoCache* ImplPtr = Impl.Get())
	{
		ImplPtr->Serialize(Ar);
	}
}

FUsdInfoCache& UUsdInfoCache::GetInner()
{
	return *Impl;
}

const FUsdInfoCache& UUsdInfoCache::GetInner() const
{
	return *Impl;
}
