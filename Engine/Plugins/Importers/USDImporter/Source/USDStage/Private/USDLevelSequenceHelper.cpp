// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDLevelSequenceHelper.h"

#include "Objects/USDPrimLinkCache.h"
#include "USDAssetUserData.h"
#include "USDAttributeUtils.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDIntegrationUtils.h"
#include "USDLayerUtils.h"
#include "USDListener.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDPrimTwin.h"
#include "USDProjectSettings.h"
#include "USDSkeletalDataConversion.h"
#include "USDStageActor.h"
#include "USDTypesConversion.h"
#include "USDValueConversion.h"

#include "UsdWrappers/PcpLayerStack.h"
#include "UsdWrappers/PcpLayerStackIdentifier.h"
#include "UsdWrappers/PcpMapExpression.h"
#include "UsdWrappers/PcpNodeRef.h"
#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPayloadsProxy.h"
#include "UsdWrappers/SdfPrimSpec.h"
#include "UsdWrappers/SdfReferencesProxy.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdEditContext.h"
#include "UsdWrappers/UsdGeomBBoxCache.h"
#include "UsdWrappers/UsdGeomXformable.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdResolveInfo.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdTyped.h"

#include "Animation/AnimSequence.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/HeterogeneousVolumeComponent.h"
#include "Components/LightComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SceneComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SpotLightComponent.h"
#include "ControlRigObjectBinding.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "GroomCache.h"
#include "GroomComponent.h"
#include "HAL/IConsoleManager.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "Misc/ITransaction.h"
#include "Misc/TransactionObjectEvent.h"
#include "MovieScene.h"
#include "MovieSceneGeometryCacheSection.h"
#include "MovieSceneGeometryCacheTrack.h"
#include "MovieSceneGroomCacheSection.h"
#include "MovieSceneGroomCacheTrack.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTrack.h"
#include "Rigs/FKControlRig.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "Sound/SoundAttenuation.h"
#include "Sound/SoundBase.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Templates/SharedPointer.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Tracks/MovieSceneVisibilityTrack.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Notifications/SNotificationList.h"

#if WITH_EDITOR
#include "ControlRigBlueprintLegacy.h"
#include "Editor.h"
#include "Editor/TransBuffer.h"
#include "Exporters/AnimSeqExportOption.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencer.h"
#include "MovieSceneToolHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif	  // WITH_EDITOR

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdMedia/tokens.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

#define LOCTEXT_NAMESPACE "USDLevelSequenceHelper"

#if USE_USD_SDK

namespace UE::USDLevelSequenceHelper::Private
{
	/**
	 * Similar to FrameRate.AsFrameNumber(TimeSeconds) except that it uses RoundToDouble instead of FloorToDouble, to
	 * prevent issues with floating point precision
	 */
	FFrameNumber RoundAsFrameNumber(const FFrameRate& FrameRate, double TimeSeconds)
	{
		const double TimeAsFrame = (double(TimeSeconds) * FrameRate.Numerator) / FrameRate.Denominator;
		return FFrameNumber(static_cast<int32>(FMath::RoundToDouble(TimeAsFrame)));
	}

	/**
	 * We always want to mark the LevelSequences we spawn for non-local layers as read-only. This because our current
	 * approach is that only local layers can be written to, meaning there is no point in allowing the user to edit these
	 * sequences as those changes won't be written out to USD.
	 * We use this struct to let us temporarily set a MovieScene to ReadOnly == false while we're adding keyframes to it.
	 */
	struct FScopedReadOnlyDisable
	{
		FScopedReadOnlyDisable(UMovieScene* InMovieScene, const UE::FSdfLayer& InLayer, const UE::FUsdStage& InOwnerStage)
			: MovieScene(InMovieScene)
			, Layer(InLayer)
			, OwnerStage(InOwnerStage)
		{
#if WITH_EDITOR
			if (MovieScene)
			{
				// Keep track of movie scenes that were already read-only too: Maybe the user or some
				// other mechanism made them that way, so we'll want to put those back later
				bWasReadOnly = MovieScene->IsReadOnly();
			}

			MovieScene->SetReadOnly(false);
#endif	  // WITH_EDITOR
		}

		~FScopedReadOnlyDisable()
		{
#if WITH_EDITOR
			bool bRestoreReadOnly = bWasReadOnly;

			// If the sequence originally was ReadOnly for any reason, we know we need to put it back to ReadOnly.
			// Otherwise, we want to set it as ReadOnly only if Layer is not part of the stage's local layer stack.
			if (!bWasReadOnly && OwnerStage && Layer)
			{
				bRestoreReadOnly = !OwnerStage.HasLocalLayer(Layer);
			}

			if (bRestoreReadOnly)
			{
				MovieScene->SetReadOnly(true);
			}
#endif	  // WITH_EDITOR
		}

	private:
		FScopedReadOnlyDisable(const FScopedReadOnlyDisable& Other) = delete;
		FScopedReadOnlyDisable(FScopedReadOnlyDisable&& Other) = delete;
		FScopedReadOnlyDisable& operator=(const FScopedReadOnlyDisable& Other) = delete;
		FScopedReadOnlyDisable& operator=(FScopedReadOnlyDisable&& Other) = delete;

	private:
		bool bWasReadOnly = false;
		UMovieScene* MovieScene = nullptr;
		const UE::FSdfLayer Layer;
		const UE::FUsdStage OwnerStage;
	};

	// Like UMovieScene::FindTrack, except that if we require class T it will return a track of type T or any type that derives from T
	template<typename TrackType>
	TrackType* FindTrackTypeOrDerived(const UMovieScene* MovieScene, const FGuid& Guid, const FName& TrackName = NAME_None)
	{
		if (!MovieScene || !Guid.IsValid())
		{
			return nullptr;
		}

		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(Guid))
		{
			for (UMovieSceneTrack* Track : Binding->GetTracks())
			{
				if (TrackType* CastTrack = Cast<TrackType>(Track))
				{
					if (TrackName == NAME_None || Track->GetTrackName() == TrackName)
					{
						return CastTrack;
					}
				}
			}
		}

		return nullptr;
	}

	// Returns the UObject that is bound to the track. Will only consider possessables (and ignore spawnables)
	// since we don't currently have any workflow where an opened USD stage would interact with UE spawnables.
	UObject* LocateBoundObject(const UMovieSceneSequence& MovieSceneSequence, const FMovieScenePossessable& Possessable)
	{
		UMovieScene* MovieScene = MovieSceneSequence.GetMovieScene();
		if (!MovieScene)
		{
			return nullptr;
		}

		const FGuid& Guid = Possessable.GetGuid();
		const FGuid& ParentGuid = Possessable.GetParent();

		// If we have a parent guid, we must provide the object as a context because really the binding path
		// will just contain the component name
		UObject* ParentContext = nullptr;
		if (ParentGuid.IsValid())
		{
			if (FMovieScenePossessable* ParentPossessable = MovieScene->FindPossessable(ParentGuid))
			{
				ParentContext = LocateBoundObject(MovieSceneSequence, *ParentPossessable);
			}
		}

		TArray<UObject*, TInlineAllocator<1>> Objects;
		MovieSceneSequence.LocateBoundObjects(Guid, UE::UniversalObjectLocator::FResolveParams(ParentContext), nullptr, Objects);
		if (Objects.Num() > 0)
		{
			return Objects[0];
		}

		return nullptr;
	}

	void DisableTrack(UMovieSceneTrack* Track, UMovieScene* MovieScene, const FString& ComponentBindingString, const FString& TrackName, bool bDisable)
	{
		if (!Track || !MovieScene)
		{
			return;
		}

		if (Track->IsEvalDisabled() == bDisable)
		{
			return;
		}

		Track->Modify();
		Track->SetEvalDisabled(bDisable);
	}

#if WITH_EDITOR
	TSharedPtr<ISequencer> GetOpenedSequencerForLevelSequence(ULevelSequence* LevelSequence)
	{
		const bool bFocusIfOpen = false;
		IAssetEditorInstance* AssetEditor = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(
														  LevelSequence,
														  bFocusIfOpen
													  )
													: nullptr;
		ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		return LevelSequenceEditor ? LevelSequenceEditor->GetSequencer() : nullptr;
	}

	// Rough copy of UControlRigSequencerEditorLibrary::BakeToControlRig, except that it allows us to control which
	// sequence player is used, lets us use our own existing AnimSequence for the ControlRig track, doesn't force
	// the control rig editor mode to open and doesn't crash itself when changing the edit mode away from the
	// control rig
	bool BakeToControlRig(
		UWorld* World,
		ULevelSequence* LevelSequence,
		UClass* InClass,
		UAnimSequence* AnimSequence,
		USkeletalMeshComponent* SkeletalMeshComp,
		UAnimSeqExportOption* ExportOptions,
		bool bReduceKeys,
		float Tolerance,
		const FGuid& ComponentBinding
	)
	{
		UMovieScene* MovieScene = LevelSequence->GetMovieScene();
		if (!MovieScene || !SkeletalMeshComp || !SkeletalMeshComp->GetSkeletalMeshAsset() || !SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
		{
			return false;
		}

		bool bResult = false;
		bool bCreatedTempSequence = false;
		ALevelSequenceActor* OutActor = nullptr;
		UMovieSceneControlRigParameterTrack* Track = nullptr;
		FMovieSceneSequencePlaybackSettings Settings;

		// Always use a hidden player for this so that we don't affect/are affected by any Sequencer the user
		// may have opened. Plus, if we have sublayers and subsequences its annoying to managed the Sequencer
		// currently focused LevelSequence
		IMovieScenePlayer* Player = nullptr;
		ULevelSequencePlayer* LevelPlayer = nullptr;
		{
			Player = LevelPlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
			if (!Player || !LevelPlayer)
			{
				goto cleanup;
			}

			// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
			FFrameTime StartTime = FFrameRate::TransformTime(
				UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value,
				MovieScene->GetTickResolution(),
				MovieScene->GetDisplayRate()
			);
			LevelPlayer->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTime, EUpdatePositionMethod::Play));
		}

		MovieScene->Modify();

		// We allow baking with no AnimSequence (to allow rigging with no previous animation), so if we don't
		// have an AnimSequence yet we need to bake a temp one
		if (!AnimSequence)
		{
			bCreatedTempSequence = true;
			AnimSequence = NewObject<UAnimSequence>();
			AnimSequence->SetSkeleton(SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton());

			ExportOptions->bTransactRecording = false;

			FMovieSceneSequenceTransform RootToLocalTransform;
			FAnimExportSequenceParameters AESP;
			AESP.Player = Player;
			AESP.RootToLocalTransform = RootToLocalTransform;
			AESP.MovieSceneSequence = LevelSequence;
			AESP.RootMovieSceneSequence = LevelSequence;
			bResult = MovieSceneToolHelpers::ExportToAnimSequence(AnimSequence, ExportOptions, AESP, SkeletalMeshComp);
			if (!bResult)
			{
				goto cleanup;
			}
		}

		// Disable any extra existing control rig tracks for this binding.
		// Reuse one of the control rig parameter tracks if we can
		{
			TArray<UMovieSceneTrack*> Tracks = MovieScene
												   ->FindTracks(UMovieSceneControlRigParameterTrack::StaticClass(), ComponentBinding, NAME_None);
			for (UMovieSceneTrack* AnyOleTrack : Tracks)
			{
				UMovieSceneControlRigParameterTrack* ValidTrack = Cast<UMovieSceneControlRigParameterTrack>(AnyOleTrack);
				if (ValidTrack)
				{
					Track = ValidTrack;
					Track->Modify();
					for (UMovieSceneSection* Section : Track->GetAllSections())
					{
						Section->SetIsActive(false);
					}
				}
			}

			if (!Track)
			{
				Track = Cast<UMovieSceneControlRigParameterTrack>(
					MovieScene->AddTrack(UMovieSceneControlRigParameterTrack::StaticClass(), ComponentBinding)
				);
				Track->Modify();
			}
		}

		if (Track)
		{
			FString ObjectName = InClass->GetName();
			ObjectName.RemoveFromEnd(TEXT("_C"));
			UControlRig* ControlRig = NewObject<UControlRig>(Track, InClass, FName(*ObjectName), RF_Transactional);
			if (InClass != UFKControlRig::StaticClass() && !ControlRig->SupportsEvent(TEXT("Backwards Solve")))
			{
				MovieScene->RemoveTrack(*Track);
				goto cleanup;
			}

			ControlRig->Modify();
			ControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
			ControlRig->GetObjectBinding()->BindToObject(SkeletalMeshComp);
			ControlRig->GetDataSourceRegistry()->RegisterDataSource(UControlRig::OwnerComponent, ControlRig->GetObjectBinding()->GetBoundObject());
			ControlRig->Initialize();
			ControlRig->RequestInit();
			ControlRig->SetBoneInitialTransformsFromSkeletalMeshComponent(SkeletalMeshComp, true);
			ControlRig->Evaluate_AnyThread();

			// Find the animation section's start frame, or else the baked control rig tracks will always be
			// placed at the start of the movie scene playback range, instead of following where the actual
			// animation section is
			bool bFoundAtLeastOneSection = false;
			FFrameNumber ControlRigSectionStartFrame = TNumericLimits<int32>::Max();
			UMovieSceneSkeletalAnimationTrack* SkelTrack = Cast<UMovieSceneSkeletalAnimationTrack>(
				MovieScene->FindTrack(UMovieSceneSkeletalAnimationTrack::StaticClass(), ComponentBinding, NAME_None)
			);
			if (SkelTrack)
			{
				for (const UMovieSceneSection* Section : SkelTrack->GetAllSections())
				{
					if (const UMovieSceneSkeletalAnimationSection* SkelSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
					{
						if (SkelSection->Params.Animation == AnimSequence)
						{
							TRange<FFrameNumber> Range = SkelSection->ComputeEffectiveRange();
							if (Range.HasLowerBound())
							{
								bFoundAtLeastOneSection = true;
								ControlRigSectionStartFrame = FMath::Min(ControlRigSectionStartFrame, Range.GetLowerBoundValue());
								break;
							}
						}
					}
				}
			}
			if (!bFoundAtLeastOneSection)
			{
				ControlRigSectionStartFrame = 0;
			}

			// This is unused
			const FFrameNumber StartTime = 0;
			const bool bSequencerOwnsControlRig = true;
			UMovieSceneSection* NewSection = Track->CreateControlRigSection(StartTime, ControlRig, bSequencerOwnsControlRig);
			UMovieSceneControlRigParameterSection* ParamSection = Cast<UMovieSceneControlRigParameterSection>(NewSection);

			Track->SetTrackName(FName(*ObjectName));
			Track->SetDisplayName(FText::FromString(ObjectName));

			const FFrameNumber SequenceStart{0};
			UMovieSceneControlRigParameterSection::FLoadAnimSequenceData Data;
			Data.bKeyReduce = bReduceKeys;
			Data.Tolerance = Tolerance;
			Data.bResetControls = true;
			Data.StartFrame = ControlRigSectionStartFrame;
			ParamSection->LoadAnimSequenceIntoThisSection(
				AnimSequence,
				SequenceStart,
				MovieScene,
				SkeletalMeshComp,
				Data,
				EMovieSceneKeyInterpolation::SmartAuto
			);

			// Disable Skeletal Animation Tracks
			if (SkelTrack)
			{
				SkelTrack->Modify();

				for (UMovieSceneSection* Section : SkelTrack->GetAllSections())
				{
					if (Section)
					{
						Section->TryModify();
						Section->SetIsActive(false);
					}
				}
			}

			bResult = true;
		}

cleanup:
		if (bCreatedTempSequence && AnimSequence)
		{
			// Animation compression happens asynchonously so here we make sure it's completed before marking it as garbage
			const bool bWantResults = false;
			AnimSequence->WaitOnExistingCompression(bWantResults);
			AnimSequence->MarkAsGarbage();
		}

		if (LevelPlayer)
		{
			LevelPlayer->Stop();
		}

		if (OutActor && World)
		{
			World->DestroyActor(OutActor);
		}

		return bResult;
	}

	void ShowTransformTrackOnCameraComponentWarning(const USceneComponent* Component)
	{
		const UCineCameraComponent* CameraComponent = Cast<const UCineCameraComponent>(Component);
		if (!CameraComponent)
		{
			return;
		}
		const AActor* OwnerActor = CameraComponent->GetOwner();
		if (!OwnerActor)
		{
			return;
		}

		FObjectKey NewComponentKey{Component};
		static TSet<FObjectKey> WarnedComponents;
		if (WarnedComponents.Contains(NewComponentKey))
		{
			return;
		}
		WarnedComponents.Add(NewComponentKey);

		const FText Text = LOCTEXT("TransformTrackOnCameraComponentText", "USD: Transform track on camera component");

		const FText SubText = FText::Format(
			LOCTEXT(
				"TransformTrackOnCameraComponentSubText",
				"The LevelSequence binding to the camera component '{0}' has a transform track, which is not supported.\n\nFor animating camera "
				"transforms, please bind a transform track to the '{1}' CameraActor directly, or to its root scene component instead."
			),
			FText::FromString(Component->GetName()),
			FText::FromString(OwnerActor->GetActorLabel())
		);

		USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings && Settings->bShowTransformTrackOnCameraComponentWarning)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(Text);
			Toast.SubText = SubText;
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;
			Toast.ButtonDetails.Emplace(
				LOCTEXT("OverridenOpinionMessageOk", "Ok"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);
			// This is flipped because the default checkbox message is "Don't prompt again"
			Toast.CheckBoxState = Settings->bShowTransformTrackOnCameraComponentWarning ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
				[](ECheckBoxState NewState)
				{
					if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
					{
						// This is flipped because the default checkbox message is "Don't prompt again"
						Settings->bShowTransformTrackOnCameraComponentWarning = NewState == ECheckBoxState::Unchecked;
						Settings->SaveConfig();
					}
				}
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}

	void ShowStageActorPropertyTrackWarning(FName PropertyName)
	{
		const FText Text = LOCTEXT("TrackUnboundTitle", "USD: Failed to bind property");

		const FText SubText = FText::Format(
			LOCTEXT(
				"TrackUnboundMessage",
				"Cannot bind the Stage Actor property '{0}' to it's own Level Sequence!\n\nThis sequence is an analogue for animation contained in the USD stage. For now it is not possible to create bindings or bind tracks that cannot be translated back into USD information."
			),
			FText::FromName(PropertyName)
		);

		USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

		static TWeakPtr<SNotificationItem> Notification;

		FNotificationInfo Toast(Text);
		Toast.SubText = SubText;
		Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		Toast.bUseLargeFont = false;
		Toast.bFireAndForget = false;
		Toast.FadeOutDuration = 0.0f;
		Toast.ExpireDuration = 0.0f;
		Toast.bUseThrobber = false;
		Toast.bUseSuccessFailIcons = false;
		Toast.ButtonDetails.Emplace(
			LOCTEXT("TrackUnboundOk", "Ok"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda(
				[]()
				{
					if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
					{
						PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
						PinnedNotification->ExpireAndFadeout();
					}
				}
			)
		);

		// Only show one at a time
		if (!Notification.IsValid())
		{
			Notification = FSlateNotificationManager::Get().AddNotification(Toast);
		}

		if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
		{
			PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}

	void ShowVisibilityWarningIfNeeded(const UMovieScenePropertyTrack* PropertyTrack, const UE::FUsdPrim& UsdPrim)
	{
		if (!PropertyTrack || !UsdPrim)
		{
			return;
		}

		FName PropertyPath = PropertyTrack->GetPropertyName();
		if (PropertyPath != UnrealIdentifiers::HiddenPropertyName && PropertyPath != UnrealIdentifiers::HiddenInGamePropertyName)
		{
			return;
		}

		// Only show the warning after we have at least one key in the track, otherwise pressing 'File->Regenerate sequence' will
		// really just wipe the empty tracks and bindings in the first place...
		bool bHasKeys = false;
		const TArray<UMovieSceneSection*>& Sections = PropertyTrack->GetAllSections();
		for (UMovieSceneSection* Section : Sections)
		{
			if (Section->IsActive())
			{
				FMovieSceneChannelProxy& Proxy = Section->GetChannelProxy();
				for (int32 ChannelIndex = 0; ChannelIndex < Proxy.NumChannels(); ++ChannelIndex)
				{
					if (FMovieSceneBoolChannel* Channel = Proxy.GetChannel<FMovieSceneBoolChannel>(ChannelIndex))
					{
						if (Channel->GetNumKeys() > 0)
						{
							bHasKeys = true;
							break;
						}
					}
				}
			}

			if (bHasKeys)
			{
				break;
			}
		}
		if (!bHasKeys)
		{
			return;
		}

		const static FString VisibilityAttrName = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->visibility);
		UE::FUsdAttribute VisibilityAttr = UsdPrim.GetAttribute(*VisibilityAttrName);
		if (!VisibilityAttr || VisibilityAttr.GetNumTimeSamples() > 0)
		{
			// Only show the warning when first creating a visibility track for something that doesn't
			// previously have any. Presumably if the visibility animation comes from USD the user is already aware
			// of the differences between UE/USD given all the additional tracks we'll generate on the UE side
			return;
		}

		const FText Text = LOCTEXT("VisibilityWarningTitle", "USD: Inherited visibility");

		const FText SubText = LOCTEXT(
			"VisibilityWarningTooltip",
			"Visibility in USD is inherited (if a parent prim is hidden, its children are also implicitly hidden), while it is not inherited in Unreal. This means that authoring visibility animation from Unreal may have unexpected consequences on the USD stage.\n\nYou may want to use 'File -> Regenerate sequence' to resynchronize the LevelSequence with the current state of the stage, whenever is convenient."
		);

		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings && Settings->bShowInheritedVisibilityWarning)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(Text);
			Toast.SubText = SubText;
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;
			Toast.ButtonDetails.Emplace(
				LOCTEXT("OverridenOpinionMessageOk", "Ok"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);
			// This is flipped because the default checkbox message is "Don't prompt again"
			Toast.CheckBoxState = Settings->bShowInheritedVisibilityWarning ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
				[](ECheckBoxState NewState)
				{
					if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
					{
						// This is flipped because the default checkbox message is "Don't prompt again"
						Settings->bShowInheritedVisibilityWarning = NewState == ECheckBoxState::Unchecked;
						Settings->SaveConfig();
					}
				}
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}
#endif	  // WITH_EDITOR
}	 // namespace UE::USDLevelSequenceHelper::Private

class FUsdLevelSequenceHelperImpl : private FGCObject
{
public:
	FUsdLevelSequenceHelperImpl();
	~FUsdLevelSequenceHelperImpl();

	ULevelSequence* Init(const UE::FUsdStage& InUsdStage);
	bool Serialize(FArchive& Ar);
	void SetPrimLinkCache(UUsdPrimLinkCache* PrimLinkCache);
	void SetBBoxCache(TSharedPtr<UE::FUsdGeomBBoxCache> InBBoxCache);
	bool HasData() const;
	void Clear();

private:
	struct FLayerTimeInfo
	{
		FString Identifier;
		FString FilePath;

		// Time offsets for LevelSequence subsections that correspond to sublayers
		TMap<FString, UE::FSdfLayerOffset> SublayerIdentifierToOffsets;

		// Time offsets for LevelSequence subsections that correspond to prim reference composition arcs
		// TODO: Instead of layer identifier, use a struct with layer identifier + referencer prim path
		TMap<FString, UE::FSdfLayerOffset> ReferenceIdentifierToOffsets;

		// Time offsets for LevelSequence subsections that correspond to prim payload composition arcs
		TMap<FString, UE::FSdfLayerOffset> PayloadIdentifierToOffsets;

		TOptional<double> StartTimeCode;
		TOptional<double> EndTimeCode;

		bool IsAnimated() const
		{
			return !FMath::IsNearlyEqual(StartTimeCode.Get(0.0), EndTimeCode.Get(0.0));
		}
	};

	// FGCObject interface
protected:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FUsdLevelSequenceHelperImpl");
	}

	// Sequences handling
public:

	void CreateSequencesForLayerStack(const UE::FSdfLayer& RootLayer);
	void BindToUsdStageActor(AUsdStageActor* InStageActor);
	void UnbindFromUsdStageActor();
	EUsdRootMotionHandling GetRootMotionHandling() const;
	void SetRootMotionHandling(EUsdRootMotionHandling NewValue);
	void OnStageActorRenamed();

	void SubscribeToEditorEvents();
	void UnsubscribeToEditorEvents();

	ULevelSequence* GetMainLevelSequence() const
	{
		return MainLevelSequence;
	}
	TArray<ULevelSequence*> GetSubSequences() const
	{
		TArray<ULevelSequence*> SubSequences;
		ObjectPtrDecay(LevelSequencesByIdentifier).GenerateValueArray(SubSequences);
		SubSequences.Remove(MainLevelSequence);

		return SubSequences;
	}

	FUsdLevelSequenceHelper::FOnSkelAnimationBaked& GetOnSkelAnimationBaked()
	{
		return OnSkelAnimationBaked;
	}

	void DirtySequenceHierarchyCache();
	const FMovieSceneSequenceTransform& GetSubsequenceTransform(const ULevelSequence* Sequence);

private:
	ULevelSequence* FindOrAddSequenceForAttribute(const UE::FUsdAttribute& Attribute, UE::FSdfLayer* OutSequenceLayer = nullptr);

	// WARNING: These two functions will not create the required parent LevelSequences and SubSequence sections, and are only 
	// meant for edge cases/exceptions or internal use
	ULevelSequence* FindSequenceForIdentifier(const FString& SequenceIdentitifer) const;
	ULevelSequence* FindOrAddSequenceForLayer(const UE::FSdfLayer& Layer, bool* bOutCreatedNew = nullptr);

	ULevelSequence* CreateSequenceAndParentsRecursively(const UE::FPcpNodeRef& PcpNode, const UE::FSdfLayer& Layer);

	UE::FSdfLayer FindEditTargetForSubsequence(ULevelSequence* Sequence);

	/** Removes PrimTwin as a user of Sequence. If Sequence is now unused, remove its subsection and itself. */
	void RemoveSequenceForPrim(ULevelSequence& Sequence, const UUsdPrimTwin& PrimTwin);

private:
	TObjectPtr<ULevelSequence> MainLevelSequence;
	TMap<FString, TObjectPtr<ULevelSequence>> LevelSequencesByIdentifier;
	TMap<ULevelSequence*, FString> IdentifierByLevelSequence;

	TSet<FName> LocalLayersSequences;							 // List of sequences associated with sublayers

	bool bSequenceHierarchyCacheIsDirty = false;
	FMovieSceneSequenceHierarchy SequenceHierarchyCache;		 // Cache for the hierarchy of level sequences and subsections. Go through GetSubsequenceTransform()
																 // instead of using this directly.
	TMap<ULevelSequence*, FMovieSceneSequenceID> SequencesID;	 // Tracks the FMovieSceneSequenceID for each Sequence in the hierarchy. We assume
																 // that each Sequence is only present once in the hierarchy.
																 // TODO: This will need to change if we support multiple subsections for the same sequence

	// Sequence Name to Layer Identifier Map. Relationship: N Sequences to 1 Layer.
	TMap<FString, FString> LayerIdentifierByLevelSequenceName;

	// Sections handling
private:
	/** Returns the UMovieSceneSubSection associated with SubSequence on the Sequence UMovieSceneSubTrack if it exists */
	UMovieSceneSubSection* FindSubSequenceSection(ULevelSequence& Sequence, ULevelSequence& SubSequence);
	void CreateSubSequenceSection(ULevelSequence& Sequence, ULevelSequence& Subsequence, const UE::FSdfLayerOffset& SubLayerOffset);
	void RemoveSubSequenceSection(ULevelSequence& Sequence, ULevelSequence& SubSequence);

	/**
	 * Adjusts all subsection ends, in all sequences, to match the stage's root layer's endTimeCode
	 */
	void UpdateSubSectionTimeRanges(bool bShowToast = false);

	// Tracks handling
private:
	/** Creates a time track on the ULevelSequence corresponding to Info */
	void CreateTimeTrack(const FUsdLevelSequenceHelperImpl::FLayerTimeInfo& Info);
	void RemoveTimeTrack(const FUsdLevelSequenceHelperImpl::FLayerTimeInfo* Info);

	void AddCommonTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim, bool bForceVisibilityTracks = false);
	void AddBoundsTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim, TOptional<bool> HasAnimatedBounds = {});
	void AddCameraTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim);
	void AddLightTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim, const TSet<FName>& PropertyPathsToRead = {});
	void AddSkeletalTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim);
	void AddGroomTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim);
	void AddGeometryCacheTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim);
	void AddVolumeTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim);
	void AddAudioTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim);

	template<typename TrackType>
	TrackType* AddTrack(
		const FName& PropertyPath,
		const UUsdPrimTwin& PrimTwin,
		USceneComponent& ComponentToBind,
		ULevelSequence& Sequence,
		bool bIsMuted = false
	);

	void RemovePossessable(const UUsdPrimTwin& PrimTwin);

	// Prims handling
public:
	// If bForceVisibilityTracks is true, we'll add and bake the visibility tracks for this prim even if the
	// prim itself doesn't have animated visibility (so that we can handle its visibility in case one of its
	// parents does have visibility animations)
	void AddPrim(UUsdPrimTwin& PrimTwin, bool bForceVisibilityTracks = false, TOptional<bool> HasAnimatedBounds = {});
	void RemovePrim(const UUsdPrimTwin& PrimTwin);

	// These functions assume the skeletal animation tracks (if any) were already added to the level sequence
	void UpdateControlRigTracks(UUsdPrimTwin& PrimTwin);

private:
	// Used to track which LevelSequences are being used by which prim.
	// TODO: This was mostly used by a legacy way of guessing reference/payload offsets, and could maybe be removed?
	TMultiMap<FName, FString> PrimPathByLevelSequenceName;

	struct FPrimTwinBindings
	{
		// clang fix for std::is_default_constructible_v
		// returning false in inlined code of outer class
		FPrimTwinBindings()
		{
		}

		ULevelSequence* Sequence = nullptr;

		// For now we support one binding per component type (mostly so we can fit a binding to a scene component and
		// camera component for a Camera prim twin)
		TMap<TWeakObjectPtr<const UClass>, FGuid> ObjectClassToBindingGuid;
	};
	friend FArchive& operator<<(FArchive& Ar, FUsdLevelSequenceHelperImpl::FPrimTwinBindings& Bindings)
	{
		Ar << Bindings.Sequence;
		Ar << Bindings.ObjectClassToBindingGuid;
		return Ar;
	}

	TMap<TWeakObjectPtr<const UUsdPrimTwin>, FPrimTwinBindings> PrimTwinToBindings;

	// Time codes handling
private:
	FLayerTimeInfo& FindOrAddLayerTimeInfo(const UE::FSdfLayer& Layer);
	FLayerTimeInfo* FindLayerTimeInfo(const UE::FSdfLayer& Layer);

	/** Updates the Usd LayerOffset with new offset/scale values when Section has been moved by the user */
	void UpdateUsdLayerOffsetFromSection(const UMovieSceneSequence* Sequence, const UMovieSceneSubSection* Section);

	/** Updates LayerTimeInfo with Layer */
	void UpdateLayerTimeInfoFromLayer(FLayerTimeInfo& LayerTimeInfo, const UE::FSdfLayer& Layer);

	/** Updates MovieScene with LayerTimeInfo */
	void UpdateMovieSceneTimeRanges(UMovieScene& MovieScene, const FLayerTimeInfo& LayerTimeInfo, bool bUpdateViewRanges = true);

	double GetFramesPerSecond() const;
	double GetTimeCodesPerSecond() const;

	FGuid GetOrCreateComponentBinding(const UUsdPrimTwin& PrimTwin, USceneComponent& ComponentToBind, ULevelSequence& Sequence);

	TMap<FString, FLayerTimeInfo> LayerTimeInfosByLayerIdentifier;	  // Maps a LayerTimeInfo to a given Layer through its identifier

																	  // Changes handling
public:
	void StartMonitoringChanges()
	{
		MonitoringChangesWhenZero.Decrement();
	}
	void StopMonitoringChanges()
	{
		MonitoringChangesWhenZero.Increment();
	}
	bool IsMonitoringChanges() const
	{
		return MonitoringChangesWhenZero.GetValue() == 0;
	}

	/**
	 * Used as a fire-and-forget block that will prevent any levelsequence object (tracks, moviescene, sections, etc.) change from being written to
	 * the stage. We unblock during HandleTransactionStateChanged.
	 */
	void BlockMonitoringChangesForThisTransaction();

private:
	void OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& Event);
	void OnUsdObjectsChanged(const UsdUtils::FObjectChangesByPath& InfoChanges, const UsdUtils::FObjectChangesByPath& ResyncChanges);
	void HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState);
	void HandleMovieSceneChange(UMovieScene& MovieScene);
	void HandleSubSectionChange(UMovieSceneSubSection& Section);
	void HandleControlRigSectionChange(UMovieSceneControlRigParameterSection& Section);
	void HandleTrackChange(UMovieSceneTrack& Track, bool bIsMuteChange);
	void HandleDisplayRateChange(const double DisplayRate);

	FDelegateHandle OnObjectTransactedHandle;
	FDelegateHandle OnUsdObjectsChangedHandle;

private:
	void RefreshSequencer();

private:
	static const EObjectFlags DefaultObjFlags;
	static const double DefaultFramerate;
	static const TCHAR* TimeTrackName;
	static const double EmptySubSectionRange;	 // How many frames should an empty subsection cover, only needed so that the subsection is visible
												 // and the user can edit it

	FUsdLevelSequenceHelper::FOnSkelAnimationBaked OnSkelAnimationBaked;

	TWeakObjectPtr<AUsdStageActor> StageActor = nullptr;

	// We keep a pointer to these directly because we may be called via the
	// USDStageImporter directly, when we don't have an available actor.
	// This has to be weak or else we get a circular reference, as this will hold on the PrimLinkCache,
	// that has an Outer reference to the stage actor, that owns this
	TWeakObjectPtr<UUsdPrimLinkCache> PrimLinkCache = nullptr;
	TSharedPtr<UE::FUsdGeomBBoxCache> BBoxCache = nullptr;

	EUsdRootMotionHandling RootMotionHandling = EUsdRootMotionHandling::NoAdditionalRootMotion;
	FGuid StageActorBinding;

	// Only when this is zero we write LevelSequence object (tracks, moviescene, sections, etc.) transactions back to the USD stage
	FThreadSafeCounter MonitoringChangesWhenZero;

	// When we call BlockMonitoringChangesForThisTransaction, we record the FGuid of the current transaction. We'll early out of all
	// OnObjectTransacted calls for that transaction We keep a set here in order to remember all the blocked transactions as we're going through them
	TSet<FGuid> BlockedTransactionGuids;

	UE::FUsdStage UsdStage;
};

const EObjectFlags FUsdLevelSequenceHelperImpl::DefaultObjFlags = EObjectFlags::RF_Transactional | EObjectFlags::RF_Transient
																  | EObjectFlags::RF_Public;
const double FUsdLevelSequenceHelperImpl::DefaultFramerate = 24.0;
const TCHAR* FUsdLevelSequenceHelperImpl::TimeTrackName = TEXT("Time");
const double FUsdLevelSequenceHelperImpl::EmptySubSectionRange = 10.0;

FUsdLevelSequenceHelperImpl::FUsdLevelSequenceHelperImpl()
	: MainLevelSequence(nullptr)
{
	// Don't subscribe to editor events here: The LevelSequenceHelper is a member struct of the stage actor
	// and the USD Import Context, so we may be an Impl of a CDO, that can't really do anything with those
	// events anyway. We'll subscribe only if/when we actually receive a stage (on Init())
}

FUsdLevelSequenceHelperImpl::~FUsdLevelSequenceHelperImpl()
{
	if (StageActor.IsValid())
	{
		StageActor->GetUsdListener().GetOnObjectsChanged().Remove(OnUsdObjectsChangedHandle);
	}

	UnsubscribeToEditorEvents();
}

ULevelSequence* FUsdLevelSequenceHelperImpl::Init(const UE::FUsdStage& InUsdStage)
{
	UsdStage = InUsdStage;
	const UE::FSdfLayer RootLayer = UsdStage.GetRootLayer();

	Clear();

	// Set this first as we'll need our MainLevelSequence setup before our call to CreateSubSequenceSection, as
	// that involves calling UMovieSceneCompiledDataManager::CompileHierarchy() with it
	MainLevelSequence = FindOrAddSequenceForLayer(RootLayer);

	CreateSequencesForLayerStack(RootLayer);

	// We call Init with a default (invalid) stage to "clear", so only subscribe to these events
	// if we've actually been given a valid stage
	if (UsdStage)
	{
		SubscribeToEditorEvents();
	}
	else
	{
		UnsubscribeToEditorEvents();
	}

	return MainLevelSequence;
}

bool FUsdLevelSequenceHelperImpl::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdLevelSequenceHelperImpl::Serialize);

	Ar << MainLevelSequence;
	Ar << LevelSequencesByIdentifier;
	Ar << IdentifierByLevelSequence;
	Ar << LocalLayersSequences;
	Ar << SequencesID;
	Ar << LayerIdentifierByLevelSequenceName;
	Ar << PrimPathByLevelSequenceName;
	Ar << PrimTwinToBindings;
	Ar << RootMotionHandling;
	Ar << StageActorBinding;

	return true;
}

void FUsdLevelSequenceHelperImpl::SetPrimLinkCache(UUsdPrimLinkCache* InPrimLinkCache)
{
	// PrimLinkCache.Reset(InPrimLinkCache);
	PrimLinkCache = InPrimLinkCache;
}

void FUsdLevelSequenceHelperImpl::SetBBoxCache(TSharedPtr<UE::FUsdGeomBBoxCache> InBBoxCache)
{
	BBoxCache = InBBoxCache;
}

bool FUsdLevelSequenceHelperImpl::HasData() const
{
	if (!MainLevelSequence)
	{
		return false;
	}

	UMovieScene* MovieScene = MainLevelSequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	if (MovieScene->GetPossessableCount() > 0)
	{
		return true;
	}

	UMovieSceneSubTrack* Track = MovieScene->FindTrack<UMovieSceneSubTrack>();
	if (!Track)
	{
		return false;
	}

	for (UMovieSceneSection* Section : Track->GetAllSections())
	{
		if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			if (SubSection->GetSequence())
			{
				return true;
			}
		}
	}

	return false;
}

void FUsdLevelSequenceHelperImpl::Clear()
{
	MainLevelSequence = nullptr;
	LevelSequencesByIdentifier.Empty();
	IdentifierByLevelSequence.Empty();
	LocalLayersSequences.Empty();
	LayerIdentifierByLevelSequenceName.Empty();
	LayerTimeInfosByLayerIdentifier.Empty();
	PrimPathByLevelSequenceName.Empty();
	SequencesID.Empty();
	PrimTwinToBindings.Empty();
	SequenceHierarchyCache = FMovieSceneSequenceHierarchy();
}

void FUsdLevelSequenceHelperImpl::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(MainLevelSequence);
	Collector.AddReferencedObjects(LevelSequencesByIdentifier);
}

void FUsdLevelSequenceHelperImpl::CreateSequencesForLayerStack(const UE::FSdfLayer& RootLayer)
{
	if (!UsdStage)
	{
		return;
	}

	ULevelSequence* RootSequence = FindOrAddSequenceForLayer(RootLayer);
	if (!RootSequence)
	{
		return;
	}

	LocalLayersSequences.Add(RootSequence->GetFName());

	TFunction<void(const UE::FSdfLayer&, ULevelSequence&)> RecursivelyCreateSequencesForLayer;
	RecursivelyCreateSequencesForLayer = [&RecursivelyCreateSequencesForLayer, this](const UE::FSdfLayer& Layer, ULevelSequence& Sequence)
	{
		UMovieScene* MovieScene = Sequence.GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		for (const FString& SubLayerPath : Layer.GetSubLayerPaths())
		{
			UE::FSdfLayer SubLayer = UsdUtils::FindLayerForSubLayerPath(Layer, SubLayerPath);
			if (!SubLayer)
			{
				continue;
			}

			ULevelSequence* SubSequence = FindOrAddSequenceForLayer(SubLayer);
			if (!SubSequence)
			{
				continue;
			}

			// Make sure we don't parse an already parsed layer
			if (LocalLayersSequences.Contains(SubSequence->GetFName()))
			{
				continue;
			}
			LocalLayersSequences.Add(SubSequence->GetFName());

			// We can't reuse these between calls as we may invalidate these two pointers with the recursive calls
			const FLayerTimeInfo* LayerTimeInfo = FindLayerTimeInfo(Layer);
			const UE::FSdfLayerOffset* Offset = LayerTimeInfo->SublayerIdentifierToOffsets.Find(SubLayer.GetIdentifier());
			if (!Offset)
			{
				continue;
			}

			CreateSubSequenceSection(Sequence, *SubSequence, *Offset);

			RecursivelyCreateSequencesForLayer(SubLayer, *SubSequence);
		}
	};

	// Create level sequences for all sub layers (accessible via the main level sequence but otherwise hidden)
	RecursivelyCreateSequencesForLayer(RootLayer, *RootSequence);
}

void FUsdLevelSequenceHelperImpl::BindToUsdStageActor(AUsdStageActor* InStageActor)
{
	UnbindFromUsdStageActor();

	StageActor = InStageActor;
	SetPrimLinkCache(InStageActor ? InStageActor->PrimLinkCache : nullptr);
	SetBBoxCache(InStageActor ? InStageActor->GetBBoxCache() : nullptr);
	SetRootMotionHandling(InStageActor ? InStageActor->RootMotionHandling : EUsdRootMotionHandling::NoAdditionalRootMotion);

	if (!StageActor.IsValid() || !MainLevelSequence || !MainLevelSequence->GetMovieScene())
	{
		return;
	}

	OnUsdObjectsChangedHandle = StageActor->GetUsdListener().GetOnObjectsChanged().AddRaw(this, &FUsdLevelSequenceHelperImpl::OnUsdObjectsChanged);

	// Bind stage actor
	StageActorBinding = MainLevelSequence->GetMovieScene()->AddPossessable(
#if WITH_EDITOR
		StageActor->GetActorLabel(),
#else
		StageActor->GetName(),
#endif	  // WITH_EDITOR
		StageActor->GetClass()
	);
	MainLevelSequence->BindPossessableObject(StageActorBinding, *StageActor, StageActor->GetWorld());

	CreateTimeTrack(FindOrAddLayerTimeInfo(UsdStage.GetRootLayer()));
}

void FUsdLevelSequenceHelperImpl::UnbindFromUsdStageActor()
{
	if (UsdStage)
	{
		RemoveTimeTrack(FindLayerTimeInfo(UsdStage.GetRootLayer()));
	}

	if (MainLevelSequence && MainLevelSequence->GetMovieScene())
	{
		if (MainLevelSequence->GetMovieScene()->RemovePossessable(StageActorBinding))
		{
			MainLevelSequence->UnbindPossessableObjects(StageActorBinding);
		}
	}

	StageActorBinding = FGuid::NewGuid();

	if (StageActor.IsValid())
	{
		StageActor->GetUsdListener().GetOnObjectsChanged().Remove(OnUsdObjectsChangedHandle);
		StageActor.Reset();
	}

	SetPrimLinkCache(nullptr);
	SetRootMotionHandling(EUsdRootMotionHandling::NoAdditionalRootMotion);
}

EUsdRootMotionHandling FUsdLevelSequenceHelperImpl::GetRootMotionHandling() const
{
	return RootMotionHandling;
}

void FUsdLevelSequenceHelperImpl::SetRootMotionHandling(EUsdRootMotionHandling NewValue)
{
	RootMotionHandling = NewValue;
}

void FUsdLevelSequenceHelperImpl::OnStageActorRenamed()
{
	AUsdStageActor* StageActorPtr = StageActor.Get();
	if (!StageActorPtr)
	{
		return;
	}

	FMovieScenePossessable NewPossessable{
#if WITH_EDITOR
		StageActorPtr->GetActorLabel(),
#else
		StageActorPtr->GetName(),
#endif	  // WITH_EDITOR
		StageActorPtr->GetClass()
	};
	FGuid NewId = NewPossessable.GetGuid();

	bool bDidSomething = false;
	for (const auto& Pair : LevelSequencesByIdentifier)
	{
		ULevelSequence* Sequence = Pair.Value;
		if (!Sequence)
		{
			continue;
		}

		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		const bool bDidRenameMovieScene = MovieScene->ReplacePossessable(StageActorBinding, NewPossessable);
		if (bDidRenameMovieScene)
		{
			Sequence->UnbindPossessableObjects(NewId);
			Sequence->BindPossessableObject(NewId, *StageActorPtr, StageActorPtr->GetWorld());

			bDidSomething = true;
		}
	}

	if (bDidSomething)
	{
		StageActorBinding = NewId;
	}
}

void FUsdLevelSequenceHelperImpl::SubscribeToEditorEvents()
{
#if WITH_EDITOR
	UnsubscribeToEditorEvents();

	if (GEditor)
	{
		OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FUsdLevelSequenceHelperImpl::OnObjectTransacted);

		if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
		{
			Transactor->OnTransactionStateChanged().AddRaw(this, &FUsdLevelSequenceHelperImpl::HandleTransactionStateChanged);
		}
	}
#endif	  // WITH_EDITOR
}

void FUsdLevelSequenceHelperImpl::UnsubscribeToEditorEvents()
{
#if WITH_EDITOR
	if (GEditor)
	{
		if (OnObjectTransactedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
			OnObjectTransactedHandle.Reset();
		}

		if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
		{
			Transactor->OnTransactionStateChanged().RemoveAll(this);
		}
	}
#endif	  // WITH_EDITOR
}

void FUsdLevelSequenceHelperImpl::DirtySequenceHierarchyCache()
{
	bSequenceHierarchyCacheIsDirty = true;
}

const FMovieSceneSequenceTransform& FUsdLevelSequenceHelperImpl::GetSubsequenceTransform(const ULevelSequence* Sequence)
{
	const static FMovieSceneSequenceTransform IdentityTransform;

	if (!Sequence)
	{
		return IdentityTransform;
	}

	if (bSequenceHierarchyCacheIsDirty)
	{
		bSequenceHierarchyCacheIsDirty = false;

		SequenceHierarchyCache = FMovieSceneSequenceHierarchy();
		UMovieSceneCompiledDataManager::CompileHierarchy(MainLevelSequence, &SequenceHierarchyCache, EMovieSceneServerClientMask::All);

		SequencesID.Reset();
		SequencesID.Add(MainLevelSequence) = MovieSceneSequenceID::Root;
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : SequenceHierarchyCache.AllSubSequenceData())
		{
			if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(Pair.Value.GetSequence()))
			{
				SequencesID.Add(LevelSequence, Pair.Key);
			}
		}
	}

	// Early out here to also catch the case where we just have MainLevelSequence and no subsequences. In that case our SequencesID
	// will be empty (as we never created a subsection) and so will our SequenceHierarchyCache
	if (Sequence == MainLevelSequence)
	{
		return IdentityTransform;
	}

	FMovieSceneSequenceID SequenceID = SequencesID.FindRef(Sequence);
	if (const FMovieSceneSubSequenceData* FoundData = SequenceHierarchyCache.FindSubData(SequenceID))
	{
		return FoundData->RootToSequenceTransform;
	}
	else
	{
		// If our main LevelSequence doesn't have any subsequence sections, the SequenceHierarchyCache will be fully empty and we will fail.
		// Even in that case though, don't show a warning when returning the request for the root as that is meant to be the identity transform
		// anyway.
		USD_LOG_WARNING(TEXT("Failed to find LevelSequence '%s' in SequenceHierarchyCache"), *Sequence->GetPathName());
	}

	return IdentityTransform;
}

ULevelSequence* FUsdLevelSequenceHelperImpl::FindOrAddSequenceForAttribute(const UE::FUsdAttribute& Attribute, UE::FSdfLayer* OutSequenceLayer)
{
	using namespace UE;

	// Get layer with strongest attribute spec
	FSdfLayer AttributeLayer = UsdUtils::FindLayerForAttribute(Attribute, UsdUtils::GetEarliestTimeCode());
	if (!AttributeLayer)
	{
		return nullptr;
	}

	// This gets the leafmost composition arc node for the strongest opinion, corresponding to the opinion that also led us to AttributeLayer
	FUsdResolveInfo ResolveInfo = Attribute.GetResolveInfo(UsdUtils::GetEarliestTimeCode());
	FPcpNodeRef Node = ResolveInfo.GetNode();

	ULevelSequence* AttributeSequence = CreateSequenceAndParentsRecursively(Node, AttributeLayer);
	if (AttributeSequence && OutSequenceLayer)
	{
		*OutSequenceLayer = AttributeLayer;
	}
	return AttributeSequence;
}

ULevelSequence* FUsdLevelSequenceHelperImpl::FindOrAddSequenceForLayer(const UE::FSdfLayer& Layer, bool* bOutCreatedNew)
{
	using namespace UsdUnreal::ObjectUtils;

	if (!Layer)
	{
		return nullptr;
	}

	const FString LayerIdentifier = Layer.GetIdentifier();
	const FString LayerDisplayName = Layer.GetDisplayName();

	ULevelSequence* Sequence = FindSequenceForIdentifier(LayerIdentifier);

	if (!Sequence)
	{
		// We only get a PrimLinkCache when importing (from UUsdStageImporter::ImportFromFile) or when BindToUsdStageActor is called, which also gives us
		// a stage actor. So if we don't have an actor but have a cache, we're importing
		const bool bIsImporting = StageActor.IsExplicitlyNull() && PrimLinkCache.IsValid();

		// Names need to be globally unique when opening the stage, or else when we reload the stage we will end up with a new ULevelSequence with the
		// same class, outer and name as the previous one and would otherwise stomp it. The old LevelSequence is likely still alive and valid due to
		// references from the transaction buffer, so we could run into trouble.
		//
		// When importing we don't want unique names across the whole transient package though, because we may randomly have old LevelSequences in the
		// transient package already which would give us random suffixes. Instead, want these asset names to be consistent between different imports
		// in order to replace previously imported assets, if desired. The stage importer will make the actual published asset names unique later if
		// needed though.
		//
		// We still never want the LevelSequences to collide *within the same import* however, as we'd just end up stomping the same LevelSequences
		// we're currently importing (see UE-283920), so here we use GetUniqueName instead
		FName UniqueSequenceName = bIsImporting ? *GetUniqueName(
													*SanitizeObjectName(FPaths::GetBaseFilename(LayerDisplayName)),
													LayerIdentifierByLevelSequenceName
												)
												: MakeUniqueObjectName(
													GetTransientPackage(),
													ULevelSequence::StaticClass(),
													*SanitizeObjectName(FPaths::GetBaseFilename(LayerDisplayName))
												);

		Sequence = NewObject<ULevelSequence>(GetTransientPackage(), UniqueSequenceName, FUsdLevelSequenceHelperImpl::DefaultObjFlags);
		Sequence->Initialize();

		UMovieScene* MovieScene = Sequence->MovieScene;
		if (!MovieScene)
		{
			return nullptr;
		}

		LayerIdentifierByLevelSequenceName.Add(Sequence->GetName(), LayerIdentifier);
		LevelSequencesByIdentifier.Add(LayerIdentifier, Sequence);
		IdentifierByLevelSequence.Add(Sequence, LayerIdentifier);

		// Here we abuse the FScopedReadOnlyDisable so that we can use the code in its destructor to set
		// Sequence to ReadOnly if Layer doesn't belong to UsdStage's local layer stack
		UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable SetToReadOnly{MovieScene, Layer, UsdStage};

		const FLayerTimeInfo& LayerTimeInfo = FindOrAddLayerTimeInfo(Layer);
		UpdateMovieSceneTimeRanges(*MovieScene, LayerTimeInfo);

		if (bOutCreatedNew)
		{
			*bOutCreatedNew = true;
		}
	}

	return Sequence;
}

ULevelSequence* FUsdLevelSequenceHelperImpl::CreateSequenceAndParentsRecursively(const UE::FPcpNodeRef& CurrentNode, const UE::FSdfLayer& CurrentLayer)
{
	using namespace UE;

	bool bIsNewSequence = false;
	ULevelSequence* CurrentSequence = FindOrAddSequenceForLayer(CurrentLayer, &bIsNewSequence);
	if (!bIsNewSequence)
	{
		// For now we still assume a layer only shows up once in the whole layer stack, so if its LevelSequence was created
		// before, it was also already added as a subsequence to another LevelSequence somewhere, and we're fully done here
		//
		// TODO: Support multiple subsections of the same LevelSequence, which can happen e.g. when we have two referencer
		// prims may reference the same layer with different offset/scale
		return CurrentSequence;
	}

	FPcpLayerStack NodeStack = CurrentNode.GetLayerStack();
	TArray<FSdfLayer> NodeStackLayers = NodeStack.GetLayers();
	FPcpLayerStackIdentifier LayerStackIdentifier = NodeStack.GetIdentifier();
	FSdfLayer NodeStackRootLayer = LayerStackIdentifier.RootLayer();
	FString NodeStackRootLayerIdentifier = NodeStackRootLayer.GetIdentifier();

	// This means the site we're chasing is not the immediately referenced/payload layer, but something that is part
	// of its internal composition, and we must dig further to connect our LevelSequences
	if (NodeStackRootLayer != CurrentLayer)
	{
		// Note that at this point these LevelSequences are not necessarily added to the main LevelSequence
		// tree of the stage's root layer. We'll do that next, when we find the proper referencer layer
		// to recurse upwards with
		CreateSequencesForLayerStack(NodeStackRootLayer);
	}

	// Find next composition arc to climb up
	FPcpNodeRef ChildNode = CurrentNode;
	FPcpNodeRef ParentNode = CurrentNode.GetParentNode();
	while (ParentNode && ParentNode.GetLayerStack() == NodeStack)
	{
		ChildNode = ParentNode;
		ParentNode = ParentNode.GetParentNode();
	}

	// If we have no parent node there is no composition arc between us and the root,
	// and so CurrentLayer is part of the local layer stack and must have already been created at Init().
	// This means we should have found our CurrentSequence via the FindOrAddSequenceForLayer call at
	// the top of this function, and something went wrong
	if (!ensure(ParentNode))
	{
		return CurrentSequence;
	}

	FPcpLayerStack ParentStack = ParentNode.GetLayerStack();
	FSdfPath PrimPathInParent = ParentNode.GetPath();

	auto HandleReferenceOrPayload = [&ParentStack,
									 &PrimPathInParent,
									 &NodeStackRootLayer,
									 this,
									 &NodeStackRootLayerIdentifier,
									 &ParentNode,
									 CurrentSequence,
									 &NodeStackLayers,
									 &CurrentLayer]<typename ProxyType>()
	{
		// Need to find the strongest spec in the parent stack that authors an additive reference Op to our child stack layer,
		// as that is the layer where we want to add our subsequence section for our LocalSequence
		FSdfLayer FoundReferencerLayer;
		FSdfLayerOffset ReferencedLayerOffset;

		for (const FSdfLayer& ParentLayer : ParentStack.GetLayers())
		{
			// Check for a spec in this layer
			FSdfPath ParentPath = PrimPathInParent;
			FSdfPrimSpec SpecInParentLayer = ParentLayer.GetPrimAtPath(ParentPath);
			while (!SpecInParentLayer && !ParentPath.IsAbsoluteRootPath())
			{
				// It could be that we don't have a spec for this exact prim in this layer but it was instead
				// a parent prim that authored the reference, so let's climb up.
				// This is probably safe to do as we're hunting for an exact reference to NodeStackRootLayer anyway
				ParentPath = ParentPath.GetParentPath();
				SpecInParentLayer = ParentLayer.GetPrimAtPath(ParentPath);
			}
			if (!SpecInParentLayer)
			{
				continue;
			}

			ProxyType ReferenceOrPayloadProxy;
			if constexpr (std::is_same_v<ProxyType, FSdfReferencesProxy>)
			{
				ReferenceOrPayloadProxy = SpecInParentLayer.GetReferenceList();
			}
			else
			{
				ReferenceOrPayloadProxy = SpecInParentLayer.GetPayloadList();
			}

			for (const auto& ReferenceOrPayload : ReferenceOrPayloadProxy.GetAppliedItems())
			{
				FString ReferenceTarget = ReferenceOrPayload.AssetPath;

				// TODO: Use asset resolver here to fetch a real filesystem path. For now we're assuming ReferenceTarget already is one

				if (FPaths::IsRelative(ReferenceTarget))
				{
					ReferenceTarget = FPaths::ConvertRelativePathToFull(FPaths::GetPath(ParentLayer.GetRealPath()), ReferenceTarget);
				}

				FSdfLayer TargetLayer = FSdfLayer::FindOrOpen(*ReferenceTarget);

				// The target of the reference we want is always the root layer of the referenced layer stack
				if (TargetLayer == NodeStackRootLayer)
				{
					FoundReferencerLayer = ParentLayer;
					ReferencedLayerOffset = ReferenceOrPayload.LayerOffset;
					break;
				}
			}

			if (FoundReferencerLayer)
			{
				break;
			}
		}

		if (FoundReferencerLayer)
		{
			// Store our offset into our FLayerOffsetInfo structs
			FLayerTimeInfo& ParentInfo = FindOrAddLayerTimeInfo(FoundReferencerLayer);
			if constexpr (std::is_same_v<ProxyType, FSdfReferencesProxy>)
			{
				ParentInfo.ReferenceIdentifierToOffsets.Add(NodeStackRootLayerIdentifier, ReferencedLayerOffset);
			}
			else
			{
				ParentInfo.PayloadIdentifierToOffsets.Add(NodeStackRootLayerIdentifier, ReferencedLayerOffset);
			}

			ULevelSequence* ParentSequence = CreateSequenceAndParentsRecursively(ParentNode, FoundReferencerLayer);
			ULevelSequence* ChildSequence = FindSequenceForIdentifier(NodeStackRootLayerIdentifier);
			if (!ensure(ParentSequence && ChildSequence))
			{
				return;
			}

			// Our parent referencer is always referencing the root of our layer stack: Not necessarily our current layer
			CreateSubSequenceSection(*ParentSequence, *ChildSequence, ReferencedLayerOffset);

			// This is a failsafe case: It seems that for value clips the assetPaths layers don't count as sublayers,
			// payloads, references, or any other composition arc... We'll still have to handle LevelSequences created
			// for these layers however, as FindOrAddSequenceForLayer() and friends will actually end up finding the attribute
			// specs inside of them. Luckily they don't have any sort of offset or scale, so here we'll just add the assetPath
			// sequence to its referencer layer LevelSequence with a subsection starting at zero
			if (!NodeStackLayers.Contains(CurrentLayer))
			{
				UE::FSdfLayerOffset NoOffset;
				CreateSubSequenceSection(*ChildSequence, *CurrentSequence, NoOffset);
			}
		}
	};

	switch (ChildNode.GetArcType())
	{
		case EPcpArcType::PcpArcTypeReference:
		{
			// Weird syntax but we're just calling the lamda specifying the template type here
			HandleReferenceOrPayload.template operator()<FSdfReferencesProxy>();
			break;
		}
		case EPcpArcType::PcpArcTypePayload:
		{
			HandleReferenceOrPayload.template operator()<FSdfPayloadsProxy>();
			break;
		}
		default:
		{
			// Reference and payload arcs are the only composition arcs that can jump to a different layer stack, so they're the
			// only ones we care about for now
			break;
		}
	}

	return CurrentSequence;
}

UE::FSdfLayer FUsdLevelSequenceHelperImpl::FindEditTargetForSubsequence(ULevelSequence* Sequence)
{
	FString LayerIdentifier = IdentifierByLevelSequence.FindRef(Sequence);
	return UE::FSdfLayer::FindOrOpen(*LayerIdentifier);
}

UMovieSceneSubSection* FUsdLevelSequenceHelperImpl::FindSubSequenceSection(ULevelSequence& Sequence, ULevelSequence& SubSequence)
{
	UMovieScene* MovieScene = Sequence.GetMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}

	UMovieSceneSubTrack* SubTrack = MovieScene->FindTrack<UMovieSceneSubTrack>();

	if (!SubTrack)
	{
		return nullptr;
	}

	UMovieSceneSection* const* SubSection = Algo::FindByPredicate(
		SubTrack->GetAllSections(),
		[&SubSequence](UMovieSceneSection* Section) -> bool
		{
			if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section))
			{
				return (SubSection->GetSequence() == &SubSequence);
			}
			else
			{
				return false;
			}
		}
	);

	if (SubSection)
	{
		return Cast<UMovieSceneSubSection>(*SubSection);
	}
	else
	{
		return nullptr;
	}
}

void FUsdLevelSequenceHelperImpl::CreateSubSequenceSection(ULevelSequence& Sequence, ULevelSequence& SubSequence, const UE::FSdfLayerOffset& SubLayerOffset)
{
	if (&Sequence == &SubSequence)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence.GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	if (!UsdStage)
	{
		return;
	}

	FFrameRate TickResolution = MovieScene->GetTickResolution();

	UMovieSceneSubTrack* SubTrack = MovieScene->FindTrack<UMovieSceneSubTrack>();
	if (!SubTrack)
	{
		SubTrack = MovieScene->AddTrack<UMovieSceneSubTrack>();
	}

	const FString* LayerIdentifier = LayerIdentifierByLevelSequenceName.Find(Sequence.GetName());
	const FString* SubLayerIdentifier = LayerIdentifierByLevelSequenceName.Find(SubSequence.GetName());
	if (!LayerIdentifier || !SubLayerIdentifier)
	{
		return;
	}

	UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(**LayerIdentifier);
	UE::FSdfLayer SubLayer = UE::FSdfLayer::FindOrOpen(**SubLayerIdentifier);

	UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, Layer, UsdStage};

	const double TimeCodesPerSecond = Layer.GetTimeCodesPerSecond();
	const bool bIsAlembicSublayer = SubLayerIdentifier->EndsWith(TEXT(".abc"));
	TRange<FFrameNumber> SubSectionRange;
	FFrameNumber StartFrame;

	if (!bIsAlembicSublayer)
	{
		// Section full duration is always [0, endTimeCode]. The play range varies: For the root layer it will be [startTimeCode, endTimeCode],
		// but for sublayers it will be [0, endTimeCode] too in order to match how USD composes sublayers with non-zero startTimeCode
		const double SubDurationTimeCodes = SubLayer.GetEndTimeCode() * SubLayerOffset.Scale;
		const double SubDurationSeconds = SubDurationTimeCodes / TimeCodesPerSecond;

		const double SubStartTimeSeconds = SubLayerOffset.Offset / TimeCodesPerSecond;
		const double SubEndTimeSeconds = SubStartTimeSeconds + SubDurationSeconds;

		StartFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, SubStartTimeSeconds);
		const FFrameNumber EndFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, SubEndTimeSeconds);

		// Don't clip subsections with their duration, so that the root layer's [startTimeCode, endTimeCode] range is the only thing clipping
		// anything, as this is how USD seems to behave. Even if a middle sublayer has startTimeCode == endTimeCode, its animations
		// (or its child sublayers') won't be clipped by it and play according to the stage's range
		const double StageEndTimeSeconds = UsdStage.GetEndTimeCode() / UsdStage.GetTimeCodesPerSecond();
		const FFrameNumber StageEndFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, StageEndTimeSeconds);

		SubSectionRange = TRange<FFrameNumber>{StartFrame, FMath::Max(StartFrame, StageEndFrame)};
	}
	else
	{
		// One issue with a sublayer from Alembic is that the usdAbc plugin does not retrieve the frame rate of the archive.
		// Another is that the start time does not necessarily represents the actual start of the animation. That's why
		// there's an option to "skip empty frames" when importing an Alembic. So instead take the start/end timecodes from
		// the parent layer. That way the user can define the animation range needed.
		SubLayer = Layer;

		const double SubStartTimeSeconds = SubLayer.GetStartTimeCode() * SubLayerOffset.Scale / TimeCodesPerSecond;
		const double SubEndTimeSeconds = SubLayer.GetEndTimeCode() * SubLayerOffset.Scale / TimeCodesPerSecond;

		StartFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, SubStartTimeSeconds);
		const FFrameNumber EndFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, SubEndTimeSeconds);

		SubSectionRange = TRange<FFrameNumber>{StartFrame, EndFrame};
	}

	UMovieSceneSubSection* SubSection = FindSubSequenceSection(Sequence, SubSequence);
	if (SubSection)
	{
		SubSection->SetRange(SubSectionRange);
	}
	else
	{
		// Always force our sections to be on separate rows.
		//
		// We compute the row ourselves instead of using AddSequence (that just passes INDEX_NONE for it), because internally
		// UMovieSceneSubTrack::AddSequenceOnRow will let sections end up on the same row if they don't overlap. We may
		// end up with zero-size sections in some cases though, and those *never* overlap, so they would end up bunched up on
		// the same row (see UE-217625)
		const int32 RowIndex = SubTrack->GetAllSections().Num();
		SubSection = SubTrack->AddSequenceOnRow(	//
			&SubSequence,
			SubSectionRange.GetLowerBoundValue(),
			SubSectionRange.Size<FFrameNumber>().Value,
			RowIndex
		);
		DirtySequenceHierarchyCache();
	}

	const double TimeCodesPerSecondDifference = TimeCodesPerSecond / SubLayer.GetTimeCodesPerSecond();
	SubSection->Parameters.TimeScale = FMath::IsNearlyZero(SubLayerOffset.Scale) ? 0.f : 1.f / (SubLayerOffset.Scale / TimeCodesPerSecondDifference);

	// As far as the Sequencer is concerned, the subsection "starts" at it's playback range start (i.e. if the inner playback range is
	// [5, 20] and we place the subsection at timeCode 14, it will try making it so that at outer timeCode 14 it plays the inner timeCode 5).
	// To match USD composition, we instead want the subsection to always start at inner timeCode zero (i.e. at outer timeCode 14 it plays
	// the inner timeCode 0). The only way of doing that is by specifying a StartFrameOffset for the subsection, that matches the playback range.
	//
	// These will be edited together by the Sequencer (so e.g. if a user drags the inner playback range it will also update the start frame offset),
	// we just have to set them up properly the first time around it seems.
	SubSection->Parameters.StartFrameOffset = -SubSequence.GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();
}

void FUsdLevelSequenceHelperImpl::RemoveSubSequenceSection(ULevelSequence& Sequence, ULevelSequence& SubSequence)
{
	if (UMovieSceneSubTrack* SubTrack = Sequence.GetMovieScene()->FindTrack<UMovieSceneSubTrack>())
	{
		if (UMovieSceneSection* SubSection = FindSubSequenceSection(Sequence, SubSequence))
		{
			SubTrack->Modify();
			SubTrack->RemoveSection(*SubSection);

			DirtySequenceHierarchyCache();
		}
	}
}

void FUsdLevelSequenceHelperImpl::UpdateSubSectionTimeRanges(bool bShowResizedSectionToast)
{
	// This is in charge of updating *all* our SubSection sizes, TimeScale, as well as their StartTimeOffsets according to the stage.
	// This does not handle playback ranges, which are updated by UpdateMovieSceneTimeRanges.
	//
	// In USD, all endTimeCode values are ignored except those for the root layer. Even those don't "hard clip" the animation either,
	// and are more like a suggestion of what the play range should be.
	// In UE, subsection ranges do in fact clip the inner animation. We can't specify an unbounded range for them though, and
	// making them max size is inconvenient UI-wise.
	//
	// Since there is no easy/efficient way of fetching a time code range for all animation contained "below a particular layer",
	// the best way of reconciling that discrepancy that we can come up with so far is to always force the subsection ends to
	// match whatever the stage's endTimeCode is, which is done here. This way, the only clipping that happens is one that is
	// seen by the root layer's endTimeCode, which can similarly seen in other DCCs (like usdview) and is one which can be easily
	// controlled (i.e. it's easy to just tweak the root layer's endTimeCode directly)

	const double StageEndTimeSeconds = UsdStage.GetEndTimeCode() / UsdStage.GetTimeCodesPerSecond();

	bool bChangedSectionSize = false;

	for (const TPair<FString, TObjectPtr<ULevelSequence>>& IdentifierAndSeq : LevelSequencesByIdentifier)
	{
		ULevelSequence* Sequence = IdentifierAndSeq.Value.Get();
		if (!Sequence)
		{
			continue;
		}

		UMovieScene* MovieScene = Sequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		UMovieSceneSubTrack* SubTrack = MovieScene->FindTrack<UMovieSceneSubTrack>();
		if (!SubTrack)
		{
			continue;
		}

		FFrameRate TickResolution = MovieScene->GetTickResolution();
		double TimeCodesPerSecond = GetTimeCodesPerSecond();
		const FFrameNumber StageEndFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, StageEndTimeSeconds);

		FLayerTimeInfo* LayerTimeInfo = LayerTimeInfosByLayerIdentifier.Find(IdentifierAndSeq.Key);

		for (UMovieSceneSection* Section : SubTrack->GetAllSections())
		{
			UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
			if (!SubSection)
			{
				continue;
			}

			UMovieSceneSequence* SubSequence = SubSection->GetSequence();
			if (!SubSequence)
			{
				continue;
			}

			FString* SubSequenceLayerIdentifier = IdentifierByLevelSequence.Find(Cast<ULevelSequence>(SubSequence));
			if (!SubSequenceLayerIdentifier)
			{
				continue;
			}

			// TODO: This will likely fail in edge cases where a layer is both a sublayer and a reference (or a reference and a payload, etc.)
			// We need to better track the actual sections to know what they represent purely on the UE side (maybe some metadata, or tracking the
			// pointers?)
			UE::FSdfLayerOffset* SubLayerOffset = LayerTimeInfo->SublayerIdentifierToOffsets.Find(*SubSequenceLayerIdentifier);
			if (!SubLayerOffset)
			{
				SubLayerOffset = LayerTimeInfo->ReferenceIdentifierToOffsets.Find(*SubSequenceLayerIdentifier);
			}
			if (!SubLayerOffset)
			{
				SubLayerOffset = LayerTimeInfo->PayloadIdentifierToOffsets.Find(*SubSequenceLayerIdentifier);
			}
			if (!SubLayerOffset)
			{
				continue;
			}

			UE::FSdfLayer SubSequenceLayer = UE::FSdfLayer::FindOrOpen(**SubSequenceLayerIdentifier);
			if (!SubSequenceLayer)
			{
				continue;
			}

			SubSection->Modify();

			// StartFrameOffset: Should match the sublayer's playback range start so that the inner sequence starts playing at its zero
			SubSection->Parameters.StartFrameOffset = -SubSequence->GetMovieScene()->GetPlaybackRange().GetLowerBoundValue();

			// TimeScale: Should match the sublayer's scale (or the inverse of it, rather)
			const double TimeCodesPerSecondDifference = TimeCodesPerSecond / SubSequenceLayer.GetTimeCodesPerSecond();
			SubSection->Parameters.TimeScale = FMath::IsNearlyZero(SubLayerOffset->Scale)
												   ? 0.f
												   : 1.f / (SubLayerOffset->Scale / TimeCodesPerSecondDifference);

			// Section start/end: The start should match the sublayer's offset, and the end should always clip to the stage's endTimeCode
			TRange<FFrameNumber> Range = SubSection->GetRange();
			FFrameNumber NewLower = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(
				TickResolution,
				SubLayerOffset->Offset / TimeCodesPerSecond
			);
			FFrameNumber NewUpper = FMath::Max(StageEndFrame, NewLower);
			if (NewLower == Range.GetLowerBoundValue() && NewUpper == Range.GetUpperBoundValue())
			{
				continue;
			}
			Range.SetLowerBoundValue(NewLower);
			Range.SetUpperBoundValue(NewUpper);
			SubSection->SetRange(Range);

			bChangedSectionSize = true;
		}
	}

	if (bChangedSectionSize && bShowResizedSectionToast)
	{
		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings && Settings->bShowSubsectionSnappingWarning)
		{
			const FText Text = LOCTEXT("SnappedSectionTitle", "USD: Subsections adjusted");

			const FText SubText = LOCTEXT(
				"SnappedSectionTitleSubText",
				"In USD, start and endTimeCodes from any layer but the root layer are mostly ignored, while in UE a subsequence section range does clip the underlying animation.\n\nIn order to reconcile the behavior, subsequence sections on any of the StageActor's generated LevelSequences will automatically snap to the stage's playback range instead, which prevents any unwanted animation clipping."
			);

			USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(Text);
			Toast.SubText = SubText;
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;
			Toast.ButtonDetails.Emplace(
				LOCTEXT("OverridenOpinionMessageOk", "Ok"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);
			// This is flipped because the default checkbox message is "Don't prompt again"
			Toast.CheckBoxState = Settings->bShowSubsectionSnappingWarning ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
				[](ECheckBoxState NewState)
				{
					if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
					{
						// This is flipped because the default checkbox message is "Don't prompt again"
						Settings->bShowSubsectionSnappingWarning = NewState == ECheckBoxState::Unchecked;
						Settings->SaveConfig();
					}
				}
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}
}

void FUsdLevelSequenceHelperImpl::CreateTimeTrack(const FLayerTimeInfo& Info)
{
	ULevelSequence* Sequence = FindSequenceForIdentifier(Info.Identifier);

	if (!Sequence || !StageActorBinding.IsValid())
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	if (!MovieScene)
	{
		return;
	}

	UMovieSceneFloatTrack* TimeTrack = MovieScene->FindTrack<UMovieSceneFloatTrack>(
		StageActorBinding,
		FName(FUsdLevelSequenceHelperImpl::TimeTrackName)
	);
	if (TimeTrack)
	{
		TimeTrack->Modify();
		TimeTrack->RemoveAllAnimationData();
	}
	else
	{
		TimeTrack = MovieScene->AddTrack<UMovieSceneFloatTrack>(StageActorBinding);
		if (!TimeTrack)
		{
			return;
		}

		TimeTrack->SetPropertyNameAndPath(FName(FUsdLevelSequenceHelperImpl::TimeTrackName), "Time");

		MovieScene->SetEvaluationType(EMovieSceneEvaluationType::FrameLocked);
	}

	// Always setup the time track even if the layer is "not animated" as we need this to refresh
	// the start/end keyframes whenever we resize the playback range to being [0, 0] as will (which
	// would be considered "not animated")
	{
		const double StartTimeCode = Info.StartTimeCode.Get(0.0);
		const double EndTimeCode = Info.EndTimeCode.Get(0.0);
		const double TimeCodesPerSecond = GetTimeCodesPerSecond();

		FFrameRate DestTickRate = MovieScene->GetTickResolution();
		FFrameNumber StartFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(DestTickRate, StartTimeCode / TimeCodesPerSecond);
		FFrameNumber EndFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(DestTickRate, EndTimeCode / TimeCodesPerSecond);

		TRange<FFrameNumber> PlaybackRange(StartFrame, EndFrame);

		bool bSectionAdded = false;

		if (UMovieSceneFloatSection* TimeSection = Cast<UMovieSceneFloatSection>(TimeTrack->FindOrAddSection(0, bSectionAdded)))
		{
			TimeSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
			TimeSection->SetRange(TRange<FFrameNumber>::All());

			TArray<FFrameNumber> FrameNumbers;
			FrameNumbers.Add(UE::MovieScene::DiscreteInclusiveLower(PlaybackRange));
			FrameNumbers.Add(UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange));

			TArray<FMovieSceneFloatValue> FrameValues;
			FrameValues.Add_GetRef(FMovieSceneFloatValue(StartTimeCode)).InterpMode = ERichCurveInterpMode::RCIM_Linear;
			FrameValues.Add_GetRef(FMovieSceneFloatValue(EndTimeCode)).InterpMode = ERichCurveInterpMode::RCIM_Linear;

			FMovieSceneFloatChannel* TimeChannel = TimeSection->GetChannelProxy().GetChannel<FMovieSceneFloatChannel>(0);
			TimeChannel->Set(FrameNumbers, FrameValues);

			// It's probably for the best to always keep this "read-only" because all of our animations are coming from
			// the actual parsed Sequencer tracks now, so if a user edits this it will likely have no effect on the animation
			// that is actually visible
			TimeSection->SetIsLocked(true);

			RefreshSequencer();
		}
	}
}

void FUsdLevelSequenceHelperImpl::RemoveTimeTrack(const FLayerTimeInfo* LayerTimeInfo)
{
	if (!UsdStage || !LayerTimeInfo || !StageActorBinding.IsValid())
	{
		return;
	}

	ULevelSequence* Sequence = FindSequenceForIdentifier(LayerTimeInfo->Identifier);

	if (!Sequence)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	if (!MovieScene)
	{
		return;
	}

	UMovieSceneFloatTrack* TimeTrack = MovieScene->FindTrack<UMovieSceneFloatTrack>(
		StageActorBinding,
		FName(FUsdLevelSequenceHelperImpl::TimeTrackName)
	);
	if (TimeTrack)
	{
		MovieScene->RemoveTrack(*TimeTrack);
	}
}

void FUsdLevelSequenceHelperImpl::AddCommonTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim, bool bForceVisibilityTracks)
{
	USceneComponent* ComponentToBind = PrimTwin.GetSceneComponent();
	if (!ComponentToBind)
	{
		return;
	}

	UE::FUsdGeomXformable Xformable(Prim);

	// If this xformable has an op to reset the xform stack and one of its ancestors is animated, then we need to make
	// a transform track for it even if its transform is not animated by itself. This because that op effectively means
	// "discard the parent transform and treat this as a direct world transform", but when reading we'll manually recompute
	// the relative transform to its parent anyway (for simplicity's sake). If that parent (or any of its ancestors) is
	// being animated, we'll need to recompute this for every animation keyframe
	TArray<double> AncestorTimeSamples;
	bool bNeedTrackToCompensateResetXformOp = false;
	if (Xformable.GetResetXformStack())
	{
		UE::FUsdPrim AncestorPrim = Prim.GetParent();
		while (AncestorPrim && !AncestorPrim.IsPseudoRoot())
		{
			if (UE::FUsdGeomXformable AncestorXformable{AncestorPrim})
			{
				TArray<double> TimeSamples;
				if (AncestorXformable.GetTimeSamples(&TimeSamples) && TimeSamples.Num() > 0)
				{
					bNeedTrackToCompensateResetXformOp = true;

					AncestorTimeSamples.Append(TimeSamples);
				}

				// The exception is if our ancestor also wants to reset its xform stack (i.e. its transform is meant to be
				// used as the world transform). In this case we don't need to care about higher up ancestors anymore, as
				// their transforms wouldn't affect below this prim anyway
				if (AncestorXformable.GetResetXformStack())
				{
					break;
				}
			}

			AncestorPrim = AncestorPrim.GetParent();
		}
	}

	// Check whether we should ignore the prim's local transform or not
	bool bIgnorePrimLocalTransform = false;
	switch (GetRootMotionHandling())
	{
		default:
		case EUsdRootMotionHandling::NoAdditionalRootMotion:
		{
			break;
		}
		case EUsdRootMotionHandling::UseMotionFromSkelRoot:
		{
			if (Prim.IsA(TEXT("SkelRoot")))
			{
				bIgnorePrimLocalTransform = true;
			}
			break;
		}
		case EUsdRootMotionHandling::UseMotionFromSkeleton:
		{
			if (Prim.IsA(TEXT("Skeleton")))
			{
				bIgnorePrimLocalTransform = true;
			}
			break;
		}
	}

	// Check if we need to add Transform tracks.
	// In case we're e.g. a Cube with animated "size", which needs to become animated transforms
	if (Xformable.TransformMightBeTimeVarying() || bNeedTrackToCompensateResetXformOp || Prim.IsA(TEXT("Gprim")))
	{
		TArray<double> TimeSampleUnion;

		// Get all *animated* attributes that may contribute to the transform
		bool bAreAllMuted = true;
		TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, UnrealIdentifiers::TransformPropertyName);
		for (int32 Index = Attrs.Num() - 1; Index >= 0; --Index)
		{
			const UE::FUsdAttribute& Attr = Attrs[Index];

			TArray<double> TimeSamplesForAttr;
			if (!Attr.GetTimeSamples(TimeSamplesForAttr) || TimeSamplesForAttr.Num() == 0)
			{
				const int32 Count = 1;
				Attrs.RemoveAt(Index, Count, EAllowShrinking::No);
				continue;
			}

			if (!UsdUtils::IsAttributeMuted(Attr, UsdStage))
			{
				bAreAllMuted = false;

				// Union the time samples so we know to always sample where we have a value for a relevant attribute
				TimeSampleUnion.Append(TimeSamplesForAttr);
			}
		}

		UE::FUsdAttribute StrongestAttr = UsdUtils::GetStrongestAttribute(Attrs, UsdUtils::GetEarliestTimeCode());

		UE::FSdfLayer Layer;
		ULevelSequence* AttributeSequence = FindOrAddSequenceForAttribute(StrongestAttr, &Layer);

		// If we're creating a brand new transform track to compensate resetXformOp we may not have any animated attribute
		// already, but we still need to do this
		if (!AttributeSequence && bNeedTrackToCompensateResetXformOp)
		{
			Layer = UsdUtils::FindLayerForPrim(Prim);
			AttributeSequence = FindOrAddSequenceForLayer(Layer);
		}

		// Get the Subsequence where we should create our track according to that Layer
		if (AttributeSequence)
		{
			const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(AttributeSequence);

			if (UMovieScene* MovieScene = AttributeSequence->GetMovieScene())
			{
				UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, Layer, UsdStage};

				if (bNeedTrackToCompensateResetXformOp)
				{
					TimeSampleUnion.Append(AncestorTimeSamples);
				}

				// Note that since we sort, we can cheaply handle duplicate timeSamples on this array because
				// UsdToUnreal::ConvertTransformTimeSamples ignores consecutive duplicates anyway (using FMath::IsNearlyEqual too)
				TimeSampleUnion.Sort();

				if (UMovieScene3DTransformTrack* TransformTrack = AddTrack<UMovieScene3DTransformTrack>(
						UnrealIdentifiers::TransformPropertyName,
						PrimTwin,
						*ComponentToBind,
						*AttributeSequence,
						bAreAllMuted
					))
				{
					UsdToUnreal::FPropertyTrackReader Reader = UsdToUnreal::CreatePropertyTrackReader(
						Prim,
						UnrealIdentifiers::TransformPropertyName,
						bIgnorePrimLocalTransform
					);
					UsdToUnreal::ConvertTransformTimeSamples(UsdStage, TimeSampleUnion, Reader.TransformReader, *TransformTrack, SequenceTransform);
				}

				PrimPathByLevelSequenceName.AddUnique(AttributeSequence->GetFName(), Prim.GetPrimPath().GetString());
			}
		}
	}

	TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, UnrealIdentifiers::HiddenInGamePropertyName);
	if (Attrs.Num() > 0)
	{
		UE::FUsdAttribute VisibilityAttribute = Attrs[0];
		if (VisibilityAttribute)
		{
			// Collect all the time samples we'll need to sample our visibility at (USD has inherited visibilities, so every time
			// a parent has a key, we need to recompute the child visibility at that moment too)
			TArray<double> TotalVisibilityTimeSamples;
			VisibilityAttribute.GetTimeSamples(TotalVisibilityTimeSamples);

			// If we're adding a visibility track because a parent has visibility animations, we want to write our baked visibility
			// tracks on the same layer as the first one of our parents that actually has animated visibility.
			// There's no ideal place for this because it's essentially a fake track that we're creating, and we may have arbitrarily
			// many parents and specs on multiple layers, but this is hopefully at least *a* reasonable answer.
			UE::FUsdAttribute FirstAnimatedVisibilityParentAttr;

			if (bForceVisibilityTracks)
			{
				// TODO: Improve this, as this is extremely inefficient since we'll be parsing this tree for the root down and repeatedly
				// redoing this one child at a time...
				UE::FUsdPrim ParentPrim = Prim.GetParent();
				while (ParentPrim && !ParentPrim.IsPseudoRoot())
				{
					if (UsdUtils::HasAnimatedVisibility(ParentPrim))
					{
						TArray<UE::FUsdAttribute> ParentAttrs = UsdUtils::GetAttributesForProperty(
							ParentPrim,
							UnrealIdentifiers::HiddenInGamePropertyName
						);
						if (ParentAttrs.Num() > 0)
						{
							UE::FUsdAttribute ParentVisAttr = ParentAttrs[0];

							TArray<double> TimeSamples;
							if (ParentVisAttr && (ParentVisAttr.GetTimeSamples(TimeSamples) && TimeSamples.Num()))
							{
								if (!FirstAnimatedVisibilityParentAttr)
								{
									FirstAnimatedVisibilityParentAttr = ParentVisAttr;
								}

								TotalVisibilityTimeSamples.Append(TimeSamples);
							}
						}
					}

					ParentPrim = ParentPrim.GetParent();
				}

				// Put these in order for the sampling below, but don't worry about duplicates: The baking process already skips
				// consecutive duplicates anyway
				TotalVisibilityTimeSamples.Sort();
			}

			// Pick which attribute we will use to fetch the target LevelSequence to put our baked tracks
			UE::FUsdAttribute AttributeForSequence;
			if (VisibilityAttribute.GetNumTimeSamples() > 0)
			{
				AttributeForSequence = VisibilityAttribute;
			}
			if (!AttributeForSequence && bForceVisibilityTracks && FirstAnimatedVisibilityParentAttr)
			{
				AttributeForSequence = FirstAnimatedVisibilityParentAttr;
			}

			if (AttributeForSequence && TotalVisibilityTimeSamples.Num() > 0)
			{
				UE::FSdfLayer SequenceLayer;
				if (ULevelSequence* AttributeSequence = FindOrAddSequenceForAttribute(AttributeForSequence, &SequenceLayer))
				{
					const bool bIsMuted = UsdUtils::IsAttributeMuted(AttributeForSequence, UsdStage);

					if (UMovieScene* MovieScene = AttributeSequence->GetMovieScene())
					{
						UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, SequenceLayer, UsdStage};

						if (UMovieSceneVisibilityTrack* VisibilityTrack = AddTrack<UMovieSceneVisibilityTrack>(
								UnrealIdentifiers::HiddenInGamePropertyName,
								PrimTwin,
								*ComponentToBind,
								*AttributeSequence,
								bIsMuted
							))
						{
							const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(AttributeSequence);
							UsdToUnreal::FPropertyTrackReader Reader = UsdToUnreal::CreatePropertyTrackReader(
								Prim,
								UnrealIdentifiers::HiddenInGamePropertyName
							);
							UsdToUnreal::ConvertBoolTimeSamples(
								UsdStage,
								TotalVisibilityTimeSamples,
								Reader.BoolReader,
								*VisibilityTrack,
								SequenceTransform
							);
						}

						PrimPathByLevelSequenceName.AddUnique(AttributeSequence->GetFName(), Prim.GetPrimPath().GetString());
					}
				}
			}
		}
	}
}

void FUsdLevelSequenceHelperImpl::AddBoundsTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim, TOptional<bool> HasAnimatedBounds)
{
	if (HasAnimatedBounds.IsSet())
	{
		if (!HasAnimatedBounds.GetValue())
		{
			return;
		}
	}

	USceneComponent* ComponentToBind = PrimTwin.GetSceneComponent();
	if (!ComponentToBind || !Prim)
	{
		return;
	}

	if (!ensure(BBoxCache))
	{
		return;
	}

	// We only actually use bounds when drawing alternative draw modes
	EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(Prim);
	if (DrawMode == EUsdDrawMode::Default)
	{
		return;
	}

	// We need to manually check if the prim has animated bounds now.
	// We tried earlying out due to other reasons first as this can be expensive
	if (!HasAnimatedBounds.IsSet())
	{
		if (!UsdUtils::HasAnimatedBounds(Prim, BBoxCache->GetIncludedPurposes(), BBoxCache->GetUseExtentsHint(), BBoxCache->GetIgnoreVisibility()))
		{
			return;
		}
	}

	// Find the Sequence where we'll author the tracks
	ULevelSequence* TargetSequence = nullptr;
	UE::FSdfLayer TargetLayer;
	bool bIsMuted = false;
	if (Prim.IsA(TEXT("Boundable")))
	{
		UE::FUsdAttribute ExtentAttr = Prim.GetAttribute(TEXT("extent"));
		if (ExtentAttr && ExtentAttr.HasAuthoredValue())
		{
			bIsMuted = UsdUtils::IsAttributeMuted(ExtentAttr, Prim.GetStage());
			TargetSequence = FindOrAddSequenceForAttribute(ExtentAttr, &TargetLayer);
		}
	}
	if (!TargetSequence)
	{
		if (Prim.HasAPI(TEXT("GeomModelAPI")))
		{
			UE::FUsdAttribute ExtentsHintAttr = Prim.GetAttribute(TEXT("extentsHint"));
			if (ExtentsHintAttr && ExtentsHintAttr.HasAuthoredValue())
			{
				bIsMuted = UsdUtils::IsAttributeMuted(ExtentsHintAttr, Prim.GetStage());
				TargetSequence = FindOrAddSequenceForAttribute(ExtentsHintAttr, &TargetLayer);
			}
		}
	}
	if (!TargetSequence)
	{
		// For the other track types we mostly look for a correspondence between one or more USD attributes and UE properties.
		// For these tracks however we may not have any authored `extent` or `extentsHint` yet (and would only author them on-demand), so
		// we may need to create tracks that correspond purely to computed, non-authored values in USD.
		// If the user manually modifies these, we'll author these as `extent` or `extentsHint` depending on the prim, but only on-demand.
		//
		// TODO: These may not be connected to the LevelSequence hierarchy in edge cases
		UE::FSdfLayer PrimLayer = UsdUtils::FindLayerForPrim(Prim);
		TargetSequence = FindOrAddSequenceForLayer(PrimLayer);
		TargetLayer = PrimLayer;
	}
	if (!TargetSequence)
	{
		return;
	}

	const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(TargetSequence);

	UMovieScene* MovieScene = TargetSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, TargetLayer, UsdStage};

	UMovieSceneDoubleVectorTrack* MinTrack = AddTrack<UMovieSceneDoubleVectorTrack>(
		GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin),
		PrimTwin,
		*ComponentToBind,
		*TargetSequence,
		bIsMuted
	);
	if (!ensure(MinTrack))
	{
		return;
	}
	MinTrack->SetNumChannelsUsed(3);

	UMovieSceneDoubleVectorTrack* MaxTrack = AddTrack<UMovieSceneDoubleVectorTrack>(
		GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMax),
		PrimTwin,
		*ComponentToBind,
		*TargetSequence,
		bIsMuted
	);
	if (!ensure(MaxTrack))
	{
		return;
	}
	MaxTrack->SetNumChannelsUsed(3);

	TArray<double> BoundsTimeSamples;
	const bool bHasAnimatedBounds = UsdUtils::GetAnimatedBoundsTimeSamples(
		Prim,
		BoundsTimeSamples,
		BBoxCache->GetIncludedPurposes(),
		BBoxCache->GetUseExtentsHint(),
		BBoxCache->GetIgnoreVisibility()
	);
	// GetAnimatedBoundsTimeSamples uses the same underlying code to check if a prim has animated bounds or not,
	// so if we're this deep in creating bounds tracks it better agree that the bounds are animated
	ensure(bHasAnimatedBounds);

	UsdToUnreal::ConvertBoundsTimeSamples(Prim, BoundsTimeSamples, SequenceTransform, *MinTrack, *MaxTrack, BBoxCache.Get());

	PrimPathByLevelSequenceName.AddUnique(TargetSequence->GetFName(), Prim.GetPrimPath().GetString());
}

namespace UE::USDLevelSequenceHelper::Private
{
	static TSet<FName> TrackedCameraProperties = {
		UnrealIdentifiers::CurrentFocalLengthPropertyName,
		UnrealIdentifiers::ManualFocusDistancePropertyName,
		UnrealIdentifiers::CurrentAperturePropertyName,
		UnrealIdentifiers::SensorWidthPropertyName,
		UnrealIdentifiers::SensorHeightPropertyName,
		UnrealIdentifiers::SensorHorizontalOffsetPropertyName,
		UnrealIdentifiers::SensorVerticalOffsetPropertyName,
		UnrealIdentifiers::ExposureCompensationPropertyName,
		UnrealIdentifiers::ProjectionModePropertyName,
		UnrealIdentifiers::OrthoFarClipPlanePropertyName,
		UnrealIdentifiers::OrthoNearClipPlanePropertyName,
		UnrealIdentifiers::CustomNearClipppingPlanePropertyName,
		UnrealIdentifiers::DepthOfFieldFstopPropertyName,
		UnrealIdentifiers::CameraISOPropertyName,
		UnrealIdentifiers::CameraShutterSpeedPropertyName
	};
}

void FUsdLevelSequenceHelperImpl::AddCameraTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim)
{
	// For ACineCameraActor the camera component is not the actual root component, so we need to fetch it manually here
	ACineCameraActor* CameraActor = Cast<ACineCameraActor>(PrimTwin.GetSceneComponent()->GetOwner());
	if (!CameraActor)
	{
		return;
	}
	UCineCameraComponent* ComponentToBind = CameraActor->GetCineCameraComponent();
	if (!ComponentToBind)
	{
		return;
	}

	for (const FName& PropertyName : UE::USDLevelSequenceHelper::Private::TrackedCameraProperties)
	{
		TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, PropertyName);
		if (Attrs.Num() < 1)
		{
			continue;
		}

		// Camera attributes should always match UE properties 1-to-1 here so just get the first
		const UE::FUsdAttribute& Attr = Attrs[0];
		if (!Attr || Attr.GetNumTimeSamples() == 0)
		{
			continue;
		}

		// Find out the sequence where this attribute should be written to
		UE::FSdfLayer SequenceLayer;
		if (ULevelSequence* AttributeSequence = FindOrAddSequenceForAttribute(Attr, &SequenceLayer))
		{
			const bool bIsMuted = UsdUtils::IsAttributeMuted(Attr, UsdStage);

			UMovieScene* MovieScene = AttributeSequence->GetMovieScene();
			if (!MovieScene)
			{
				continue;
			}

			UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, SequenceLayer, UsdStage};

			TArray<double> TimeSamples;
			if (!Attr.GetTimeSamples(TimeSamples))
			{
				continue;
			}

			if (UMovieSceneFloatTrack* FloatTrack = AddTrack<
					UMovieSceneFloatTrack>(PropertyName, PrimTwin, *ComponentToBind, *AttributeSequence, bIsMuted))
			{
				const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(AttributeSequence);
				UsdToUnreal::FPropertyTrackReader Reader = UsdToUnreal::CreatePropertyTrackReader(Prim, PropertyName);
				UsdToUnreal::ConvertFloatTimeSamples(UsdStage, TimeSamples, Reader.FloatReader, *FloatTrack, SequenceTransform);
			}

			PrimPathByLevelSequenceName.AddUnique(AttributeSequence->GetFName(), Prim.GetPrimPath().GetString());
		}
	}
}

void FUsdLevelSequenceHelperImpl::AddLightTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim, const TSet<FName>& PropertyPathsToRead)
{
	using namespace UnrealIdentifiers;

	USceneComponent* ComponentToBind = PrimTwin.GetSceneComponent();
	if (!ComponentToBind)
	{
		return;
	}

	enum class ETrackType : uint8
	{
		Bool,
		Float,
		Color,
	};

	TMap<FName, ETrackType> PropertyPathToTrackType;
	PropertyPathToTrackType.Add(IntensityPropertyName, ETrackType::Float);
	PropertyPathToTrackType.Add(LightColorPropertyName, ETrackType::Color);

	if (const ULightComponent* LightComponent = Cast<ULightComponent>(ComponentToBind))
	{
		PropertyPathToTrackType.Add(UseTemperaturePropertyName, ETrackType::Bool);
		PropertyPathToTrackType.Add(TemperaturePropertyName, ETrackType::Float);

		if (const URectLightComponent* RectLightComponent = Cast<URectLightComponent>(ComponentToBind))
		{
			PropertyPathToTrackType.Add(SourceWidthPropertyName, ETrackType::Float);
			PropertyPathToTrackType.Add(SourceHeightPropertyName, ETrackType::Float);
		}
		else if (const UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(ComponentToBind))
		{
			PropertyPathToTrackType.Add(SourceRadiusPropertyName, ETrackType::Float);

			if (const USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(ComponentToBind))
			{
				PropertyPathToTrackType.Add(OuterConeAnglePropertyName, ETrackType::Float);
				PropertyPathToTrackType.Add(InnerConeAnglePropertyName, ETrackType::Float);
			}
		}
		else if (const UDirectionalLightComponent* DirectionalLightComponent = Cast<UDirectionalLightComponent>(ComponentToBind))
		{
			PropertyPathToTrackType.Add(LightSourceAnglePropertyName, ETrackType::Float);
		}
	}

	// If we were told to specifically read only some property paths, ignore the other ones
	if (PropertyPathsToRead.Num() > 0)
	{
		for (TMap<FName, ETrackType>::TIterator Iter = PropertyPathToTrackType.CreateIterator(); Iter; ++Iter)
		{
			const FName& PropertyPath = Iter->Key;
			if (!PropertyPathsToRead.Contains(PropertyPath))
			{
				Iter.RemoveCurrent();
			}
		}
	}

	for (const TPair<FName, ETrackType>& Pair : PropertyPathToTrackType)
	{
		const FName& PropertyPath = Pair.Key;
		ETrackType TrackType = Pair.Value;

		TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, PropertyPath);
		if (Attrs.Num() < 1)
		{
			continue;
		}

		// The main attribute is the first one, and that will dictate whether the track is muted or not
		// This because we don't want to mute the intensity track if just our rect light width track is muted, for example
		UE::FUsdAttribute MainAttr = Attrs[0];
		const bool bIsMuted = MainAttr && MainAttr.GetNumTimeSamples() > 0 && UsdUtils::IsAttributeMuted(MainAttr, UsdStage);

		// Remove attributes we failed to find on this prim (no authored data)
		// As long as we have at least one attribute with timesamples we can carry on, because we can rely on fallback/default values for the others
		for (int32 AttrIndex = Attrs.Num() - 1; AttrIndex >= 0; --AttrIndex)
		{
			UE::FUsdAttribute& Attr = Attrs[AttrIndex];
			FString AttrPath = Attr.GetPath().GetString();

			if (!Attr || Attr.GetNumTimeSamples() == 0)
			{
				Attrs.RemoveAt(AttrIndex);
			}
		}

		TArray<double> UnionedTimeSamples;
		if (Attrs.Num() == 0 || !UE::FUsdAttribute::GetUnionedTimeSamples(Attrs, UnionedTimeSamples))
		{
			continue;
		}

		UsdUtils::DensifyTimeSamplesIfNeeded(Prim, PropertyPath, UnionedTimeSamples);

		UE::FSdfLayer AttributeLayer;
		ULevelSequence* AttributeSequence = FindOrAddSequenceForAttribute(MainAttr, &AttributeLayer);
		if (!AttributeSequence)
		{
			return;
		}

		UMovieScene* MovieScene = AttributeSequence->GetMovieScene();
		if (!MovieScene)
		{
			continue;
		}

		UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, AttributeLayer, UsdStage};

		const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(AttributeSequence);

		UsdToUnreal::FPropertyTrackReader Reader = UsdToUnreal::CreatePropertyTrackReader(Prim, PropertyPath);

		switch (TrackType)
		{
			case ETrackType::Bool:
			{
				if (UMovieSceneBoolTrack* BoolTrack = AddTrack<
						UMovieSceneBoolTrack>(PropertyPath, PrimTwin, *ComponentToBind, *AttributeSequence, bIsMuted))
				{
					UsdToUnreal::ConvertBoolTimeSamples(UsdStage, UnionedTimeSamples, Reader.BoolReader, *BoolTrack, SequenceTransform);
				}
				break;
			}
			case ETrackType::Float:
			{
				if (UMovieSceneFloatTrack* FloatTrack = AddTrack<
						UMovieSceneFloatTrack>(PropertyPath, PrimTwin, *ComponentToBind, *AttributeSequence, bIsMuted))
				{
					UsdToUnreal::ConvertFloatTimeSamples(UsdStage, UnionedTimeSamples, Reader.FloatReader, *FloatTrack, SequenceTransform);
				}
				break;
			}
			case ETrackType::Color:
			{
				if (UMovieSceneColorTrack* ColorTrack = AddTrack<
						UMovieSceneColorTrack>(PropertyPath, PrimTwin, *ComponentToBind, *AttributeSequence, bIsMuted))
				{
					UsdToUnreal::ConvertColorTimeSamples(UsdStage, UnionedTimeSamples, Reader.ColorReader, *ColorTrack, SequenceTransform);
				}
				break;
			}
			default:
				continue;
		}

		PrimPathByLevelSequenceName.AddUnique(AttributeSequence->GetFName(), Prim.GetPrimPath().GetString());
	}
}

void FUsdLevelSequenceHelperImpl::AddSkeletalTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim)
{
	USkeletalMeshComponent* ComponentToBind = Cast<USkeletalMeshComponent>(PrimTwin.GetSceneComponent());
	if (!ComponentToBind)
	{
		return;
	}

	UUsdPrimLinkCache* PrimLinkCachePtr = PrimLinkCache.Get();
	if (!PrimLinkCachePtr)
	{
		return;
	}

	UE::FUsdPrim SkelRootPrim = UsdUtils::GetClosestParentSkelRoot(Prim);

	// We'll place the skeletal animation track wherever the SkelAnimation prim is defined (not necessarily the
	// same layer as the skel root)
	UE::FUsdPrim SkelAnimationPrim = UsdUtils::FindAnimationSource(SkelRootPrim, Prim);
	if (!SkelAnimationPrim)
	{
		return;
	}

	// Fetch the UAnimSequence asset from the asset cache. Ideally we'd call AUsdStageActor::GetGeneratedAssets,
	// but we may belong to a FUsdStageImportContext, and so there's no AUsdStageActor at all to use.
	// At this point it doesn't matter much though, because we shouldn't need to uncollapse a SkelAnimation prim path anyway
	const UE::FSdfPath PrimPath = Prim.GetPrimPath();
	UAnimSequence* Sequence = PrimLinkCachePtr->GetInner().GetSingleAssetForPrim<UAnimSequence>(PrimPath);
	if (!Sequence)
	{
		return;
	}

	UE::FUsdAttribute TranslationsAttr = SkelAnimationPrim.GetAttribute(TEXT("translations"));
	UE::FUsdAttribute RotationsAttr = SkelAnimationPrim.GetAttribute(TEXT("rotations"));
	UE::FUsdAttribute ScalesAttr = SkelAnimationPrim.GetAttribute(TEXT("scales"));
	UE::FUsdAttribute BlendShapeWeightsAttr = SkelAnimationPrim.GetAttribute(TEXT("blendShapeWeights"));

	UE::FUsdAttribute StrongestAttribute = UsdUtils::GetStrongestAttribute(
		{TranslationsAttr, RotationsAttr, ScalesAttr, BlendShapeWeightsAttr},
		UsdUtils::GetEarliestTimeCode()
	);
	if (!StrongestAttribute)
	{
		return;
	}

	UE::FSdfLayer SkelAnimationLayer;
	ULevelSequence* SkelAnimationSequence = FindOrAddSequenceForAttribute(StrongestAttribute, &SkelAnimationLayer);
	if (!SkelAnimationSequence)
	{
		return;
	}

	UMovieScene* MovieScene = SkelAnimationSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, SkelAnimationLayer, UsdStage};

	// We will mute all SkelAnimation attributes if we mute, so here let's only consider something muted
	// if it has all attributes muted as well.
	// We know at least one of these attributes ones is valid and animated because we have an UAnimSequence
	const bool bIsMuted = (!TranslationsAttr || UsdUtils::IsAttributeMuted(TranslationsAttr, UsdStage))
						  && (!RotationsAttr || UsdUtils::IsAttributeMuted(RotationsAttr, UsdStage))
						  && (!ScalesAttr || UsdUtils::IsAttributeMuted(ScalesAttr, UsdStage))
						  && (!BlendShapeWeightsAttr || UsdUtils::IsAttributeMuted(BlendShapeWeightsAttr, UsdStage));

	if (UMovieSceneSkeletalAnimationTrack* SkeletalTrack = AddTrack<
			UMovieSceneSkeletalAnimationTrack>(SkelAnimationPrim.GetName(), PrimTwin, *ComponentToBind, *SkelAnimationSequence, bIsMuted))
	{
		UE::FUsdResolveInfo Info = StrongestAttribute.GetResolveInfo(UsdUtils::GetEarliestTimeCode());
		UE::FPcpNodeRef PcpNode = Info.GetNode();
		UE::FPcpMapExpression MapToRoot = PcpNode.GetMapToRoot();
		UE::FSdfLayerOffset TotalOffset = MapToRoot.GetTimeOffset();

		double StartSecondsInStage = 0.0f;
		if (UUsdAnimSequenceAssetUserData* UserData = Sequence->GetAssetUserData<UUsdAnimSequenceAssetUserData>())
		{
			StartSecondsInStage = UserData->StageStartOffsetSeconds;
		}

		const double StartTimeCodeInStage = StartSecondsInStage * UsdStage.GetTimeCodesPerSecond();
		const double StartTimeCodeInLayer = (StartTimeCodeInStage - TotalOffset.Offset) / TotalOffset.Scale;
		const double StartSecondsInLayer = StartTimeCodeInLayer / SkelAnimationLayer.GetTimeCodesPerSecond();
		FFrameNumber StartTickInLayer = FFrameTime::FromDecimal(StartSecondsInLayer * MovieScene->GetTickResolution().AsDecimal()).RoundToFrame();

		SkeletalTrack->Modify();
		SkeletalTrack->RemoveAllAnimationData();

		UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(
			SkeletalTrack->AddNewAnimation(StartTickInLayer, Sequence)
		);
		NewSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
	}

	PrimPathByLevelSequenceName.AddUnique(SkelAnimationSequence->GetFName(), PrimPath.GetString());
}

void FUsdLevelSequenceHelperImpl::AddGeometryCacheTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim)
{
	UGeometryCacheComponent* ComponentToBind = Cast<UGeometryCacheComponent>(PrimTwin.GetSceneComponent());
	UUsdPrimLinkCache* PrimLinkCachePtr = PrimLinkCache.Get();
	if (!ComponentToBind || !PrimLinkCachePtr)
	{
		return;
	}

	// Fetch the geometry cache asset from the asset cache. If there's none, don't actually need to create track
	const UE::FSdfPath PrimPath = Prim.GetPrimPath();
	UGeometryCache* GeometryCache = PrimLinkCachePtr->GetInner().GetSingleAssetForPrim<UGeometryCache>(PrimPath);
	if (!GeometryCache)
	{
		return;
	}

	if (ComponentToBind->GetGeometryCache() != GeometryCache)
	{
		return;
	}

	TArray<UE::FUsdAttribute> AnimatedMeshAttributes = UsdUtils::GetAnimatedMeshAttributes(Prim);
	UE::FUsdAttribute StrongestAttribute = UsdUtils::GetStrongestAttribute(AnimatedMeshAttributes, UsdUtils::GetEarliestTimeCode());

	UE::FSdfLayer GeometryCacheLayer;
	ULevelSequence* GeometryCacheSequence = FindOrAddSequenceForAttribute(StrongestAttribute, &GeometryCacheLayer);
	if (!ensure(GeometryCacheSequence)) // We should always be able to find a place to put our section
	{
		return;
	}

	UMovieScene* MovieScene = GeometryCacheSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, GeometryCacheLayer, UsdStage};

	const bool bIsMuted = false;
	if (UMovieSceneGeometryCacheTrack* GeometryCacheTrack = AddTrack<
			UMovieSceneGeometryCacheTrack>(Prim.GetName(), PrimTwin, *ComponentToBind, *GeometryCacheSequence, bIsMuted))
	{
		UE::FUsdResolveInfo Info = StrongestAttribute.GetResolveInfo(UsdUtils::GetEarliestTimeCode());
		UE::FPcpNodeRef PcpNode = Info.GetNode();
		UE::FPcpMapExpression MapToRoot = PcpNode.GetMapToRoot();
		UE::FSdfLayerOffset TotalOffset = MapToRoot.GetTimeOffset();

		double StartSecondsInStage = 0;
		if (UUsdGeometryCacheAssetUserData* UserData = GeometryCache->GetAssetUserData<UUsdGeometryCacheAssetUserData>())
		{
			StartSecondsInStage = UserData->StageStartOffsetSeconds;
		}

		const double StartTimeCodeInStage = StartSecondsInStage * UsdStage.GetTimeCodesPerSecond();
		const double StartTimeCodeInLayer = (StartTimeCodeInStage - TotalOffset.Offset) / TotalOffset.Scale;
		const double StartSecondsInLayer = StartTimeCodeInLayer / GeometryCacheLayer.GetTimeCodesPerSecond();
		FFrameNumber StartTickInLayer = FFrameTime::FromDecimal(StartSecondsInLayer * MovieScene->GetTickResolution().AsDecimal()).RoundToFrame();

		GeometryCacheTrack->Modify();
		GeometryCacheTrack->RemoveAllAnimationData();

		UMovieSceneGeometryCacheSection* NewSection = Cast<UMovieSceneGeometryCacheSection>(
			GeometryCacheTrack->AddNewAnimation(StartTickInLayer, ComponentToBind)
		);
		NewSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;

		// The geometry cache assumes a playback rate respective to the stage, but we need to make this work with a playback
		// rate respective to the LevelSequence the section is being placed in, as we'll end up putting timeScale on
		// our parent sequences. E.g. if we have a sublayer with a scale of 2.0, we'll put a timeScale of 0.5 on the
		// subsequence section, and here we'll set PlayRate to 2.0 as well: When playing the parent LevelSequence
		// we'll see the "native" playback rate of the geometry cache, but if we played the child LevelSequence alone
		// we'd see the geometry cache in the rate that matches the actual layer (even though the asset was produced for the 
		// stage as a whole)
		if (!FMath::IsNearlyEqual(TotalOffset.Scale, 1.0))
		{
			NewSection->Params.PlayRate = TotalOffset.Scale;

#if WITH_EDITOR
			// See UMovieSceneGeometryCacheSection::PostEditChangeProperty
			FPropertyChangedEvent PropertyChangeEvent{
				FMovieSceneGeometryCacheParams::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FMovieSceneGeometryCacheParams, PlayRate))
			};
			NewSection->Modify();
			NewSection->PostEditChangeProperty(PropertyChangeEvent);
#endif // WITH_EDITOR
		}
	}

	PrimPathByLevelSequenceName.AddUnique(GeometryCacheSequence->GetFName(), PrimPath.GetString());
}

void FUsdLevelSequenceHelperImpl::AddGroomTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim)
{
	UGroomComponent* ComponentToBind = Cast<UGroomComponent>(PrimTwin.GetSceneComponent());
	if (!ComponentToBind)
	{
		return;
	}

	UUsdPrimLinkCache* PrimLinkCachePtr = PrimLinkCache.Get();
	if (!PrimLinkCachePtr)
	{
		return;
	}

	// Fetch the groom cache asset from the asset cache. If there's none, don't actually need to create track
	const FString PrimPath = Prim.GetPrimPath().GetString();
	const FString GroomCachePath = FString::Printf(TEXT("%s_strands_cache"), *PrimPath);
	UGroomCache* GroomCache = PrimLinkCachePtr->GetInner().GetSingleAssetForPrim<UGroomCache>(UE::FSdfPath{*GroomCachePath});
	if (!GroomCache)
	{
		return;
	}

	if (ComponentToBind->GroomCache.Get() != GroomCache)
	{
		return;
	}

	TArray<UE::FUsdAttribute> AnimatedAttributes = UsdUtils::GetAnimatedAttributes(Prim);
	UE::FUsdAttribute StrongestAttribute = UsdUtils::GetStrongestAttribute(AnimatedAttributes, UsdUtils::GetEarliestTimeCode());

	UE::FSdfLayer GroomLayer;
	ULevelSequence* GroomAnimationSequence = FindOrAddSequenceForAttribute(StrongestAttribute, &GroomLayer);
	if (!GroomAnimationSequence)
	{
		return;
	}

	UMovieScene* MovieScene = GroomAnimationSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, GroomLayer, UsdStage};

	const bool bIsMuted = false;
	if (UMovieSceneGroomCacheTrack* GroomCacheTrack = AddTrack<
			UMovieSceneGroomCacheTrack>(Prim.GetName(), PrimTwin, *ComponentToBind, *GroomAnimationSequence, bIsMuted))
	{
		GroomCacheTrack->Modify();
		GroomCacheTrack->RemoveAllAnimationData();

		const FFrameNumber StartOffset;
		UMovieSceneGroomCacheSection* NewSection = Cast<UMovieSceneGroomCacheSection>(GroomCacheTrack->AddNewAnimation(StartOffset, ComponentToBind));
		NewSection->EvalOptions.CompletionMode = EMovieSceneCompletionMode::KeepState;
	}

	PrimPathByLevelSequenceName.AddUnique(GroomAnimationSequence->GetFName(), PrimPath);
}

void FUsdLevelSequenceHelperImpl::AddVolumeTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim)
{
	using namespace UnrealIdentifiers;

	UHeterogeneousVolumeComponent* VolumeComponent = Cast<UHeterogeneousVolumeComponent>(PrimTwin.GetSceneComponent());
	if (!VolumeComponent)
	{
		return;
	}

	const FName PropertyPath = GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame);

	// Here we'll just get *any* of the filePath attrs from this Volume prim to check for the
	// muted bool. We won't use the attribute itself for the baking though, as our timeSamples are already
	// on the SparseVolumeTexture AssetUserData, and the keyframe values are just their indices
	TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(Prim, PropertyPath);
	if (Attrs.Num() < 1)
	{
		return;
	}
	UE::FUsdAttribute MainAttr = Attrs[0];
	const bool bIsMuted = MainAttr && MainAttr.GetNumTimeSamples() > 0 && UsdUtils::IsAttributeMuted(MainAttr, UsdStage);

	UE::FSdfLayer PrimLayer;
	ULevelSequence* FrameAttributeSequence = FindOrAddSequenceForAttribute(MainAttr, &PrimLayer);
	if (!FrameAttributeSequence)
	{
		return;
	}

	UMovieScene* MovieScene = FrameAttributeSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, PrimLayer, UsdStage};

	UUsdSparseVolumeTextureAssetUserData* UserData = nullptr;
	const int32 ElementIndex = 0;
	UMaterialInterface* CurrentMaterial = VolumeComponent->GetMaterial(ElementIndex);
	if (CurrentMaterial)
	{
		TArray<FMaterialParameterInfo> ParameterInfo;
		TArray<FGuid> ParameterIds;
		CurrentMaterial->GetAllSparseVolumeTextureParameterInfo(ParameterInfo, ParameterIds);

		// Follow the theme of only ever caring about the first SVT parameter of the material, as that is all
		// that the UHeterogeneousVolumeComponent will ever animate anyway. If we're in here we already know
		// that this first SVT should be animated at any case
		if (ParameterInfo.Num() > 0)
		{
			const FMaterialParameterInfo& Info = ParameterInfo[0];
			USparseVolumeTexture* SparseVolumeTexture = nullptr;
			if (CurrentMaterial->GetSparseVolumeTextureParameterValue(Info, SparseVolumeTexture) && SparseVolumeTexture)
			{
				if (SparseVolumeTexture->GetNumFrames() > 1)
				{
					UserData = Cast<UUsdSparseVolumeTextureAssetUserData>(UsdUnreal::ObjectUtils::GetAssetUserData(SparseVolumeTexture));
				}
			}
		}
	}
	if (!UserData)
	{
		return;
	}

	TArray<double>* TimeSamples = &UserData->TimeSamplePathTimeCodes;
	TArray<int32>* FrameIndices = nullptr;
	if (UserData->TimeSamplePathIndices.Num() == UserData->TimeSamplePathTimeCodes.Num())
	{
		FrameIndices = &UserData->TimeSamplePathIndices;
	}
	else
	{
		USD_LOG_WARNING(
			TEXT(
				"Ignoring AssetUserData TimeSamplePathIndices when generating Sequencer tracks for Prim '%s' because it has %d entries, while it should have the same number of entries as TimeSamplePathTimeCodes (%d)"
			),
			*Prim.GetPrimPath().GetString(),
			UserData->TimeSamplePathIndices.Num(),
			UserData->TimeSamplePathTimeCodes.Num()
		);
	}

	if (!TimeSamples || TimeSamples->Num() < 2)
	{
		return;
	}

	// Our TimeSamples are local to the layer where they were defined, but we need to convert them
	// to be with respect to the stage in order to find the right locations for the key frames
	UE::FUsdPrim PrimForOffsetCalculation = Prim;
	if (UserData->SourceOpenVDBAssetPrimPaths.Num() > 0)
	{
		const FString& FirstAssetPrimPath = UserData->SourceOpenVDBAssetPrimPaths[0];
		UE::FUsdPrim FirstAssetPrim = Prim.GetStage().GetPrimAtPath(UE::FSdfPath{*FirstAssetPrimPath});
		if (FirstAssetPrim)
		{
			PrimForOffsetCalculation = FirstAssetPrim;
		}
	}
	UE::FSdfLayerOffset CombinedOffset = UsdUtils::GetPrimToStageOffset(UE::FUsdPrim{PrimForOffsetCalculation});
	TArray<double> ConvertedTimeSamples;
	ConvertedTimeSamples.Reserve(TimeSamples->Num());
	for (double TimeSample : *TimeSamples)
	{
		ConvertedTimeSamples.Add(TimeSample * CombinedOffset.Scale + CombinedOffset.Offset);
	}

	// We still have a Sequence transform though, as that converts the TimeSamples from being global
	// to the stage to the particular subsequence where they are going to be added to.
	const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(FrameAttributeSequence);

	// Unlike the other cases we can create our Reader right here, because the only thing we need to generate
	// the track are the TimeSamples
	UsdToUnreal::FPropertyTrackReader Reader;
	Reader.FloatReader = [&ConvertedTimeSamples, FrameIndices, TargetIndex = 0](double UsdTimeCode) mutable -> float
	{
		// Reference: FUsdVolVolumeTranslator::UpdateComponents
		for (; TargetIndex + 1 < ConvertedTimeSamples.Num(); ++TargetIndex)
		{
			if (ConvertedTimeSamples[TargetIndex + 1] > UsdTimeCode)
			{
				break;
			}
		}
		TargetIndex = FMath::Clamp(TargetIndex, 0, ConvertedTimeSamples.Num() - 1);

		if (FrameIndices && FrameIndices->IsValidIndex(TargetIndex))
		{
			TargetIndex = (*FrameIndices)[TargetIndex];
		}

		return static_cast<float>(TargetIndex);
	};

	if (UMovieSceneFloatTrack* FloatTrack = AddTrack<UMovieSceneFloatTrack>(PropertyPath, PrimTwin, *VolumeComponent, *FrameAttributeSequence, bIsMuted))
	{
		// The component won't really linearly interpolate anything and will just do the analogous of constant interpolation,
		// so it would be nice if our keys showed that too
		ERichCurveInterpMode InterpolationModeOverride = ERichCurveInterpMode::RCIM_Constant;
		UsdToUnreal::ConvertFloatTimeSamples(
			UsdStage,
			ConvertedTimeSamples,
			Reader.FloatReader,
			*FloatTrack,
			SequenceTransform,
			InterpolationModeOverride
		);
	}

	PrimPathByLevelSequenceName.AddUnique(FrameAttributeSequence->GetFName(), Prim.GetPrimPath().GetString());
}

namespace UE::USDLevelSequenceHelper::Private
{
	template<typename T>
	TOptional<T> GetAuthoredValue(const UE::FUsdPrim& Prim, const FString& AttrName)
	{
		if (UE::FUsdAttribute Attr = Prim.GetAttribute(*AttrName))
		{
			UE::FVtValue VtValue;
			if (Attr.HasAuthoredValue() && Attr.Get(VtValue) && !VtValue.IsEmpty())
			{
				return UsdUtils::GetUnderlyingValue<T>(VtValue);
			}
		}

		return TOptional<T>{};
	}
};

void FUsdLevelSequenceHelperImpl::AddAudioTracks(const UUsdPrimTwin& PrimTwin, const UE::FUsdPrim& Prim)
{
	using namespace UnrealIdentifiers;
	using namespace UE::USDLevelSequenceHelper::Private;

	UAudioComponent* AudioComponent = Cast<UAudioComponent>(PrimTwin.GetSceneComponent());
	if (!AudioComponent)
	{
		return;
	}

	UUsdPrimLinkCache* PrimLinkCachePtr = PrimLinkCache.Get();
	if (!PrimLinkCachePtr)
	{
		return;
	}

	// Note: We pull the audio directly from the info cache here, and not the component:
	// See big comment within FUsdMediaSpatialAudioTranslator::UpdateComponents
	USoundBase* Sound = PrimLinkCachePtr->GetInner().GetSingleAssetForPrim<USoundBase>(Prim.GetPrimPath());
	if (!Sound)
	{
		return;
	}

	static FString FilePathToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->filePath);
	static FString AuralModeToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->auralMode);
	static FString MediaOffsetSecondsToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->mediaOffset);
	static FString StartTimeCodeToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->startTime);
	static FString EndTimeCodeToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->endTime);
	static FString PlaybackModeToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->playbackMode);
	static FString GainToken = UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->gain);

	// Set up the actual audio track
	{
		// If we're using references so that our prim references another layer with the SpatialAudio prim,
		// using just "the layer for the prim" here would mean we end up with the referencer layer, and so
		// we'd end up placing the audio track directly on the LevelSequence for the referencer layer, which
		// is not what we want.
		// Instead, we use the file path attribute as the "main attribute" for the audio track: If you author
		// filePath on the referencer layer, the audio track will end up on the corresponding LevelSequence.
		// If you author it on the referenced layer, the audio track will end up on that subsequence
		UE::FUsdAttribute Attr = Prim.GetAttribute(*FilePathToken);

		UE::FSdfLayer SequenceLayer;
		ULevelSequence* AttributeSequence = FindOrAddSequenceForAttribute(Attr, &SequenceLayer);
		if (!AttributeSequence)
		{
			return;
		}

		UMovieScene* MovieScene = AttributeSequence->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		UE::USDLevelSequenceHelper::Private::FScopedReadOnlyDisable ReadOnlyGuard{MovieScene, SequenceLayer, UsdStage};

		// Since the schema has fallbacks for these, we should always have values here.
		//
		// Note that since startTime and endTime have pxr::SdfTimeCode data types, USD will already do the
		// proper conversions regarding sublayer/reference offset and scale. In other words, these timeCode
		// values as retrieved by UE::FUsdAttribute::Get() are *already relative to the stage*.
		// Example: If startTime was set to 15 for a SpatialAudio prim defined in a sublayer, added to the stage
		// with offset of 10 and scale of 2, when querying the attribute for the prim we'd get that startTime
		// actually has the value of 40, because 15 * 2 + 10 = 40
		TOptional<double> MediaOffset = GetAuthoredValue<double>(Prim, MediaOffsetSecondsToken);
		TOptional<pxr::SdfTimeCode> StartTimeCode = GetAuthoredValue<pxr::SdfTimeCode>(Prim, StartTimeCodeToken);
		TOptional<pxr::SdfTimeCode> EndTimeCode = GetAuthoredValue<pxr::SdfTimeCode>(Prim, EndTimeCodeToken);
		TOptional<pxr::TfToken> PlaybackMode = GetAuthoredValue<pxr::TfToken>(Prim, PlaybackModeToken);
		TOptional<pxr::TfToken> AuralMode = GetAuthoredValue<pxr::TfToken>(Prim, AuralModeToken);

		// The documentation mentions to swap these in edge cases like negative scaling
		if (StartTimeCode.IsSet() && EndTimeCode.IsSet())
		{
			if (StartTimeCode.GetValue() > EndTimeCode.GetValue())
			{
				Swap(StartTimeCode, EndTimeCode);
			}
		}

		// Handle the different playback modes
		bool bIsLooping = false;
		if (PlaybackMode.IsSet())
		{
			const pxr::TfToken PlaybackModeValue = PlaybackMode.GetValue();
			if (PlaybackModeValue == pxr::UsdMediaTokens->onceFromStart)
			{
				// "Play the audio once, starting at startTime, continuing until the audio completes."
				EndTimeCode.Reset();
			}
			else if (PlaybackModeValue == pxr::UsdMediaTokens->onceFromStartToEnd)
			{
				// "Play the audio once beginning at startTime, continuing until endTime or until the
				// audio completes, whichever comes first."
				//
				// Do nothing: Just continue using startTime and endTime as we have already
			}
			else if (PlaybackModeValue == pxr::UsdMediaTokens->loopFromStart)
			{
				// "Start playing the audio at startTime and continue looping through to the stage's
				// authored endTimeCode."
				bIsLooping = true;
				EndTimeCode = Prim.GetStage().GetEndTimeCode();
			}
			else if (PlaybackModeValue == pxr::UsdMediaTokens->loopFromStartToEnd)
			{
				// "Start playing the audio at startTime and continue looping through, stopping
				// the audio at endTime."
				bIsLooping = true;
			}
			else if (PlaybackModeValue == pxr::UsdMediaTokens->loopFromStage)
			{
				// "Start playing the audio at the stage's authored startTimeCode and continue looping
				// through to the stage's authored endTimeCode. This can be useful for ambient sounds
				// that should always be active."
				bIsLooping = true;

				// Fetch time range from the stage instead
				UE::FSdfLayer RootLayer = Prim.GetStage().GetRootLayer();
				if (RootLayer)
				{
					StartTimeCode = RootLayer.GetStartTimeCode();

					// Since the fallback value for EndTimeCode is also 0, we have to take care to check
					// whether the stage actually has it authored or not
					if (RootLayer.HasEndTimeCode())
					{
						EndTimeCode = RootLayer.GetEndTimeCode();
					}
					else
					{
						EndTimeCode.Reset();
					}
				}
			}
		}

		// Convert time codes to being relative to the stage to being relative to the Subsequence they are in
		//
		// This may seem like it undoes the conversions above, and it really does: For cases where we have a
		// sublayer with an offset and scale, we'll end up adding the layer-local time samples to the subsequence,
		// like we want. Using PrimToStage AND the SequenceTransform is needed for a different case however: Prim
		// references with sublayer and offsets. In that case the prim itself may have a sublayer and offset, but
		// its track will be placed in the levelsequence for the *referencer* layer: This means we want to see the
		// keys on that layer instead, at times relative to it
		FFrameNumber StartFrameNumber;
		TOptional<FFrameNumber> EndFrameNumber;
		FFrameNumber MediaFrameNumberOffset;
		{
			const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(AttributeSequence);

			const FFrameRate Resolution = MovieScene->GetTickResolution();
			const FFrameRate DisplayRate = MovieScene->GetDisplayRate();
			const double StageTimeCodesPerSecond = Prim.GetStage().GetTimeCodesPerSecond();
			const FFrameRate StageFrameRate(StageTimeCodesPerSecond, 1);

			if (StartTimeCode.IsSet())
			{
				double DoubleValue = static_cast<double>(StartTimeCode.GetValue());
				int32 FrameNumber = FMath::FloorToInt(DoubleValue);
				float SubFrameNumber = DoubleValue - FrameNumber;
				FFrameTime FrameTime(FrameNumber, SubFrameNumber);
				FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
				KeyFrameTime *= SequenceTransform;
				StartFrameNumber = KeyFrameTime.FloorToFrame();
			}

			if (EndTimeCode.IsSet())
			{
				double DoubleValue = static_cast<double>(EndTimeCode.GetValue());
				int32 FrameNumber = FMath::FloorToInt(DoubleValue);
				float SubFrameNumber = DoubleValue - FrameNumber;
				FFrameTime FrameTime(FrameNumber, SubFrameNumber);
				FFrameTime KeyFrameTime = FFrameRate::TransformTime(FrameTime, StageFrameRate, Resolution);
				KeyFrameTime *= SequenceTransform;
				EndFrameNumber = KeyFrameTime.FloorToFrame();
			}

			if (MediaOffset.IsSet())
			{
				double OffsetSeconds = MediaOffset.GetValue();
				MediaFrameNumberOffset = Resolution.AsFrameNumber(OffsetSeconds);
			}
		}

		// Get the attenuation settings we'll use if we're in spatial mode
		USoundAttenuation* Attenuation = nullptr;
		if (AuralMode.IsSet() && AuralMode.GetValue() == pxr::UsdMediaTokens->spatial)
		{
			const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>();
			if (ProjectSettings)
			{
				Attenuation = Cast<USoundAttenuation>(ProjectSettings->DefaultSoundAttenuation.TryLoad());
			}
		}

		MovieScene->Modify();
		MovieScene->SetClockSource(EUpdateClockSource::Audio);

		// We name it AudioTrack here instead of prim name because in general we'll already have the actor and component
		// binding show the prim name anyway... It's not very useful to see "MyPrim / MyPrim / MyPrim" on the Sequencer
		const bool bIsMuted = false;
		if (UMovieSceneAudioTrack* AudioTrack = AddTrack<
				UMovieSceneAudioTrack>(TEXT("Audio track"), PrimTwin, *AudioComponent, *AttributeSequence, bIsMuted))
		{
			AudioTrack->Modify();
			AudioTrack->RemoveAllAnimationData();

			const FFrameNumber Time = StartFrameNumber;
			UMovieSceneAudioSection* NewAudioSection = Cast<UMovieSceneAudioSection>(AudioTrack->AddNewSound(Sound, Time));

			if (NewAudioSection)
			{
				NewAudioSection->Modify();
				NewAudioSection->SetLooping(bIsLooping);
				NewAudioSection->SetStartOffset(MediaFrameNumberOffset);

				// Parse gain into channel volume
				UE::FUsdAttribute GainAttr = Prim.GetAttribute(*GainToken);
				if (GainAttr && GainAttr.HasAuthoredValue())
				{
					// We have to dig through the channel proxy because UMovieSceneAudioSection doesn't expose non-const access
					// to the volume channel...
					FMovieSceneChannelProxy& ChannelProxy = NewAudioSection->GetChannelProxy();
					FMovieSceneFloatChannel* VolumeChannel = ChannelProxy.GetChannel<FMovieSceneFloatChannel>(0);
					if (VolumeChannel)
					{
						// Set default value into the channel in case we don't have any animation
						UE::FVtValue DefaultValue;
						if (GainAttr.Get(DefaultValue))
						{
							TOptional<double> DefaultGain = UsdUtils::GetUnderlyingValue<double>(DefaultValue);
							if (DefaultGain.IsSet())
							{
								VolumeChannel->SetDefault(static_cast<float>(DefaultGain.GetValue()));
							}
						}

						// Parse time-sampled animation into the volume channel
						TArray<double> TimeSamples;
						if (GainAttr.GetTimeSamples(TimeSamples) && TimeSamples.Num() > 0)
						{
							const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(AttributeSequence);
							UsdToUnreal::FPropertyTrackReader Reader = UsdToUnreal::CreatePropertyTrackReader(Prim, TEXT("Volume"));
							UsdToUnreal::ConvertFloatTimeSamples(
								UsdStage,
								TimeSamples,
								Reader.FloatReader,
								*VolumeChannel,
								*MovieScene,
								SequenceTransform
							);
						}
					}
				}

				// Enable spatial audio if we should
				if (Attenuation)
				{
					const bool bOverrideAttenuation = true;
					NewAudioSection->SetOverrideAttenuation(bOverrideAttenuation);
					NewAudioSection->SetAttenuationSettings(Attenuation);
				}

				// Start with the auto-size range because the section can't be unbounded, so if we want
				// the audio to play to completion we'd otherwise need to specify the end of the section
				// ourselves in order to get a valid range
				TOptional<TRange<FFrameNumber>> Range = NewAudioSection->GetAutoSizeRange();
				if (Range.IsSet())
				{
					TRange<FFrameNumber>& RangeValue = Range.GetValue();
					RangeValue.SetLowerBound(TRangeBound<FFrameNumber>{StartFrameNumber});
					if (EndFrameNumber.IsSet())
					{
						RangeValue.SetUpperBound(TRangeBound<FFrameNumber>{EndFrameNumber.GetValue()});
					}
					NewAudioSection->SetRange(RangeValue);
				}
			}
		}

		PrimPathByLevelSequenceName.AddUnique(AttributeSequence->GetFName(), Prim.GetPrimPath().GetString());
	}
}

void FUsdLevelSequenceHelperImpl::AddPrim(UUsdPrimTwin& PrimTwin, bool bForceVisibilityTracks, TOptional<bool> HasAnimatedBounds)
{
	if (!UsdStage)
	{
		return;
	}

	UE::FSdfPath PrimPath{*PrimTwin.PrimPath};
	UE::FUsdPrim UsdPrim{UsdStage.GetPrimAtPath(PrimPath)};
	if (!UsdPrim)
	{
		return;
	}

	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.DisableGeoCacheTracks"));
	const bool bDisableGeoCacheTracks = CVar && CVar->GetBool();

	if (UsdPrim.IsA(TEXT("Camera")))
	{
		AddCameraTracks(PrimTwin, UsdPrim);
	}
	else if (UsdPrim.HasAPI(TEXT("LightAPI")))
	{
		AddLightTracks(PrimTwin, UsdPrim);
	}
	else if (UsdPrim.IsA(TEXT("Skeleton")))
	{
		AddSkeletalTracks(PrimTwin, UsdPrim);
	}
	else if (!bDisableGeoCacheTracks && UsdPrim.IsA(TEXT("Mesh")) && UsdUtils::IsAnimated(UsdPrim))
	{
		AddGeometryCacheTracks(PrimTwin, UsdPrim);
	}
	else if (UsdUtils::PrimHasSchema(UsdPrim, UnrealIdentifiers::GroomAPI))
	{
		AddGroomTracks(PrimTwin, UsdPrim);
	}
	else if (UsdPrim.IsA(TEXT("Volume")))
	{
		AddVolumeTracks(PrimTwin, UsdPrim);
	}
	else if (UsdPrim.IsA(TEXT("SpatialAudio")))
	{
		AddAudioTracks(PrimTwin, UsdPrim);
	}

	AddCommonTracks(PrimTwin, UsdPrim, bForceVisibilityTracks);

	AddBoundsTracks(PrimTwin, UsdPrim, HasAnimatedBounds);

	RefreshSequencer();
}

template<typename TrackType>
TrackType* FUsdLevelSequenceHelperImpl::AddTrack(
	const FName& TrackName,
	const UUsdPrimTwin& PrimTwin,
	USceneComponent& ComponentToBind,
	ULevelSequence& Sequence,
	bool bIsMuted
)
{
	if (!UsdStage)
	{
		return nullptr;
	}

	UMovieScene* MovieScene = Sequence.GetMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}

	const FGuid ComponentBinding = GetOrCreateComponentBinding(PrimTwin, ComponentToBind, Sequence);

	TrackType* Track = MovieScene->FindTrack<TrackType>(ComponentBinding, TrackName);
	if (Track)
	{
		Track->Modify();
		Track->RemoveAllAnimationData();
	}
	else
	{
		Track = MovieScene->AddTrack<TrackType>(ComponentBinding);
		if (!Track)
		{
			return nullptr;
		}

		if constexpr (std::is_base_of_v<UMovieScenePropertyTrack, TrackType>)
		{
			Track->SetPropertyNameAndPath(TrackName, TrackName.ToString());
		}
#if WITH_EDITOR
		else if constexpr (std::is_base_of_v<UMovieSceneSkeletalAnimationTrack, TrackType>)
		{
			Track->SetDisplayName(FText::FromName(TrackName));
		}
		else if constexpr (std::is_base_of_v<UMovieSceneAudioTrack, TrackType>)
		{
			Track->SetDisplayName(FText::FromName(TrackName));
		}
#endif	  // WITH_EDITOR
	}

	UE::USDLevelSequenceHelper::Private::DisableTrack(Track, MovieScene, ComponentBinding.ToString(), Track->GetName(), bIsMuted);

	return Track;
}

void FUsdLevelSequenceHelperImpl::RemovePrim(const UUsdPrimTwin& PrimTwin)
{
	if (!UsdStage)
	{
		return;
	}

	// We can't assume that the UsdPrim still exists in the stage, it might have been removed already so work from the PrimTwin PrimPath.

	TSet<FName> PrimSequences;

	for (TPair<FName, FString>& PrimPathByLevelSequenceNamePair : PrimPathByLevelSequenceName)
	{
		if (PrimPathByLevelSequenceNamePair.Value == PrimTwin.PrimPath)
		{
			PrimSequences.Add(PrimPathByLevelSequenceNamePair.Key);
		}
	}

	TSet<ULevelSequence*> SequencesToRemoveForPrim;

	for (const FName& PrimSequenceName : PrimSequences)
	{
		for (const auto& IdentifierSequencePair : LevelSequencesByIdentifier)
		{
			if (IdentifierSequencePair.Value && IdentifierSequencePair.Value->GetFName() == PrimSequenceName)
			{
				SequencesToRemoveForPrim.Add(IdentifierSequencePair.Value);
			}
		}
	}

	RemovePossessable(PrimTwin);

	for (ULevelSequence* SequenceToRemoveForPrim : SequencesToRemoveForPrim)
	{
		RemoveSequenceForPrim(*SequenceToRemoveForPrim, PrimTwin);
	}

	RefreshSequencer();
}

void FUsdLevelSequenceHelperImpl::UpdateControlRigTracks(UUsdPrimTwin& PrimTwin)
{
#if WITH_EDITOR
	if (!UsdStage)
	{
		return;
	}

	UUsdPrimLinkCache* PrimLinkCachePtr = PrimLinkCache.Get();
	if (!PrimLinkCachePtr)
	{
		return;
	}

	UE::FSdfPath PrimPath{*PrimTwin.PrimPath};
	UE::FUsdPrim SkeletonPrim{UsdStage.GetPrimAtPath(PrimPath)};
	if (!SkeletonPrim)
	{
		return;
	}

	USkeletalMeshComponent* ComponentToBind = Cast<USkeletalMeshComponent>(PrimTwin.GetSceneComponent());
	if (!ComponentToBind)
	{
		return;
	}

	// Block here because USD needs to fire and respond to notices for the DefinePrim call to work,
	// but we need UsdUtils::BindAnimationSource to run before we get in here again or else we'll
	// repeatedly create Animation prims
	FScopedBlockNoticeListening BlockNotices(StageActor.Get());

	UE::FUsdPrim SkelRootPrim = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim);
	UE::FUsdPrim SkelAnimationPrim = UsdUtils::FindAnimationSource(SkelRootPrim, SkeletonPrim);

	// Temporarily consider how our API schema can be applied to the Skeleton prim or a parent SkelRoot
	UE::FUsdPrim PrimWithSchema;
	if (UsdUtils::PrimHasSchema(SkeletonPrim, UnrealIdentifiers::ControlRigAPI))
	{
		PrimWithSchema = SkeletonPrim;
	}
	else if (SkelRootPrim && UsdUtils::PrimHasSchema(SkelRootPrim, UnrealIdentifiers::ControlRigAPI))
	{
		PrimWithSchema = SkelRootPrim;
	}

	// We'll place the skeletal animation track wherever the SkelAnimation prim is defined (not necessarily the
	// same layer as the skel root)
	UE::FSdfLayer SkelAnimationLayer;
	if (SkelAnimationPrim)
	{
		SkelAnimationLayer = UsdUtils::FindLayerForPrim(SkelAnimationPrim);
	}
	else
	{
		// If this SkelRoot doesn't have any animation, lets create a new one on the current edit target
		SkelAnimationLayer = UsdStage.GetEditTarget();

		FString UniqueChildName = UsdUtils::GetValidChildName(TEXT("Animation"), SkelRootPrim);
		SkelAnimationPrim = UsdStage.DefinePrim(SkelRootPrim.GetPrimPath().AppendChild(*UniqueChildName), TEXT("SkelAnimation"));
		if (!SkelAnimationPrim)
		{
			return;
		}

		// Let's always choose to author animSource within skeletons, as it works best in setups where
		// we have authored nested SkelRoots: The outer animSource would be inherited by the inner animSource otherwise!
		UsdUtils::BindAnimationSource(SkeletonPrim, SkelAnimationPrim);
	}
	if (!SkelAnimationLayer)
	{
		return;
	}

	// Fetch the UAnimSequence asset from the asset cache. Ideally we'd call AUsdStageActor::GetGeneratedAssets,
	// but we may belong to a FUsdStageImportContext, and so there's no AUsdStageActor at all to use.
	// At this point it doesn't matter much though, because we shouldn't need to uncollapse a SkelAnimation prim path anyway
	UAnimSequence* AnimSequence = PrimLinkCachePtr->GetInner().GetSingleAssetForPrim<UAnimSequence>(PrimPath);

	UE::FUsdEditContext EditContext{UsdStage, SkelAnimationLayer};
	FString Identifier = SkelAnimationLayer.GetIdentifier();

	// Force-create these because these are mandatory anyway
	// (https://graphics.pixar.com/usd/release/api/_usd_skel__schemas.html#UsdSkel_SkelAnimation)
	UE::FUsdAttribute JointsAttr = SkelAnimationPrim.CreateAttribute(TEXT("joints"), TEXT("token[]"));
	UE::FUsdAttribute TranslationsAttr = SkelAnimationPrim.CreateAttribute(TEXT("translations"), TEXT("float3[]"));
	UE::FUsdAttribute RotationsAttr = SkelAnimationPrim.CreateAttribute(TEXT("rotations"), TEXT("quatf[]"));
	UE::FUsdAttribute ScalesAttr = SkelAnimationPrim.CreateAttribute(TEXT("scales"), TEXT("half3[]"));
	UE::FUsdAttribute BlendShapeWeightsAttr = SkelAnimationPrim.GetAttribute(TEXT("blendShapeWeights"));

	ULevelSequence* SkelAnimationSequence = FindOrAddSequenceForAttribute(JointsAttr, &SkelAnimationLayer);
	if (!SkelAnimationSequence)
	{
		return;
	}

	UMovieScene* MovieScene = SkelAnimationSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const FGuid ComponentBinding = GetOrCreateComponentBinding(PrimTwin, *ComponentToBind, *SkelAnimationSequence);

	UMovieSceneControlRigParameterTrack* ControlRigTrack = MovieScene->FindTrack<UMovieSceneControlRigParameterTrack>(ComponentBinding);

	// We should be in control rig track mode but don't have any tracks yet --> Setup for Control Rig
	if (!ControlRigTrack)
	{
		bool bControlRigReduceKeys = false;
		if (UE::FUsdAttribute Attr = PrimWithSchema.GetAttribute(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReduceKeys)))
		{
			UE::FVtValue Value;
			if (Attr.Get(Value) && !Value.IsEmpty())
			{
				if (TOptional<bool> UnderlyingValue = UsdUtils::GetUnderlyingValue<bool>(Value))
				{
					bControlRigReduceKeys = UnderlyingValue.GetValue();
				}
			}
		}

		float ControlRigReduceTolerance = 0.001f;
		if (UE::FUsdAttribute Attr = PrimWithSchema.GetAttribute(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReductionTolerance)))
		{
			UE::FVtValue Value;
			if (Attr.Get(Value) && !Value.IsEmpty())
			{
				if (TOptional<float> UnderlyingValue = UsdUtils::GetUnderlyingValue<float>(Value))
				{
					ControlRigReduceTolerance = UnderlyingValue.GetValue();
				}
			}
		}

		bool bIsFKControlRig = false;
		if (UE::FUsdAttribute Attr = PrimWithSchema.GetAttribute(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig)))
		{
			UE::FVtValue Value;
			if (Attr.Get(Value))
			{
				if (TOptional<bool> UseFKOptional = UsdUtils::GetUnderlyingValue<bool>(Value))
				{
					bIsFKControlRig = UseFKOptional.GetValue();
				}
			}
		}

		UClass* ControlRigClass = nullptr;
		if (bIsFKControlRig)
		{
			ControlRigClass = UFKControlRig::StaticClass();
		}
		else
		{
			FString ControlRigBPPath;
			if (UE::FUsdAttribute Attr = PrimWithSchema.GetAttribute(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigPath)))
			{
				UE::FVtValue Value;
				if (Attr.Get(Value) && !Value.IsEmpty())
				{
					ControlRigBPPath = UsdUtils::Stringify(Value);
				}
			}

			if (UControlRigBlueprint* BP = Cast<UControlRigBlueprint>(FSoftObjectPath(ControlRigBPPath).TryLoad()))
			{
				ControlRigClass = BP->GetRigVMBlueprintGeneratedClass();
			}
		}

		if (ControlRigClass)
		{
			UAnimSeqExportOption* NewOptions = NewObject<UAnimSeqExportOption>();

			UE::USDLevelSequenceHelper::Private::BakeToControlRig(
				ComponentToBind->GetWorld(),
				SkelAnimationSequence,
				ControlRigClass,
				AnimSequence,
				ComponentToBind,
				NewOptions,
				bControlRigReduceKeys,
				ControlRigReduceTolerance,
				ComponentBinding
			);

			RefreshSequencer();
		}
	}

	PrimPathByLevelSequenceName.AddUnique(SkelAnimationSequence->GetFName(), PrimPath.GetString());
#endif	  // WITH_EDITOR
}

void FUsdLevelSequenceHelperImpl::RemoveSequenceForPrim(ULevelSequence& Sequence, const UUsdPrimTwin& PrimTwin)
{
	TArray<FString> PrimPathsForSequence;
	PrimPathByLevelSequenceName.MultiFind(Sequence.GetFName(), PrimPathsForSequence);

	if (PrimPathsForSequence.Find(PrimTwin.PrimPath) != INDEX_NONE)
	{
		PrimPathByLevelSequenceName.Remove(Sequence.GetFName(), PrimTwin.PrimPath);

		// If Sequence isn't used anymore, remove it and its subsection
		if (!PrimPathByLevelSequenceName.Contains(Sequence.GetFName()) && !LocalLayersSequences.Contains(Sequence.GetFName()))
		{
			ULevelSequence* ParentSequence = MainLevelSequence;
			FMovieSceneSequenceID SequenceID = SequencesID.FindRef(&Sequence);

			if (FMovieSceneSequenceHierarchyNode* NodeData = SequenceHierarchyCache.FindNode(SequenceID))
			{
				FMovieSceneSequenceID ParentSequenceID = NodeData->ParentID;

				if (FMovieSceneSubSequenceData* ParentSubSequenceData = SequenceHierarchyCache.FindSubData(ParentSequenceID))
				{
					ParentSequence = Cast<ULevelSequence>(ParentSubSequenceData->GetSequence());
				}
			}

			if (ParentSequence)
			{
				RemoveSubSequenceSection(*ParentSequence, Sequence);
			}

			FString OldIdentifier;
			if (IdentifierByLevelSequence.RemoveAndCopyValue(&Sequence, OldIdentifier) && !OldIdentifier.IsEmpty())
			{
				LevelSequencesByIdentifier.Remove(OldIdentifier);
			}

			LayerIdentifierByLevelSequenceName.Remove(Sequence.GetName());

			DirtySequenceHierarchyCache();
		}
	}
}

void FUsdLevelSequenceHelperImpl::RemovePossessable(const UUsdPrimTwin& PrimTwin)
{
	FPrimTwinBindings* Bindings = PrimTwinToBindings.Find(&PrimTwin);
	if (!Bindings)
	{
		return;
	}

	if (!Bindings->Sequence)
	{
		return;
	}

	UMovieScene* MovieScene = Bindings->Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	// The RemovePossessable calls Modify the MovieScene already, but the UnbindPossessableObject
	// ones don't modify the Sequence and change properties, so we must modify them here
	Bindings->Sequence->Modify();

	for (const TPair<TWeakObjectPtr<const UClass>, FGuid>& Pair : Bindings->ObjectClassToBindingGuid)
	{
		const FGuid& ComponentPossessableGuid = Pair.Value;

		FGuid ActorPossessableGuid;
		if (FMovieScenePossessable* ComponentPossessable = MovieScene->FindPossessable(ComponentPossessableGuid))
		{
			ActorPossessableGuid = ComponentPossessable->GetParent();
		}

		// This will also remove all tracks bound to this guid
		if (MovieScene->RemovePossessable(ComponentPossessableGuid))
		{
			Bindings->Sequence->UnbindPossessableObjects(ComponentPossessableGuid);
		}

		// If our parent binding has nothing else in it, we should remove it too
		bool bRemoveActorBinding = true;
		if (ActorPossessableGuid.IsValid())
		{
			for (int32 PossessableIndex = 0; PossessableIndex < MovieScene->GetPossessableCount(); ++PossessableIndex)
			{
				const FMovieScenePossessable& SomePossessable = MovieScene->GetPossessable(PossessableIndex);
				if (SomePossessable.GetParent() == ActorPossessableGuid)
				{
					bRemoveActorBinding = false;
					break;
				}
			}
		}
		if (bRemoveActorBinding)
		{
			MovieScene->RemovePossessable(ActorPossessableGuid);
			Bindings->Sequence->UnbindPossessableObjects(ActorPossessableGuid);
		}
	}

	PrimTwinToBindings.Remove(&PrimTwin);
}

void FUsdLevelSequenceHelperImpl::RefreshSequencer()
{
#if WITH_EDITOR
	if (!MainLevelSequence || !GIsEditor)
	{
		return;
	}

	if (TSharedPtr<ISequencer> Sequencer = UE::USDLevelSequenceHelper::Private::GetOpenedSequencerForLevelSequence(MainLevelSequence))
	{
		// Don't try refreshing the sequencer if its displaying a stale sequence (e.g. during busy transitions like import) as it
		// can crash
		if (UMovieSceneSequence* FocusedSequence = Sequencer->GetFocusedMovieSceneSequence())
		{
			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshTree);
		}
	}
#endif	  // WITH_EDITOR
}

void FUsdLevelSequenceHelperImpl::UpdateUsdLayerOffsetFromSection(const UMovieSceneSequence* Sequence, const UMovieSceneSubSection* Section)
{
	if (!Section || !Sequence)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	UMovieSceneSequence* SubSequence = Section->GetSequence();
	if (!MovieScene || !SubSequence)
	{
		return;
	}

	const FString* LayerIdentifier = LayerIdentifierByLevelSequenceName.Find(Sequence->GetName());
	const FString* SubSequenceLayerIdentifier = LayerIdentifierByLevelSequenceName.Find(SubSequence->GetName());

	if (!LayerIdentifier || !SubSequenceLayerIdentifier)
	{
		return;
	}

	FLayerTimeInfo* LayerTimeInfo = LayerTimeInfosByLayerIdentifier.Find(*LayerIdentifier);
	FLayerTimeInfo* SubSequenceLayerTimeInfo = LayerTimeInfosByLayerIdentifier.Find(*SubSequenceLayerIdentifier);

	if (!LayerTimeInfo || !SubSequenceLayerTimeInfo)
	{
		return;
	}

	const double TimeCodesPerSecond = GetTimeCodesPerSecond();
	const double SubStartTimeCode = SubSequenceLayerTimeInfo->StartTimeCode.Get(0.0);
	const double SubEndTimeCode = SubSequenceLayerTimeInfo->EndTimeCode.Get(0.0);

	FFrameRate TickResolution = MovieScene->GetTickResolution();
	FFrameNumber ModifiedStartFrame = Section->GetInclusiveStartFrame();
	FFrameNumber ModifiedEndFrame = Section->GetExclusiveEndFrame();

	// This will obviously be quantized to frame intervals for now
	double SubSectionStartTimeCode = TickResolution.AsSeconds(ModifiedStartFrame) * TimeCodesPerSecond;

	const double FixedPlayRate = Section->Parameters.TimeScale.GetType() == EMovieSceneTimeWarpType::FixedPlayRate
									 ? Section->Parameters.TimeScale.AsFixedPlayRate()
									 : 1.f;

	UE::FSdfLayerOffset NewLayerOffset;
	NewLayerOffset.Scale = FMath::IsNearlyZero(FixedPlayRate) ? 0.f : 1.f / FixedPlayRate;
	NewLayerOffset.Offset = SubSectionStartTimeCode;

	if (FMath::IsNearlyZero(NewLayerOffset.Offset))
	{
		NewLayerOffset.Offset = 0.0;
	}

	if (FMath::IsNearlyEqual(NewLayerOffset.Scale, 1.0))
	{
		NewLayerOffset.Scale = 1.0;
	}

	// Prevent twins from being rebuilt when we update the layer offsets
	TOptional<FScopedBlockNoticeListening> BlockNotices;
	if (StageActor.IsValid())
	{
		BlockNotices.Emplace(StageActor.Get());
	}

	// Check if we have a subsection due to a sublayer
	if (UE::FSdfLayerOffset* SublayerOffset = LayerTimeInfo->SublayerIdentifierToOffsets.Find(*SubSequenceLayerIdentifier))
	{
		UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(*LayerTimeInfo->Identifier);
		if (!Layer)
		{
			USD_LOG_WARNING(TEXT("Failed to update sublayer '%s'"), *LayerTimeInfo->Identifier);
			return;
		}

		const TArray<FString>& SubLayerPaths = Layer.GetSubLayerPaths();
		const TArray<UE::FSdfLayerOffset>& SubLayerOffsets = Layer.GetSubLayerOffsets();
		if (SubLayerPaths.Num() != SubLayerOffsets.Num())
		{
			return;
		}

		int32 SubLayerIndex = INDEX_NONE;
		for (int32 Index = 0; Index < SubLayerPaths.Num(); ++Index)
		{
			if (UE::FSdfLayer SubLayer = UsdUtils::FindLayerForSubLayerPath(Layer, SubLayerPaths[Index]))
			{
				if (SubLayer.GetIdentifier() == *SubSequenceLayerIdentifier)
				{
					SubLayerIndex = Index;
					break;
				}
			}
		}

		if (SubLayerIndex != INDEX_NONE)
		{
			Layer.SetSubLayerOffset(NewLayerOffset, SubLayerIndex);
			UpdateLayerTimeInfoFromLayer(*LayerTimeInfo, Layer);
		}
	}
	// Check if we have a subsection due to a reference
	else if (UE::FSdfLayerOffset* ReferencedOffset = LayerTimeInfo->ReferenceIdentifierToOffsets.Find(*SubSequenceLayerIdentifier))
	{
		// TODO: Handle updates to subsections created due to references and payloads. We'd need the referencer prim path at this
		// point, which we do not record
		return;
	}
	// Check if we have a subsection due to a payload
	else if (UE::FSdfLayerOffset* PayloadOffset = LayerTimeInfo->PayloadIdentifierToOffsets.Find(*SubSequenceLayerIdentifier))
	{
		// TODO: Handle updates to subsections created due to references and payloads. We'd need the referencer prim path at this
		// point, which we do not record
		return;
	}
}

void FUsdLevelSequenceHelperImpl::UpdateMovieSceneTimeRanges(UMovieScene& MovieScene, const FLayerTimeInfo& LayerTimeInfo, bool bUpdateViewRanges)
{
	const double FramesPerSecond = GetFramesPerSecond();

	double StartTimeCode = 0.0;
	double EndTimeCode = 0.0f;
	double TimeCodesPerSecond = 24.0;
	if (LayerTimeInfo.IsAnimated())
	{
		StartTimeCode = LayerTimeInfo.StartTimeCode.Get(0.0);
		EndTimeCode = LayerTimeInfo.EndTimeCode.Get(0.0);

		if (UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(*LayerTimeInfo.Identifier))
		{
			TimeCodesPerSecond = Layer.GetTimeCodesPerSecond();
		}
		else
		{
			TimeCodesPerSecond = GetTimeCodesPerSecond();
		}

		const FFrameRate TickResolution = MovieScene.GetTickResolution();
		const FFrameNumber StartFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, StartTimeCode / TimeCodesPerSecond);
		const FFrameNumber EndFrame = UE::USDLevelSequenceHelper::Private::RoundAsFrameNumber(TickResolution, EndTimeCode / TimeCodesPerSecond);
		TRange<FFrameNumber> TimeRange = TRange<FFrameNumber>::Inclusive(StartFrame, EndFrame);

		MovieScene.SetPlaybackRange(TimeRange);
	}

	// Do this even when "not animated" so that we always start with the playback range in view, even if the start/end are on top
	// of each other
	if (bUpdateViewRanges)
	{
		MovieScene.SetViewRange(StartTimeCode / TimeCodesPerSecond - 1.0f, 1.0f + EndTimeCode / TimeCodesPerSecond);
		MovieScene.SetWorkingRange(StartTimeCode / TimeCodesPerSecond - 1.0f, 1.0f + EndTimeCode / TimeCodesPerSecond);
	}

	// Always set these even if we're not animated because if a child layer IS animated and has a different framerate we'll get a warning
	// from the sequencer. Realistically it makes no difference because if the root layer is not animated (i.e. has 0 for start and end timecodes)
	// nothing will actually play, but this just prevents the warning
	MovieScene.SetDisplayRate(FFrameRate(FramesPerSecond, 1));
}

void FUsdLevelSequenceHelperImpl::BlockMonitoringChangesForThisTransaction()
{
	if (ITransaction* Trans = GUndo)
	{
		FTransactionContext Context = Trans->GetContext();

		// We're already blocking this one, so ignore this so that we don't increment our counter too many times
		if (BlockedTransactionGuids.Contains(Context.TransactionId))
		{
			return;
		}

		BlockedTransactionGuids.Add(Context.TransactionId);

		StopMonitoringChanges();
	}
}

void FUsdLevelSequenceHelperImpl::OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& Event)
{
	// Refresh the sequencer on the next tick, or else control rig sections will be missing their keyframes in some undo/redo scenarios.
	// The repro for this is an extension of the one on UE-191861:
	// 	- Open a stage with a SkelRoot
	// 	- Open the stage actor's LevelSequence on the Sequencer
	//    NOTE: After this point, do not select or interact with the sequencer in any way, just observe it
	//  - Right-click the SkelRoot and add the ControlRigAPI
	//  - Enable the option to Use FKControlRig
	//  - Undo
	//  - Redo
	// At this point the track will be back, but the keyframes will be missing. Some interactions with the track at this point can cause a crash too.
	// The really bizarre part is that *any transaction* after this will cause the keyframes to pop back up (selecting something, moving an unrelated
	// object on the viewport, etc.).
	//
	// This is due to this mechanism on the Sequencer code where calls to MarkAsChanged (which is a member function of tracks, sections,
	// MovieScene, etc. and is used to let the UI know it needs to refresh something) can be deferred.
	// The thing that determines where a call is deferred or not is a global, private variable (check FScopedSignedObjectModifyDefer's
	// implementation).
	//
	// I think something is causing this mechanism to be stuck deferring everything, or maybe it's some interaction with our code in some way
	// (not sure at this point). But what I do know is that Sequencer.cpp also has this class FDeferredSignedObjectChangeHandler that listens
	// to OnObjectTransacted and UndoRedo (much like we're doing right here) and has a member FScopedSignedObjectModifyDefer object (called
	// "DeferTransactionChanges") that is destroyed when the transaction is complete/canceled. Once that happens, the deferred calls are flushed
	// and the Sequencer refreshes. This is why *any transaction* causes the keyframes to be drawn back.
	//
	// Here we skip that middleman of needing an extra transaction and just flush it right now, to cause our keyframes to show up again.
	// We could also check the Object to try to limit the scope of this trick, but an alternate repro for this involves deleting the control rig track
	// and undo->redoing. In that case only the track object would transact, and you can imagine removing the entire binding, and maybe then only the
	// binding would transact, etc., which would make a robust check on the Object annoying and difficult to maintain.
	// Given that all this does is essentially refresh the Sequencer UI (and only if it had stuck deferred calls!), it's probably not the worst thing
	// in the world to check it every undo/redo anyway.
	//
	// Annoyingly we also need to do this on the next tick though, because we need to make sure this runs this after
	// FDeferredSignedObjectChangeHandler itself
	if (Event.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[](float Time)
			{
				const bool bInForceFlush = true;
				UE::MovieScene::FScopedSignedObjectModifyDefer ForceFlush{bInForceFlush};
				return false;
			}
		));
	}

	if (!MainLevelSequence || !IsMonitoringChanges() || !IsValid(Object) || !UsdStage || BlockedTransactionGuids.Contains(Event.GetTransactionId()))
	{
		return;
	}

	const ULevelSequence* LevelSequence = Object->GetTypedOuter<ULevelSequence>();
	if (!LevelSequence || (LevelSequence != MainLevelSequence && !SequencesID.Contains(LevelSequence)))
	{
		// This is not one of our managed level sequences, so ignore changes
		return;
	}

	// Never write back to the stage if we don't have authority
	if (StageActor.IsValid() && !StageActor->HasAuthorityOverStage())
	{
		return;
	}

	if (UMovieScene* MovieScene = Cast<UMovieScene>(Object))
	{
		HandleMovieSceneChange(*MovieScene);
	}
	else if (UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Object))
	{
		HandleSubSectionChange(*SubSection);
		UpdateSubSectionTimeRanges();
	}
	else if (UMovieSceneTrack* Track = Cast<UMovieSceneTrack>(Object))
	{
		const bool bIsMuteChange = Event.GetChangedProperties().Contains(TEXT("bIsEvalDisabled"));
		HandleTrackChange(*Track, bIsMuteChange);
	}
	else if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(Object))
	{
		const bool bIsMuteChange = Event.GetChangedProperties().Contains(TEXT("bIsActive"));

		if (UMovieSceneTrack* ParentTrack = Section->GetTypedOuter<UMovieSceneTrack>())
		{
			HandleTrackChange(*ParentTrack, bIsMuteChange);
		}

#if WITH_EDITOR
		if (!bIsMuteChange)
		{
			if (UMovieSceneControlRigParameterSection* CRSection = Cast<UMovieSceneControlRigParameterSection>(Section))
			{
				if (GEditor)
				{
					// We have to do this on next tick because HandleControlRigSectionChange will internally bake
					// the sequence, repeteadly updating the ControlRig hierarchy. There is no way to silence
					// FControlRigEditMode from here, and FControlRigEditMode::OnHierarchyModified ends up creating
					// a brand new scoped transaction, which asserts inside UTransBuffer::CheckState when if finds
					// out that the previous transaction wasn't fully complete (OnObjectTransacted gets called before
					// the current transaction is fully done).
					GEditor->GetTimerManager()->SetTimerForNextTick(
						[this, CRSection]()
						{
							HandleControlRigSectionChange(*CRSection);
						}
					);
				}
			}
		}
#endif	  // WITH_EDITOR
	}
}

void FUsdLevelSequenceHelperImpl::OnUsdObjectsChanged(
	const UsdUtils::FObjectChangesByPath& InfoChanges,
	const UsdUtils::FObjectChangesByPath& ResyncChanges
)
{
	AUsdStageActor* StageActorPtr = StageActor.Get();
	if (!StageActorPtr || !StageActorPtr->IsListeningToUsdNotices())
	{
		return;
	}

	if (!MainLevelSequence)
	{
		return;
	}

	UE::FUsdStage ActiveStage = StageActorPtr->GetUsdStage();
	if (!ActiveStage)
	{
		return;
	}

	TArray<UE::FSdfLayer> UsedLayers = ActiveStage.GetUsedLayers();

	FScopedBlockMonitoringChangesForTransaction BlockMonitoring{*this};

	TArray<UE::FSdfLayer> UpdatedLayers;

	auto IterateChanges = [this, UsedLayers](const UsdUtils::FObjectChangesByPath& Changes) -> bool
	{
		for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& InfoChange : Changes)
		{
			const FString& PrimPath = InfoChange.Key;
			if (PrimPath == TEXT("/"))
			{
				// Update info for all layers because OnUsdObjectsChanged doesn't specify which actual layer that changed
				// TODO: Change this to also use the LayersDidChange notice instead (see comments on UE-222371).
				for (const UE::FSdfLayer& UsedLayer : UsedLayers)
				{
					FString Identifier = UsedLayer.GetIdentifier();

					ULevelSequence* Sequence = LevelSequencesByIdentifier.FindRef(Identifier);
					UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;

					FLayerTimeInfo* LayerTimeInfo = LayerTimeInfosByLayerIdentifier.Find(Identifier);

					if (MovieScene && LayerTimeInfo)
					{
						UpdateLayerTimeInfoFromLayer(*LayerTimeInfo, UsedLayer);

						// We should only change this when first creating the LevelSequence, not after every edit
						const bool bUpdateViewRanges = false;
						UpdateMovieSceneTimeRanges(*MovieScene, *LayerTimeInfo, bUpdateViewRanges);
					}
				}
				return true;
			}
		}

		return false;
	};
	bool bUpdated = IterateChanges(InfoChanges);
	if (!bUpdated)
	{
		bUpdated = IterateChanges(ResyncChanges);
	}

	if (bUpdated)
	{
		UpdateSubSectionTimeRanges();
		CreateTimeTrack(FindOrAddLayerTimeInfo(UsdStage.GetRootLayer()));
	}
}

void FUsdLevelSequenceHelperImpl::HandleTransactionStateChanged(
	const FTransactionContext& InTransactionContext,
	const ETransactionStateEventType InTransactionState
)
{
	if (InTransactionState == ETransactionStateEventType::TransactionFinalized
		&& BlockedTransactionGuids.Contains(InTransactionContext.TransactionId))
	{
		StartMonitoringChanges();
	}
}

double FUsdLevelSequenceHelperImpl::GetFramesPerSecond() const
{
	if (!UsdStage)
	{
		return DefaultFramerate;
	}

	const double StageFramesPerSecond = UsdStage.GetFramesPerSecond();
	return FMath::IsNearlyZero(StageFramesPerSecond) ? DefaultFramerate : StageFramesPerSecond;
}

double FUsdLevelSequenceHelperImpl::GetTimeCodesPerSecond() const
{
	if (!UsdStage)
	{
		return DefaultFramerate;
	}

	const double StageTimeCodesPerSecond = UsdStage.GetTimeCodesPerSecond();
	return FMath::IsNearlyZero(StageTimeCodesPerSecond) ? DefaultFramerate : StageTimeCodesPerSecond;
}

FGuid FUsdLevelSequenceHelperImpl::GetOrCreateComponentBinding(
	const UUsdPrimTwin& PrimTwin,
	USceneComponent& ComponentToBind,
	ULevelSequence& Sequence
)
{
	UMovieScene* MovieScene = Sequence.GetMovieScene();
	if (!MovieScene)
	{
		return {};
	}

	FPrimTwinBindings& Bindings = PrimTwinToBindings.FindOrAdd(&PrimTwin);

	// TODO: Better support having multiple bindings for the same same on different sequences.
	// We used to ensure "Bindings.Sequence == &Sequence" here, but we shouldn't do that as it's very simple and valid to
	// construct a stage where a prim's strongest opinion for two animated attributes comes from different layers, and it
	// would have triggered this ensure

	Bindings.Sequence = &Sequence;

	if (FGuid* ExistingGuid = Bindings.ObjectClassToBindingGuid.Find(ComponentToBind.GetClass()))
	{
		return *ExistingGuid;
	}

	FGuid ComponentBinding;
	FGuid ActorBinding;
	UObject* ComponentContext = ComponentToBind.GetWorld();

	FString PrimName = FPaths::GetBaseFilename(PrimTwin.PrimPath);

	// Make sure we always bind the parent actor too
	if (AActor* Actor = ComponentToBind.GetOwner())
	{
		TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(
			Actor,
			&Sequence
		);

		ActorBinding = Sequence.FindBindingFromObject(Actor, SharedPlaybackState);
		if (!ActorBinding.IsValid())
		{
			// We use the label here because that will always be named after the prim that caused the actor
			// to be generated. If we just used our own PrimName in here we may run into situations where a child Camera prim
			// of a decomposed camera ends up naming the actor binding after itself, even though the parent Xform prim, and the
			// actor on the level, maybe named something else
			ActorBinding = MovieScene->AddPossessable(
#if WITH_EDITOR
				Actor->GetActorLabel(),
#else
				Actor->GetName(),
#endif	  // WITH_EDITOR
				Actor->GetClass()
			);
			Sequence.BindPossessableObject(ActorBinding, *Actor, Actor->GetWorld());
		}

		ComponentContext = Actor;
	}

	ComponentBinding = MovieScene->AddPossessable(PrimName, ComponentToBind.GetClass());

	if (ActorBinding.IsValid() && ComponentBinding.IsValid())
	{
		if (FMovieScenePossessable* ComponentPossessable = MovieScene->FindPossessable(ComponentBinding))
		{
			ComponentPossessable->SetParent(ActorBinding, MovieScene);
		}
	}

	// Bind component
	Sequence.BindPossessableObject(ComponentBinding, ComponentToBind, ComponentContext);
	Bindings.ObjectClassToBindingGuid.Emplace(ComponentToBind.GetClass(), ComponentBinding);
	return ComponentBinding;
}

void FUsdLevelSequenceHelperImpl::HandleMovieSceneChange(UMovieScene& MovieScene)
{
	using namespace UE::USDLevelSequenceHelper::Private;

	// It's possible to get this called when the actor and it's level sequences are being all destroyed in one go.
	// We need the FScopedBlockNotices in this function, but if our StageActor is already being destroyed, we can't reliably
	// use its listener, and so then we can't do anything. We likely don't want to write back to the stage at this point anyway.
	AUsdStageActor* StageActorPtr = StageActor.Get();
	if (!MainLevelSequence || !UsdStage || !StageActorPtr || StageActorPtr->IsActorBeingDestroyed())
	{
		return;
	}

	ULevelSequence* Sequence = MovieScene.GetTypedOuter<ULevelSequence>();
	if (!Sequence)
	{
		return;
	}

	const FString LayerIdentifier = LayerIdentifierByLevelSequenceName.FindRef(Sequence->GetName());
	FLayerTimeInfo* LayerTimeInfo = LayerTimeInfosByLayerIdentifier.Find(LayerIdentifier);
	if (!LayerTimeInfo)
	{
		return;
	}

	UE::FSdfLayer Layer = UE::FSdfLayer::FindOrOpen(*LayerTimeInfo->Identifier);
	if (!Layer)
	{
		return;
	}

	const double StageTimeCodesPerSecond = GetTimeCodesPerSecond();
	const TRange<FFrameNumber> PlaybackRange = MovieScene.GetPlaybackRange();
	const FFrameRate DisplayRate = MovieScene.GetDisplayRate();
	const FFrameRate LayerTimeCodesPerSecond(Layer.GetTimeCodesPerSecond(), 1);
	const FFrameTime StartTime = FFrameRate::TransformTime(
		UE::MovieScene::DiscreteInclusiveLower(PlaybackRange).Value,
		MovieScene.GetTickResolution(),
		LayerTimeCodesPerSecond
	);
	const FFrameTime EndTime = FFrameRate::TransformTime(
		UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange).Value,
		MovieScene.GetTickResolution(),
		LayerTimeCodesPerSecond
	);

	FScopedBlockNoticeListening BlockNotices(StageActor.Get());
	UE::FSdfChangeBlock ChangeBlock;
	if (!FMath::IsNearlyEqual(DisplayRate.AsDecimal(), GetFramesPerSecond()))
	{
		UsdStage.SetFramesPerSecond(DisplayRate.AsDecimal());

		// For whatever reason setting a stage FramesPerSecond also automatically sets its TimeCodesPerSecond to the same value, so we need to undo
		// it. This because all the sequencer does is change display rate, which is the analogue to USD's frames per second (i.e. we are only changing
		// how many frames we'll display between any two timecodes, not how many timecodes we'll display per second)
		UsdStage.SetTimeCodesPerSecond(StageTimeCodesPerSecond);

		// Propagate to all movie scenes, as USD only uses the stage FramesPerSecond so the sequences should have a unified DisplayRate to reflect
		// that
		for (auto& SequenceByIdentifier : LevelSequencesByIdentifier)
		{
			if (ULevelSequence* OtherSequence = SequenceByIdentifier.Value)
			{
				if (UMovieScene* OtherMovieScene = OtherSequence->GetMovieScene())
				{
					OtherMovieScene->SetDisplayRate(DisplayRate);
				}
			}
		}
	}

	Layer.SetStartTimeCode(StartTime.RoundToFrame().Value);
	Layer.SetEndTimeCode(EndTime.RoundToFrame().Value);

	UpdateLayerTimeInfoFromLayer(*LayerTimeInfo, Layer);

	if (Sequence == MainLevelSequence)
	{
		CreateTimeTrack(FindOrAddLayerTimeInfo(UsdStage.GetRootLayer()));
	}

	auto RemoveTimeSamplesForAttr = [](const UE::FUsdAttribute& Attr)
	{
		if (!Attr || Attr.GetNumTimeSamples() == 0)
		{
			return;
		}

		if (UE::FSdfLayer AttrLayer = UsdUtils::FindLayerForAttribute(Attr, 0.0))
		{
			UE::FSdfPath AttrPath = Attr.GetPath();
			for (double TimeSample : AttrLayer.ListTimeSamplesForPath(AttrPath))
			{
				AttrLayer.EraseTimeSample(AttrPath, TimeSample);
			}
		}
	};

	auto RemoveTimeSamplesForPropertyIfNeeded =
		[&MovieScene, &RemoveTimeSamplesForAttr](const UE::FUsdPrim& Prim, const FGuid& Guid, const FName& PropertyPath)
	{
		if (!UE::USDLevelSequenceHelper::Private::FindTrackTypeOrDerived<UMovieScenePropertyTrack>(&MovieScene, Guid, PropertyPath))
		{
			for (UE::FUsdAttribute& Attr : UsdUtils::GetAttributesForProperty(Prim, PropertyPath))
			{
				RemoveTimeSamplesForAttr(Attr);
			}
		}
	};

	// Check if we deleted things
	for (TMap<TWeakObjectPtr<const UUsdPrimTwin>, FPrimTwinBindings>::TIterator PrimTwinIt = PrimTwinToBindings.CreateIterator(); PrimTwinIt;
		 ++PrimTwinIt)
	{
		const UUsdPrimTwin* UsdPrimTwin = PrimTwinIt->Key.Get();
		FPrimTwinBindings& Bindings = PrimTwinIt->Value;

		if (Bindings.Sequence != Sequence)
		{
			continue;
		}

		for (TMap<TWeakObjectPtr<const UClass>, FGuid>::TIterator BindingIt = Bindings.ObjectClassToBindingGuid.CreateIterator(); BindingIt;
			 ++BindingIt)
		{
			const FGuid& Guid = BindingIt->Value;

			// Deleted the possessable
			if (!MovieScene.FindPossessable(Guid))
			{
				BindingIt.RemoveCurrent();
			}

			// Check if we have an animated attribute and no track for it --> We may have deleted the track, so clear that attribute
			// We could keep track of these when adding in some kind of map, but while slower this is likely more robust due to the need to support
			// undo/redo
			if (UsdPrimTwin)
			{
				USceneComponent* BoundComponent = UsdPrimTwin->GetSceneComponent();
				if (!BoundComponent)
				{
					continue;
				}

				const bool bIsCamera = BoundComponent->IsA<UCineCameraComponent>();
				const bool bIsLight = BoundComponent->IsA<ULightComponentBase>();
				const bool bIsSkeletal = BoundComponent->IsA<USkeletalMeshComponent>();

				if (UE::FUsdPrim UsdPrim = UsdStage.GetPrimAtPath(UE::FSdfPath(*UsdPrimTwin->PrimPath)))
				{
					RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::TransformPropertyName);

					// Handle visibility explicitly here because we may have a visibility track on the actor or on the component
					if (!FindTrackTypeOrDerived<UMovieScenePropertyTrack>(&MovieScene, Guid, UnrealIdentifiers::HiddenInGamePropertyName)
						&& !FindTrackTypeOrDerived<UMovieScenePropertyTrack>(&MovieScene, Guid, UnrealIdentifiers::HiddenPropertyName))
					{
						for (UE::FUsdAttribute& Attr : UsdUtils::GetAttributesForProperty(UsdPrim, UnrealIdentifiers::HiddenInGamePropertyName))
						{
							RemoveTimeSamplesForAttr(Attr);
						}
					}

					if (bIsCamera)
					{
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::CurrentFocalLengthPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::ManualFocusDistancePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::CurrentAperturePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SensorWidthPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SensorHeightPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SensorHorizontalOffsetPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SensorVerticalOffsetPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::ExposureCompensationPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::ProjectionModePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::OrthoFarClipPlanePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::OrthoNearClipPlanePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::CustomNearClipppingPlanePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::DepthOfFieldFstopPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::CameraISOPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::CameraShutterSpeedPropertyName);
					}
					else if (bIsLight)
					{
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::IntensityPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::LightColorPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::UseTemperaturePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::TemperaturePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SourceRadiusPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SourceWidthPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::SourceHeightPropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::OuterConeAnglePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::InnerConeAnglePropertyName);
						RemoveTimeSamplesForPropertyIfNeeded(UsdPrim, Guid, UnrealIdentifiers::LightSourceAnglePropertyName);
					}
					else if (bIsSkeletal)
					{
						if (!MovieScene.FindTrack(UMovieSceneSkeletalAnimationTrack::StaticClass(), Guid))
						{
							UE::FUsdPrim SkelRootPrim = UsdUtils::GetClosestParentSkelRoot(UsdPrim);
							UE::FUsdPrim SkelAnimationPrim = UsdUtils::FindAnimationSource(SkelRootPrim, UsdPrim);
							if (SkelAnimationPrim)
							{
								if (UE::FSdfLayer SkelAnimationLayer = UsdUtils::FindLayerForPrim(SkelAnimationPrim))
								{
									RemoveTimeSamplesForAttr(SkelAnimationPrim.GetAttribute(TEXT("blendShapeWeights")));
									RemoveTimeSamplesForAttr(SkelAnimationPrim.GetAttribute(TEXT("rotations")));
									RemoveTimeSamplesForAttr(SkelAnimationPrim.GetAttribute(TEXT("translations")));
									RemoveTimeSamplesForAttr(SkelAnimationPrim.GetAttribute(TEXT("scales")));
								}
							}
						}
					}
				}
			}
		}
	}

	// We may have changed things like playback ranges, so refresh the prim properties panel if relevant
	StageActorPtr->OnPrimChanged.Broadcast(UE::FSdfPath::AbsoluteRootPath().GetString(), false);

	const bool bShowToast = true;
	UpdateSubSectionTimeRanges(bShowToast);
}

void FUsdLevelSequenceHelperImpl::HandleSubSectionChange(UMovieSceneSubSection& Section)
{
	UMovieSceneSequence* ParentSequence = Section.GetTypedOuter<UMovieSceneSequence>();
	if (!ParentSequence)
	{
		return;
	}

	UpdateUsdLayerOffsetFromSection(ParentSequence, &Section);
}

void FUsdLevelSequenceHelperImpl::HandleControlRigSectionChange(UMovieSceneControlRigParameterSection& Section)
{
#if WITH_EDITOR
	AUsdStageActor* StageActorValue = StageActor.Get();
	if (!StageActorValue)
	{
		return;
	}

	UWorld* World = StageActorValue->GetWorld();
	if (!World)
	{
		return;
	}

	ULevelSequence* LevelSequence = Section.GetTypedOuter<ULevelSequence>();
	if (!LevelSequence)
	{
		return;
	}

	UMovieScene* MovieScene = LevelSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UMovieSceneTrack* ParentTrack = Section.GetTypedOuter<UMovieSceneTrack>();
	if (!ParentTrack)
	{
		return;
	}

	FGuid PossessableGuid;
	const bool bFound = MovieScene->FindTrackBinding(*ParentTrack, PossessableGuid);
	if (!bFound)
	{
		return;
	}

	FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid);
	if (!Possessable)
	{
		return;
	}

	USkeletalMeshComponent* BoundComponent = Cast<USkeletalMeshComponent>(
		UE::USDLevelSequenceHelper::Private::LocateBoundObject(*LevelSequence, *Possessable)
	);
	if (!BoundComponent)
	{
		return;
	}
	ensure(BoundComponent->Mobility != EComponentMobility::Static);

	USkeleton* Skeleton = BoundComponent->GetSkeletalMeshAsset() ? BoundComponent->GetSkeletalMeshAsset()->GetSkeleton() : nullptr;
	if (!Skeleton)
	{
		return;
	}

	UUsdPrimTwin* PrimTwin = StageActorValue->RootUsdTwin->Find(BoundComponent);
	if (!PrimTwin)
	{
		return;
	}

	UE::FUsdPrim SkeletonPrim = UsdStage.GetPrimAtPath(UE::FSdfPath(*PrimTwin->PrimPath));
	if (!SkeletonPrim)
	{
		return;
	}

	UE::FUsdPrim SkelRootPrim = UsdUtils::GetClosestParentSkelRoot(SkeletonPrim);
	if (!SkelRootPrim)
	{
		return;
	}

	// We'll place the skeletal animation track wherever the SkelAnimation prim is defined (not necessarily the
	// same layer as the skel root)
	UE::FUsdPrim SkelAnimationPrim = UsdUtils::FindAnimationSource(SkelRootPrim, SkeletonPrim);
	if (!SkelAnimationPrim)
	{
		return;
	}

	// Each sequence corresponds to a specific USD layer. If we're editing something in a sequence, then we must make
	// that layer the edit target too
	UE::FSdfLayer EditTargetLayer = FindEditTargetForSubsequence(LevelSequence);
	if (!EditTargetLayer)
	{
		return;
	}
	UE::FUsdEditContext EditContext{UsdStage, EditTargetLayer};

	TSharedPtr<ISequencer> PinnedSequencer = UE::USDLevelSequenceHelper::Private::GetOpenedSequencerForLevelSequence(MainLevelSequence);

	// Fetch a sequence player we can use. We'll almost always have the sequencer opened here (we are responding to a transaction
	// where the section was changed after all), but its possible to have a fallback too
	IMovieScenePlayer* Player = nullptr;
	ULevelSequencePlayer* LevelPlayer = nullptr;
	UMovieScene* MovieSceneForSpawnableRestore = nullptr;
	{
		if (PinnedSequencer)
		{
			Player = PinnedSequencer.Get();
		}
		else
		{
			ALevelSequenceActor* OutActor = nullptr;
			FMovieSceneSequencePlaybackSettings Settings;
			Player = LevelPlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, LevelSequence, Settings, OutActor);
			MovieSceneForSpawnableRestore = MovieScene;
		}

		if (!Player)
		{
			return;
		}
	}

	// We obviously don't want to respond to the fact that the stage will be modified since we're the
	// ones actually modifying it already
	FScopedBlockNoticeListening BlockNotices(StageActorValue);

	// Prepare for baking
	{
		if (PinnedSequencer)
		{
			PinnedSequencer->EnterSilentMode();
		}

		FAllSpawnableRestoreState SpawnableRestoreState(Player->GetSharedPlaybackState().ToSharedPtr(), MovieSceneForSpawnableRestore);
		if (LevelPlayer && SpawnableRestoreState.bWasChanged)
		{
			// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
			// Note that we never actually generate spawnables on our LevelSequence, but its a common pattern to
			// do this and the user may have added them manually
			FFrameTime StartTime = FFrameRate::TransformTime(
				UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value,
				MovieScene->GetTickResolution(),
				MovieScene->GetDisplayRate()
			);
			LevelPlayer->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTime, EUpdatePositionMethod::Play));
		}
	}

	const FMovieSceneSequenceTransform& SequenceTransform = GetSubsequenceTransform(LevelSequence);

	// Actually bake inside the UsdUtilities module as we need to manipulate USD arrays a lot
	const UsdUtils::FBlendShapeMap& BlendShapeMap = StageActorValue->GetBlendShapeMap();
	bool bBaked = UnrealToUsd::ConvertControlRigSection(
		&Section,
		SequenceTransform.Inverse(),
		MovieScene,
		Player,
		Skeleton->GetReferenceSkeleton(),
		SkelRootPrim,
		SkelAnimationPrim,
		&BlendShapeMap
	);

	// Cleanup after baking
	{
		if (LevelPlayer)
		{
			LevelPlayer->Stop();
		}

		if (PinnedSequencer)
		{
			PinnedSequencer->ExitSilentMode();
			PinnedSequencer->RequestEvaluate();
		}
	}

	if (bBaked)
	{
		// After we bake, both the sequencer and the USD stage have our updated tracks, but we still have the old
		// AnimSequence asset on the component. If we closed the Sequencer and just animated via the Time attribute,
		// we would see the old animation.
		// This event is mostly used to have the stage actor quickly regenerate the assets and components for the
		// skel root. Sadly we do need to regenerate the skeletal mesh too, since we may need to affect blend shapes
		// for the correct bake. The user can disable this behavior (e.g. for costly skeletal meshes) by setting
		// USD.RegenerateSkeletalAssetsOnControlRigBake to false.
		GetOnSkelAnimationBaked().Broadcast(PrimTwin->PrimPath);
	}
#endif	  // WITH_EDITOR
}

void FUsdLevelSequenceHelperImpl::HandleTrackChange(UMovieSceneTrack& Track, bool bIsMuteChange)
{
	if (!StageActor.IsValid())
	{
		return;
	}

	ULevelSequence* Sequence = Track.GetTypedOuter<ULevelSequence>();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	FGuid PossessableGuid;
	bool bFound = MovieScene->FindTrackBinding(Track, PossessableGuid);
	if (!bFound)
	{
		return;
	}

	FMovieScenePossessable* Possessable = MovieScene->FindPossessable(PossessableGuid);
	if (!Possessable)
	{
		return;
	}

	UObject* BoundObject = UE::USDLevelSequenceHelper::Private::LocateBoundObject(*Sequence, *Possessable);
	if (!BoundObject)
	{
		return;
	}

	// The only stage actor property we allow binding on the transient level sequence is 'Time'. Anything else we
	// need to force-unbind as not only will it be lost when reloading the stage anyway, but it can even lead to
	// crashes (e.g. UE-215067)
	UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(&Track);
	const FName PropertyPath = PropertyTrack ? PropertyTrack->GetPropertyPath() : NAME_None;
	if (BoundObject == StageActor.Get())
	{
		if (PropertyPath == GET_MEMBER_NAME_CHECKED(AUsdStageActor, Time))
		{
			// Discard any edits to the Time track
			CreateTimeTrack(FindOrAddLayerTimeInfo(UsdStage.GetRootLayer()));
		}
		else
		{
#if WITH_EDITOR
			UE::USDLevelSequenceHelper::Private::ShowStageActorPropertyTrackWarning(PropertyPath);
#endif	  // WITH_EDITOR
			MovieScene->RemoveTrack(*PropertyTrack);
		}

		return;
	}

	// Our tracked bindings are always directly to components
	USceneComponent* BoundSceneComponent = Cast<USceneComponent>(BoundObject);
	if (!BoundSceneComponent)
	{
		// The sequencer allows binding actor Transform directly, which gets deferred to its root component
		// transform. Let's also allow that here
		if (AActor* BoundActor = Cast<AActor>(BoundObject))
		{
			BoundSceneComponent = BoundActor->GetRootComponent();
		}
	}
	if (!BoundSceneComponent)
	{
		return;
	}

	UUsdPrimTwin* PrimTwin = StageActor->RootUsdTwin->Find(BoundSceneComponent);

	// If we exported/created this Camera prim ourselves, we'll have a decomposed parent Xform and a child Camera prim (to mirror
	// the ACineCameraActor structure), and we should have created prim twins for both when opening this stage.
	// If this USD layer is not authored by us, it may just be a standalone Camera prim: In this scenario the created PrimTwin
	// will be pointing at the parent USceneComponent of the spawned ACineCameraActor, and we wouldn't find anything when searching
	// for the camera component directly, so try again
	if (!PrimTwin && BoundSceneComponent->IsA<UCineCameraComponent>())
	{
		if (PropertyTrack)
		{
			// In the scenario where we're trying to make non-decomposed Camera prims work, we only ever want to write out
			// actual camera properties from the CameraComponent to the Camera prim. We won't write its USceneComponent
			// properties, as we will use the ones from the ACineCameraActor's parent USceneComponent instead
			if (UE::USDLevelSequenceHelper::Private::TrackedCameraProperties.Contains(PropertyPath))
			{
				PrimTwin = StageActor->RootUsdTwin->Find(BoundSceneComponent->GetAttachParent());
			}
#if WITH_EDITOR
			else if (PropertyPath == UnrealIdentifiers::TransformPropertyName)
			{
				// Let the user know that we currently don't support transform tracks directly on camera components
				UE::USDLevelSequenceHelper::Private::ShowTransformTrackOnCameraComponentWarning(BoundSceneComponent);
			}
#endif	  // WITH_EDITOR
		}
	}

	// Each sequence corresponds to a specific USD layer. If we're editing something in a sequence, then we must make
	// that layer the edit target too
	UE::FSdfLayer EditTargetLayer = FindEditTargetForSubsequence(Sequence);
	if (!EditTargetLayer)
	{
		return;
	}
	UE::FUsdEditContext EditContext{UsdStage, EditTargetLayer};

	if (PrimTwin)
	{
		FScopedBlockNoticeListening BlockNotices(StageActor.Get());
		UE::FUsdPrim UsdPrim = UsdStage.GetPrimAtPath(UE::FSdfPath(*PrimTwin->PrimPath));

		FPrimTwinBindings& Bindings = PrimTwinToBindings.FindOrAdd(PrimTwin);
		ensure(Bindings.Sequence == nullptr || Bindings.Sequence == Sequence);
		Bindings.Sequence = Sequence;

		// Make sure we track this binding
		const UClass* ComponentClass = BoundObject->GetClass();
		FGuid* FoundExistingGuid = Bindings.ObjectClassToBindingGuid.Find(ComponentClass);
		ensure(!FoundExistingGuid || *FoundExistingGuid == PossessableGuid);
		Bindings.ObjectClassToBindingGuid.Emplace(ComponentClass, PossessableGuid);

		// We can't do anything if our prim is an instance proxy
		if (UsdUtils::NotifyIfInstanceProxy(UsdPrim))
		{
			return;
		}

		if (bIsMuteChange)
		{
			if (PropertyTrack)
			{
				TArray<UE::FUsdAttribute> Attrs = UsdUtils::GetAttributesForProperty(UsdPrim, PropertyPath);
				if (Attrs.Num() > 0)
				{
					// Only mute/unmute the first (i.e. main) attribute: If we mute the intensity track we don't want to also mute the
					// rect width track if it has one
					UE::FUsdAttribute& Attr = Attrs[0];

					bool bAllSectionsMuted = true;
					for (const UMovieSceneSection* Section : Track.GetAllSections())	// There's no const version of "FindSection"
					{
						bAllSectionsMuted &= !Section->IsActive();
					}

					if (Track.IsEvalDisabled() || bAllSectionsMuted)
					{
						UsdUtils::MuteAttribute(Attr, UsdStage);
					}
					else
					{
						UsdUtils::UnmuteAttribute(Attr, UsdStage);
					}

					// The attribute may have an effect on the stage, so animate it right away
					StageActor->OnTimeChanged.Broadcast();
				}
			}
			else if (const UMovieSceneSkeletalAnimationTrack* SkeletalTrack = Cast<const UMovieSceneSkeletalAnimationTrack>(&Track))
			{
				bool bAllSectionsMuted = true;
				for (const UMovieSceneSection* Section : SkeletalTrack->GetAllSections())	 // There's no const version of "FindSection"
				{
					bAllSectionsMuted &= !Section->IsActive();
				}

				UE::FUsdPrim SkelRootPrim = UsdUtils::GetClosestParentSkelRoot(UsdPrim);
				if (UE::FUsdPrim SkelAnimationPrim = UsdUtils::FindAnimationSource(SkelRootPrim, UsdPrim))
				{
					UE::FUsdAttribute TranslationsAttr = SkelAnimationPrim.GetAttribute(TEXT("translations"));
					UE::FUsdAttribute RotationsAttr = SkelAnimationPrim.GetAttribute(TEXT("rotations"));
					UE::FUsdAttribute ScalesAttr = SkelAnimationPrim.GetAttribute(TEXT("scales"));
					UE::FUsdAttribute BlendShapeWeightsAttr = SkelAnimationPrim.GetAttribute(TEXT("blendShapeWeights"));

					if (Track.IsEvalDisabled() || bAllSectionsMuted)
					{
						UsdUtils::MuteAttribute(TranslationsAttr, UsdStage);
						UsdUtils::MuteAttribute(RotationsAttr, UsdStage);
						UsdUtils::MuteAttribute(ScalesAttr, UsdStage);
						UsdUtils::MuteAttribute(BlendShapeWeightsAttr, UsdStage);
					}
					else
					{
						UsdUtils::UnmuteAttribute(TranslationsAttr, UsdStage);
						UsdUtils::UnmuteAttribute(RotationsAttr, UsdStage);
						UsdUtils::UnmuteAttribute(ScalesAttr, UsdStage);
						UsdUtils::UnmuteAttribute(BlendShapeWeightsAttr, UsdStage);
					}

					// The attribute may have an effect on the stage, so animate it right away
					StageActor->OnTimeChanged.Broadcast();
				}
			}
		}
		else
		{
			// Right now we don't write out changes to SkeletalAnimation tracks, and only property tracks... the UAnimSequence
			// asset can't be modified all that much in UE anyway. Later on we may want to enable writing it out anyway though,
			// and pick up on changes to the section offset or play rate and bake out the UAnimSequence again
			if (PropertyTrack)
			{
#if WITH_EDITOR
				UE::USDLevelSequenceHelper::Private::ShowVisibilityWarningIfNeeded(PropertyTrack, UsdPrim);
#endif

				FMovieSceneSequenceTransform SequenceTransform = GetSubsequenceTransform(Sequence);

				TSet<FName> PropertyPathsToRefresh;
				UnrealToUsd::FPropertyTrackWriter
					Writer = UnrealToUsd::CreatePropertyTrackWriter(*BoundSceneComponent, *PropertyTrack, UsdPrim, PropertyPathsToRefresh);

				if (const UMovieSceneFloatTrack* FloatTrack = Cast<const UMovieSceneFloatTrack>(&Track))
				{
					// We won't need a SequenceTransform in this case because the FloatWriter will be ready to receive and write
					// keyframes local to its own sequence/layer
					if (const UHeterogeneousVolumeComponent* VolumeComponent = Cast<const UHeterogeneousVolumeComponent>(BoundSceneComponent))
					{
						if (PropertyTrack->GetPropertyName() == GET_MEMBER_NAME_CHECKED(UHeterogeneousVolumeComponent, Frame))
						{
							SequenceTransform = {};
						}
					}

					UnrealToUsd::ConvertFloatTrack(*FloatTrack, SequenceTransform, Writer.FloatWriter, UsdPrim);
				}
				else if (const UMovieSceneBoolTrack* BoolTrack = Cast<const UMovieSceneBoolTrack>(&Track))
				{
					UnrealToUsd::ConvertBoolTrack(*BoolTrack, SequenceTransform, Writer.BoolWriter, UsdPrim);
				}
				else if (const UMovieSceneVisibilityTrack* VisTrack = Cast<const UMovieSceneVisibilityTrack>(&Track))
				{
					UnrealToUsd::ConvertBoolTrack(*VisTrack, SequenceTransform, Writer.BoolWriter, UsdPrim);
				}
				else if (const UMovieSceneColorTrack* ColorTrack = Cast<const UMovieSceneColorTrack>(&Track))
				{
					UnrealToUsd::ConvertColorTrack(*ColorTrack, SequenceTransform, Writer.ColorWriter, UsdPrim);
				}
				else if (const UMovieScene3DTransformTrack* TransformTrack = Cast<const UMovieScene3DTransformTrack>(&Track))
				{
					UnrealToUsd::Convert3DTransformTrack(*TransformTrack, SequenceTransform, Writer.TransformWriter, UsdPrim);

					// If we're a Cylinder, Cube, etc. clear the animation of the primitive attributes that can affect the
					// primitive transform ("height", "radius", etc.) as we'll be writing the full combined primitive+local
					// transform directly to the Xform animation instead
					const bool bDefaultValues = false;
					const bool bTimeSampleValues = true;
					UsdUtils::AuthorIdentityTransformGprimAttributes(UsdPrim, bDefaultValues, bTimeSampleValues);
				}

				// For the bounds tracks alone we have two separate tracks we must read from at once, and write to a single
				// USD attribute. We'll have one of those already (the Track itself), but we need to find the other, if any.
				// This could be somewhat cleaned up if we had FBox tracks in the Sequencer, but it should work just fine for now
				else if (Writer.TwoVectorWriter)
				{
					const UMovieSceneDoubleVectorTrack* MinTrack = nullptr;
					const UMovieSceneDoubleVectorTrack* MaxTrack = nullptr;
					if (Track.GetTrackName() == GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin))
					{
						MinTrack = Cast<UMovieSceneDoubleVectorTrack>(&Track);
						MaxTrack = Cast<UMovieSceneDoubleVectorTrack>(MovieScene->FindTrack(
							UMovieSceneDoubleVectorTrack::StaticClass(),
							PossessableGuid,
							GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMax)
						));
					}
					else
					{
						MaxTrack = Cast<UMovieSceneDoubleVectorTrack>(&Track);
						MinTrack = Cast<UMovieSceneDoubleVectorTrack>(MovieScene->FindTrack(
							UMovieSceneDoubleVectorTrack::StaticClass(),
							PossessableGuid,
							GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin)
						));
					}

					// Realistically we'll have both of them, but we *need* at least one
					if (ensure(MinTrack || MaxTrack))
					{
						UnrealToUsd::ConvertBoundsVectorTracks(MinTrack, MaxTrack, SequenceTransform, Writer.TwoVectorWriter, UsdPrim);
					}
				}

				// Refresh tracks that needed to be updated in USD (e.g. we wrote out a new keyframe to a RectLight's width -> that
				// should become a new keyframe on our intensity track, because we use the RectLight's width for calculating intensity in UE).
				if (PropertyPathsToRefresh.Num() > 0)
				{
					// For now only our light tracks can request a refresh like this, so we don't even need to check what the refresh
					// is about: Just resync the light tracks
					AddLightTracks(*PrimTwin, UsdPrim, PropertyPathsToRefresh);
					RefreshSequencer();
				}
			}
			else if (const UMovieSceneAudioTrack* AudioTrack = Cast<const UMovieSceneAudioTrack>(&Track))
			{
				const TArray<UMovieSceneSection*>& Sections = AudioTrack->GetAudioSections();
				if (Sections.Num() > 1)
				{
					USD_LOG_WARNING(
						TEXT(
							"The audio track '%s' has %d sections, but only the first audio section of an audio track can be written out to USD for now"
						),
						*AudioTrack->GetPathName(),
						Sections.Num()
					);
				}

				if (Sections.Num() > 0)
				{
					if (UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Sections[0]))
					{
						FMovieSceneSequenceTransform Identity;
						UnrealToUsd::ConvertAudioSection(*AudioSection, Identity, UsdPrim);
					}
				}
			}
		}

		// Notify the USD Stage Editor to refresh this prim the next frame
		if (AUsdStageActor* StageActorPtr = StageActor.Get())
		{
			StageActorPtr->OnPrimChanged.Broadcast(PrimTwin->PrimPath, false);
		}
	}
}

FUsdLevelSequenceHelperImpl::FLayerTimeInfo& FUsdLevelSequenceHelperImpl::FindOrAddLayerTimeInfo(const UE::FSdfLayer& Layer)
{
	if (FLayerTimeInfo* LayerTimeInfo = FindLayerTimeInfo(Layer))
	{
		return *LayerTimeInfo;
	}

	FLayerTimeInfo LayerTimeInfo;
	UpdateLayerTimeInfoFromLayer(LayerTimeInfo, Layer);

	return LayerTimeInfosByLayerIdentifier.Add(Layer.GetIdentifier(), LayerTimeInfo);
}

FUsdLevelSequenceHelperImpl::FLayerTimeInfo* FUsdLevelSequenceHelperImpl::FindLayerTimeInfo(const UE::FSdfLayer& Layer)
{
	const FString Identifier = Layer.GetIdentifier();
	return LayerTimeInfosByLayerIdentifier.Find(Identifier);
}

void FUsdLevelSequenceHelperImpl::UpdateLayerTimeInfoFromLayer(FLayerTimeInfo& LayerTimeInfo, const UE::FSdfLayer& Layer)
{
	if (!Layer)
	{
		return;
	}

	LayerTimeInfo.Identifier = Layer.GetIdentifier();
	LayerTimeInfo.FilePath = Layer.GetRealPath();
	LayerTimeInfo.StartTimeCode = Layer.HasStartTimeCode() ? Layer.GetStartTimeCode() : TOptional<double>();
	LayerTimeInfo.EndTimeCode = Layer.HasEndTimeCode() ? Layer.GetEndTimeCode() : TOptional<double>();

	if (LayerTimeInfo.StartTimeCode.IsSet() && LayerTimeInfo.EndTimeCode.IsSet()
		&& LayerTimeInfo.EndTimeCode.GetValue() < LayerTimeInfo.StartTimeCode.GetValue())
	{
		USD_LOG_WARNING(
			TEXT("Sublayer '%s' has end time code (%f) before start time code (%f)! These values will be automatically swapped"),
			*Layer.GetIdentifier(),
			LayerTimeInfo.EndTimeCode.GetValue(),
			LayerTimeInfo.StartTimeCode.GetValue()
		);

		TOptional<double> Temp = LayerTimeInfo.StartTimeCode;
		LayerTimeInfo.StartTimeCode = LayerTimeInfo.EndTimeCode;
		LayerTimeInfo.EndTimeCode = Temp;
	}

	const TArray<FString>& SubLayerPaths = Layer.GetSubLayerPaths();
	const TArray<UE::FSdfLayerOffset>& SubLayerOffsets = Layer.GetSubLayerOffsets();
	if (SubLayerPaths.Num() != SubLayerOffsets.Num())
	{
		return;
	}

	LayerTimeInfo.SublayerIdentifierToOffsets.Empty(SubLayerPaths.Num());
	for (int32 Index = 0; Index < SubLayerPaths.Num(); ++Index)
	{
		const FString& SubLayerPath = SubLayerPaths[Index];
		const UE::FSdfLayerOffset& SubLayerOffset = SubLayerOffsets[Index];

		if (UE::FSdfLayer SubLayer = UsdUtils::FindLayerForSubLayerPath(Layer, SubLayerPath))
		{
			LayerTimeInfo.SublayerIdentifierToOffsets.Add(SubLayer.GetIdentifier(), SubLayerOffset);
		}
	}
}

ULevelSequence* FUsdLevelSequenceHelperImpl::FindSequenceForIdentifier(const FString& SequenceIdentitifer) const
{
	ULevelSequence* Sequence = nullptr;
	if (auto* FoundSequence = LevelSequencesByIdentifier.Find(SequenceIdentitifer))
	{
		Sequence = *FoundSequence;
	}

	return Sequence;
}
#else
class FUsdLevelSequenceHelperImpl
{
public:
	FUsdLevelSequenceHelperImpl()
	{
	}
	~FUsdLevelSequenceHelperImpl()
	{
	}

	ULevelSequence* Init(const UE::FUsdStage& InUsdStage)
	{
		return nullptr;
	}
	bool Serialize(FArchive& Ar)
	{
		return false;
	}
	void SetPrimLinkCache(UUsdPrimLinkCache* PrimLinkCache) {};
	void SetBBoxCache(TSharedPtr<UE::FUsdGeomBBoxCache> InBBoxCache) {};
	bool HasData() const
	{
		return false;
	};
	void Clear() {};

	void CreateLocalLayersSequences()
	{
	}

	void BindToUsdStageActor(AUsdStageActor* InStageActor)
	{
	}
	void UnbindFromUsdStageActor()
	{
	}
	EUsdRootMotionHandling GetRootMotionHandling() const
	{
		return EUsdRootMotionHandling::NoAdditionalRootMotion;
	}
	void SetRootMotionHandling(EUsdRootMotionHandling NewValue) {};
	void OnStageActorRenamed() {};

	void AddPrim(UUsdPrimTwin& PrimTwin, bool bForceVisibilityTracks, TOptional<bool> HasAnimatedBounds)
	{
	}
	void RemovePrim(const UUsdPrimTwin& PrimTwin)
	{
	}

	void UpdateControlRigTracks(UUsdPrimTwin& PrimTwin)
	{
	}

	void StartMonitoringChanges()
	{
	}
	void StopMonitoringChanges()
	{
	}
	void BlockMonitoringChangesForThisTransaction()
	{
	}

	ULevelSequence* GetMainLevelSequence() const
	{
		return nullptr;
	}
	TArray<ULevelSequence*> GetSubSequences() const
	{
		return {};
	}
};
#endif	  // USE_USD_SDK

FUsdLevelSequenceHelper::FUsdLevelSequenceHelper()
{
	UsdSequencerImpl = MakeUnique<FUsdLevelSequenceHelperImpl>();
}

FUsdLevelSequenceHelper::~FUsdLevelSequenceHelper() = default;

FUsdLevelSequenceHelper::FUsdLevelSequenceHelper(const FUsdLevelSequenceHelper& Other)
	: FUsdLevelSequenceHelper()
{
}

FUsdLevelSequenceHelper& FUsdLevelSequenceHelper::operator=(const FUsdLevelSequenceHelper& Other)
{
	// No copying, start fresh
	UsdSequencerImpl = MakeUnique<FUsdLevelSequenceHelperImpl>();
	return *this;
}

FUsdLevelSequenceHelper::FUsdLevelSequenceHelper(FUsdLevelSequenceHelper&& Other) = default;
FUsdLevelSequenceHelper& FUsdLevelSequenceHelper::operator=(FUsdLevelSequenceHelper&& Other) = default;

ULevelSequence* FUsdLevelSequenceHelper::Init(const UE::FUsdStage& UsdStage)
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->Init(UsdStage);
	}
	else
	{
		return nullptr;
	}
}

bool FUsdLevelSequenceHelper::Serialize(FArchive& Ar)
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->Serialize(Ar);
	}

	return false;
}

void FUsdLevelSequenceHelper::OnStageActorRenamed()
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->OnStageActorRenamed();
	}
}

void FUsdLevelSequenceHelper::SetPrimLinkCache(UUsdPrimLinkCache* PrimLinkCache)
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->SetPrimLinkCache(PrimLinkCache);
	}
}

void FUsdLevelSequenceHelper::SetBBoxCache(TSharedPtr<UE::FUsdGeomBBoxCache> InBBoxCache)
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->SetBBoxCache(InBBoxCache);
	}
}

bool FUsdLevelSequenceHelper::HasData() const
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->HasData();
	}

	return false;
}

void FUsdLevelSequenceHelper::Clear()
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->Clear();
	}
}

void FUsdLevelSequenceHelper::BindToUsdStageActor(AUsdStageActor* StageActor)
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->BindToUsdStageActor(StageActor);
	}
}

void FUsdLevelSequenceHelper::UnbindFromUsdStageActor()
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->UnbindFromUsdStageActor();
	}
}

EUsdRootMotionHandling FUsdLevelSequenceHelper::GetRootMotionHandling() const
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->GetRootMotionHandling();
	}

	return EUsdRootMotionHandling::NoAdditionalRootMotion;
}

void FUsdLevelSequenceHelper::SetRootMotionHandling(EUsdRootMotionHandling NewValue)
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->SetRootMotionHandling(NewValue);
	}
}

void FUsdLevelSequenceHelper::AddPrim(UUsdPrimTwin& PrimTwin, bool bForceVisibilityTracks, TOptional<bool> HasAnimatedBounds)
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->AddPrim(PrimTwin, bForceVisibilityTracks, HasAnimatedBounds);
	}
}

void FUsdLevelSequenceHelper::RemovePrim(const UUsdPrimTwin& PrimTwin)
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->RemovePrim(PrimTwin);
	}
}

void FUsdLevelSequenceHelper::UpdateControlRigTracks(UUsdPrimTwin& PrimTwin)
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->UpdateControlRigTracks(PrimTwin);
	}
}

void FUsdLevelSequenceHelper::StartMonitoringChanges()
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->StartMonitoringChanges();
	}
}

void FUsdLevelSequenceHelper::StopMonitoringChanges()
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->StopMonitoringChanges();
	}
}

void FUsdLevelSequenceHelper::BlockMonitoringChangesForThisTransaction()
{
	if (UsdSequencerImpl.IsValid())
	{
		UsdSequencerImpl->BlockMonitoringChangesForThisTransaction();
	}
}

ULevelSequence* FUsdLevelSequenceHelper::GetMainLevelSequence() const
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->GetMainLevelSequence();
	}
	else
	{
		return nullptr;
	}
}

TArray<ULevelSequence*> FUsdLevelSequenceHelper::GetSubSequences() const
{
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->GetSubSequences();
	}
	else
	{
		return {};
	}
}

FUsdLevelSequenceHelper::FOnSkelAnimationBaked& FUsdLevelSequenceHelper::GetOnSkelAnimationBaked()
{
#if USE_USD_SDK
	if (UsdSequencerImpl.IsValid())
	{
		return UsdSequencerImpl->GetOnSkelAnimationBaked();
	}
	else
#endif	  // USE_USD_SDK
	{
		static FOnSkelAnimationBaked DefaultHandler;
		return DefaultHandler;
	}
}

FScopedBlockMonitoringChangesForTransaction::FScopedBlockMonitoringChangesForTransaction(FUsdLevelSequenceHelper& InHelper)
	: FScopedBlockMonitoringChangesForTransaction(*InHelper.UsdSequencerImpl.Get())
{
}

FScopedBlockMonitoringChangesForTransaction::FScopedBlockMonitoringChangesForTransaction(FUsdLevelSequenceHelperImpl& InHelperImpl)
	: HelperImpl(InHelperImpl)
{
	// If we're transacting we can just call this and the helper will unblock itself once the transaction is finished, because
	// we need to make sure the unblocking happens after any call to OnObjectTransacted.
	if (GUndo)
	{
		HelperImpl.BlockMonitoringChangesForThisTransaction();
	}
	// If we're not in a transaction we still need to block this (can also happen e.g. if a Python change triggers a stage notice),
	// but since we don't have to worry about the OnObjectTransacted calls we can just use this RAII object here to wrap over
	// any potential changes to level sequence assets
	else
	{
		bStoppedMonitoringChanges = true;
		HelperImpl.StopMonitoringChanges();
	}
}

FScopedBlockMonitoringChangesForTransaction::~FScopedBlockMonitoringChangesForTransaction()
{
	if (bStoppedMonitoringChanges)
	{
		HelperImpl.StartMonitoringChanges();
	}
}

#undef LOCTEXT_NAMESPACE
