// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "USDLocatorEditors.h"

#include "USDLocatorFragments.h"
#include "USDStageActor.h"

#include "DragAndDrop/ActorDragDropOp.h"
#include "GameFramework/Actor.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "SceneOutlinerDragDrop.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorEditor.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "USDLocatorEditors"

namespace UE::UniversalObjectLocator
{
	class SUsdPrimLocatorEditorUI : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SUsdPrimLocatorEditorUI)
		{
		}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedPtr<IFragmentEditorHandle> InHandle)
		{
			WeakHandle = InHandle;

			// clang-format off
			ChildSlot
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(FMargin(8.0f, 4.0f))
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.Text(LOCTEXT("PrimPathLabel", "Prim path:"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				[
					SNew(SEditableTextBox)
					.MinDesiredWidth(300.0f)
					.Text_Lambda([this]() -> FText
					{
						if (TSharedPtr<IFragmentEditorHandle> Handle = WeakHandle.Pin())
						{
							const FUniversalObjectLocatorFragment& Fragment = Handle->GetFragment();
							if (const FUsdPrimLocatorFragment* CastFragment = Fragment.GetPayloadAs(FUsdPrimLocatorFragment::FragmentType))
							{
								return FText::FromString(CastFragment->PrimPath);
							}
						}

						return FText::GetEmpty();
					})
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
					{
						if (TSharedPtr<IFragmentEditorHandle> Handle = WeakHandle.Pin())
						{
							const FUniversalObjectLocatorFragment& OldFragment = Handle->GetFragment();
							const FUsdPrimLocatorFragment* OldCastFragment = OldFragment.GetPayloadAs(FUsdPrimLocatorFragment::FragmentType);

							FUniversalObjectLocatorFragment NewFragment{FUsdPrimLocatorFragment::FragmentType};
							FUsdPrimLocatorFragment* NewCastFragment = NewFragment.GetPayloadAs(FUsdPrimLocatorFragment::FragmentType);

							NewCastFragment->PrimPath = NewText.ToString();
							NewCastFragment->bPreferComponent = OldCastFragment->bPreferComponent;

							Handle->SetValue(NewFragment);
						}
					})
				]
			];
			// clang-format on
		}

		TWeakPtr<IFragmentEditorHandle> WeakHandle;
	};

	ELocatorFragmentEditorType FUsdPrimLocatorEditor::GetLocatorFragmentEditorType() const
	{
		return ELocatorFragmentEditorType::Relative;
	}

	bool FUsdPrimLocatorEditor::IsAllowedInContext(FName InContextName) const
	{
		return true;
	}

	// Note: I'm not exactly sure what is supposed to be dragged and dropped here, as any drag operation seems to
	// close the right-click binding menu anyway? I can't click and drag the bindings themselves either...
	bool FUsdPrimLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
	{
		UObject* Result = ResolveDragOperation(DragOperation, Context);
		return Result != nullptr;
	}

	UObject* FUsdPrimLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
	{
		TSharedPtr<FActorDragDropOp> ActorDrag;

		if (DragOperation->IsOfType<FSceneOutlinerDragDropOp>())
		{
			FSceneOutlinerDragDropOp* SceneOutlinerOp = static_cast<FSceneOutlinerDragDropOp*>(DragOperation.Get());
			ActorDrag = SceneOutlinerOp->GetSubOp<FActorDragDropOp>();
		}
		else if (DragOperation->IsOfType<FActorDragDropOp>())
		{
			ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(DragOperation);
		}

		if (ActorDrag)
		{
			for (const TWeakObjectPtr<AActor>& WeakActor : ActorDrag->Actors)
			{
				if (AActor* Actor = WeakActor.Get())
				{
					return FUsdPrimLocatorFragment::GetAttachParentStageActor(Actor);
				}
			}
		}

		return nullptr;
	}

	TSharedPtr<SWidget> FUsdPrimLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
	{
		return SNew(SUsdPrimLocatorEditorUI, InParameters.Handle);
	}

	FText FUsdPrimLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
	{
		if (InFragment != nullptr)
		{
			ensure(InFragment->GetFragmentTypeHandle() == FUsdPrimLocatorFragment::FragmentType);

			if (const FUsdPrimLocatorFragment* CastFragment = InFragment->GetPayloadAs(FUsdPrimLocatorFragment::FragmentType))
			{
				return FText::FromString(FPaths::GetBaseFilename(CastFragment->PrimPath));
			}
		}

		return LOCTEXT("UsdPrimLocatorEditorDisplayText", "USD Prim");
	}

	FText FUsdPrimLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
	{
		if (InFragment != nullptr)
		{
			ensure(InFragment->GetFragmentTypeHandle() == FUsdPrimLocatorFragment::FragmentType);

			if (const FUsdPrimLocatorFragment* CastFragment = InFragment->GetPayloadAs(FUsdPrimLocatorFragment::FragmentType))
			{
				return FText::Format(LOCTEXT("UsdPrimLocatorEditorTooltip", "A path to prim '{0}'"), FText::FromString(CastFragment->PrimPath));
			}
		}

		return LOCTEXT("UsdPrimLocatorEditorTooltipInvalid", "A path to a prim on a USD stage");
	}

	FSlateIcon FUsdPrimLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.USDStage");
	}

	UClass* FUsdPrimLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
	{
		if (UClass* Class = ILocatorFragmentEditor::ResolveClass(InFragment, InContext))
		{
			return Class;
		}

		if (const FUsdPrimLocatorFragment* CastFragment = InFragment.GetPayloadAs(FUsdPrimLocatorFragment::FragmentType))
		{
			if (!CastFragment->bPreferComponent)
			{
				return AActor::StaticClass();
			}
		}

		return USceneComponent::StaticClass();
	}

	FUniversalObjectLocatorFragment FUsdPrimLocatorEditor::MakeDefaultLocatorFragment() const
	{
		FUniversalObjectLocatorFragment NewFragment(FUsdPrimLocatorFragment::FragmentType);
		return NewFragment;
	}

}	 // namespace UE::UniversalObjectLocator

#undef LOCTEXT_NAMESPACE

#endif	  // WITH_EDITOR
