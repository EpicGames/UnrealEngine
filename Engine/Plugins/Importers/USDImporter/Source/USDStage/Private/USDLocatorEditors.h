// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreTypes.h"
#include "UniversalObjectLocatorEditor.h"

namespace UE::UniversalObjectLocator
{
	class FUsdPrimLocatorEditor : public ILocatorFragmentEditor
	{
	public:
		// Begin ILocatorFragmentEditor interface
		ELocatorFragmentEditorType GetLocatorFragmentEditorType() const override;
		bool IsAllowedInContext(FName InContextName) const override;
		bool IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const override;
		UObject* ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const override;
		TSharedPtr<SWidget> MakeEditUI(const FEditUIParameters& InParameters) override;
		FText GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const override;
		FText GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const override;
		FSlateIcon GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const override;
		UClass* ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const override;
		FUniversalObjectLocatorFragment MakeDefaultLocatorFragment() const override;
		// End ILocatorFragmentEditor interface
	};

}	 // namespace UE::UniversalObjectLocator

#endif	  // WITH_EDITOR
