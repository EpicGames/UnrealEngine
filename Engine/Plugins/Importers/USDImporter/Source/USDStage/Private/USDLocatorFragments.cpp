// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDLocatorFragments.h"

#include "USDStageActor.h"

#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDLocatorFragments)

UE::UniversalObjectLocator::TFragmentTypeHandle<FUsdPrimLocatorFragment> FUsdPrimLocatorFragment::FragmentType;

namespace UE::USDLocatorFragments::Private
{
	const static FString ActorLocatorPrefix = TEXT("!actor!");
}	 // namespace UE::USDLocatorFragments::Private

UE::UniversalObjectLocator::FResolveResult FUsdPrimLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	if (AUsdStageActor* StageActorContext = Cast<AUsdStageActor>(Params.Context))
	{
		Result = StageActorContext->GetGeneratedComponent(PrimPath);

		// To to resolve to the actor if we should
		if (!bPreferComponent)
		{
			if (USceneComponent* Component = Cast<USceneComponent>(Result))
			{
				if (AActor* OwnerActor = Component->GetOwner())
				{
					if (Component == OwnerActor->GetRootComponent())
					{
						Result = OwnerActor;
					}
				}
			}
		}
	}

	return FResolveResultData(Result);
}

void FUsdPrimLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	if (!bPreferComponent)
	{
		OutStringBuilder.Append(UE::USDLocatorFragments::Private::ActorLocatorPrefix);
	}
	OutStringBuilder.Append(PrimPath);
}

UE::UniversalObjectLocator::FParseStringResult FUsdPrimLocatorFragment::TryParseString(
	FStringView InString,
	const UE::UniversalObjectLocator::FParseStringParams& Params
)
{
	using namespace UE::USDLocatorFragments::Private;

	bPreferComponent = true;
	if (InString.StartsWith(ActorLocatorPrefix))
	{
		bPreferComponent = false;
		InString.RemovePrefix(ActorLocatorPrefix.Len());
	}

	PrimPath = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FUsdPrimLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;
	using namespace UE::USDLocatorFragments::Private;

	const UObject* Object = InParams.Object;
	const AUsdStageActor* StageActor = GetAttachParentStageActor(Object);

	if (StageActor)
	{
		FString FoundPrimPath = StageActor->GetSourcePrimPath(Object);
		if (FoundPrimPath.IsEmpty())
		{
			return FInitializeResult::Failure();
		}

		PrimPath = FoundPrimPath;
		bPreferComponent = Object->IsA<USceneComponent>();
	}

	return FInitializeResult::Relative(StageActor);
}

uint32 FUsdPrimLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	using namespace UE::USDLocatorFragments::Private;

	// Actors/components spawned by the stage actor are always transient
	if (!ObjectToReference->HasAnyFlags(RF_Transient))
	{
		return 0;
	}

	// If we're inside a AUsdStageActor's attachment hierarchy, let's assume it's one of the actor's spawned objects
	if (GetAttachParentStageActor(ObjectToReference))
	{
		return 3000;
	}

	return 0;
}

const AUsdStageActor* FUsdPrimLocatorFragment::GetAttachParentStageActor(const UObject* Object)
{
	const AActor* Actor = Cast<const AActor>(Object);
	if (!Actor)
	{
		if (const USceneComponent* Component = Cast<const USceneComponent>(Object))
		{
			Actor = Component->GetOwner();
		}
	}

	while (Actor)
	{
		if (const AUsdStageActor* StageActor = Cast<AUsdStageActor>(Actor))
		{
			return StageActor;
		}

		Actor = Actor->GetAttachParentActor();
	}

	return nullptr;
}

AUsdStageActor* FUsdPrimLocatorFragment::GetAttachParentStageActor(UObject* Object)
{
	return const_cast<AUsdStageActor*>(GetAttachParentStageActor(const_cast<const UObject*>(Object)));
}
