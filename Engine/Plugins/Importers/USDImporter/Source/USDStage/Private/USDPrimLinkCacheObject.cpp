// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPrimLinkCacheObject.h"

#include "Objects/USDPrimLinkCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDPrimLinkCacheObject)

UUsdPrimLinkCache::UUsdPrimLinkCache()
{
	Impl = MakePimpl<FUsdPrimLinkCache>();
}

void UUsdPrimLinkCache::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUsdPrimLinkCache::Serialize);

	if (FUsdPrimLinkCache* ImplPtr = Impl.Get())
	{
		ImplPtr->Serialize(Ar);
	}
}

FUsdPrimLinkCache& UUsdPrimLinkCache::GetInner()
{
	return *Impl;
}

const FUsdPrimLinkCache& UUsdPrimLinkCache::GetInner() const
{
	return *Impl;
}
