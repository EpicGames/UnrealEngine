// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPrimTwin.h"

#include "USDMemory.h"
#include "USDStageActor.h"

#include "Components/InstancedStaticMeshComponent.h"
#include "Containers/StringView.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDPrimTwin)

UUsdPrimTwin& UUsdPrimTwin::AddChild(const FString& InPrimPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUsdPrimTwin::AddChild);

	FScopedUnrealAllocs UnrealAllocs;	 // Make sure the call to new is done with the UE allocator

	FString Dummy;
	FString ChildPrimName;
	InPrimPath.Split(TEXT("/"), &Dummy, &ChildPrimName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	Modify();

	TObjectPtr<UUsdPrimTwin>& ChildPrim = Children.Add(UsdUtils::AddHashToPrimPath(ChildPrimName));

	// Needs public because this will mostly live on the transient package (c.f. AUsdStageActor::GetRootPrimTwin())
	ChildPrim = NewObject<UUsdPrimTwin>(this, NAME_None, GetFlags() | RF_Public);
	ChildPrim->PrimPath = InPrimPath;

	ChildPrim->Parent = this;

	return *ChildPrim;
}

void UUsdPrimTwin::RemoveChild(const TCHAR* InPrimPath)
{
	FScopedUnrealAllocs UnrealAllocs;

	Modify();

	for (TMap<FString, TObjectPtr<UUsdPrimTwin>>::TIterator ChildIt = Children.CreateIterator(); ChildIt; ++ChildIt)
	{
		if (ChildIt->Value->PrimPath == InPrimPath)
		{
			ChildIt->Value->Parent.Reset();
			ChildIt.RemoveCurrent();
			break;
		}
	}
}

void UUsdPrimTwin::Clear()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUsdPrimTwin::Clear);

	FScopedUnrealAllocs UnrealAllocs;

	Modify();

	for (const TPair<FString, TObjectPtr<UUsdPrimTwin>>& Pair : Children)
	{
		// Apparently when changing levels it is possible for these objects to already be nullptr by the time we try clearing them,
		// so its safer to check
		if (Pair.Value)
		{
			Pair.Value->Clear();
		}
	}
	Children.Empty();

	if (!PrimPath.IsEmpty())
	{
		OnDestroyed.Broadcast(*this);
	}

	AActor* ActorToDestroy = nullptr;
	if (SceneComponent.IsValid() && SceneComponent->GetOwner()->GetRootComponent() == SceneComponent.Get())
	{
		ActorToDestroy = SceneComponent->GetOwner();
	}

	if (ActorToDestroy && !ActorToDestroy->IsA<AUsdStageActor>() && !ActorToDestroy->IsActorBeingDestroyed() && ActorToDestroy->GetWorld())
	{
		// We have to manually Modify() all the actor's components because they're transient, so USceneComponent::DetachFromComponent
		// won't automatically Modify them before detaching. If we don't do this they may be first recorded into the transaction in
		// the detached state, so if that transaction is undone they'd be left detached
		TArray<USceneComponent*> ChildComponents;
		ActorToDestroy->GetComponents(ChildComponents);
		for (USceneComponent* Component : ChildComponents)
		{
			Component->Modify();
		}

		ActorToDestroy->Modify();
		ActorToDestroy->GetWorld()->DestroyActor(ActorToDestroy);
	}
	else if (SceneComponent.IsValid() && !SceneComponent->IsBeingDestroyed())
	{
		// See comment above: USceneComponent::DetachFromComponent won't Modify our components since they're transient,
		// so we need to do so manually
		if (USceneComponent* AttachParent = SceneComponent->GetAttachParent())
		{
			AttachParent->Modify();
		}
		for (USceneComponent* AttachChild : SceneComponent->GetAttachChildren())
		{
			AttachChild->Modify();
		}

		// When we translate point instancers we secretly add more ISM components for each prototype.
		// If we're destroying the scene component for the PointInstancer, we should destroy the ISMs too.
		if (SceneComponent->ComponentTags.Contains(UnrealIdentifiers::PointInstancerTag))
		{
			// Copy the array here as DestroyComponent will change the container directly
			TArray<TObjectPtr<USceneComponent>> AttachChildren = SceneComponent->GetAttachChildren();
			for (USceneComponent* AttachChild : AttachChildren)
			{
				if (UInstancedStaticMeshComponent* ISM = Cast<UInstancedStaticMeshComponent>(AttachChild))
				{
					ISM->Modify();
					ISM->DestroyComponent();
				}
			}
		}

		SceneComponent->Modify();
		SceneComponent->DestroyComponent();
		SceneComponent = nullptr;
	}
}

UUsdPrimTwin* UUsdPrimTwin::Find(const FString& InPrimPath)
{
	if (PrimPath == InPrimPath)
	{
		return this;
	}

	FString RestOfPrimPathToFind;
	FString ChildPrimName;

	FString PrimPathToFind = InPrimPath;
	PrimPathToFind.RemoveFromStart(TEXT("/"));

	if (!PrimPathToFind.Split(TEXT("/"), &ChildPrimName, &RestOfPrimPathToFind))
	{
		ChildPrimName = PrimPathToFind;
	}

	if (ChildPrimName.IsEmpty())
	{
		ChildPrimName = RestOfPrimPathToFind;
		RestOfPrimPathToFind.Empty();
	}

	const FString& HashedChildName = UsdUtils::AddHashToPrimPath(ChildPrimName);
	if (Children.Contains(HashedChildName))
	{
		if (RestOfPrimPathToFind.IsEmpty())
		{
			return Children[HashedChildName];
		}
		else
		{
			return Children[HashedChildName]->Find(RestOfPrimPathToFind);
		}
	}

	return nullptr;
}

UUsdPrimTwin* UUsdPrimTwin::Find(const USceneComponent* InSceneComponent)
{
	if (SceneComponent == InSceneComponent)
	{
		return this;
	}

	for (const TPair<FString, TObjectPtr<UUsdPrimTwin>>& Child : Children)
	{
		UUsdPrimTwin* FoundPrimTwin = Child.Value->Find(InSceneComponent);
		if (FoundPrimTwin)
		{
			return FoundPrimTwin;
		}
	}

	return nullptr;
}

USceneComponent* UUsdPrimTwin::GetSceneComponent() const
{
	if (SceneComponent.IsValid())
	{
		return SceneComponent.Get();
	}

	return nullptr;
}

void UUsdPrimTwin::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Keep properties transient as we never want them to be saved to disk, but we need to
	// duplicate them here so that whenever we duplicate the root twin we also bring our
	// prim twin hierarchy with it (happens when going into PIE, or when calling DuplicateObject inside
	// AUsdStageActor::OnObjectsReplaced)
	if (Ar.GetPortFlags() & PPF_Duplicate)
	{
		Ar << Children;
		Ar << Parent;
	}
}
