// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageActor.h"

#include "Objects/USDInfoCache.h"
#include "Objects/USDPrimLinkCache.h"
#include "Objects/USDSchemaTranslator.h"
#include "UnrealUSDWrapper.h"
#include "UObject/Package.h"
#include "USDAssetCache2.h"
#include "USDAssetCache3.h"
#include "USDAssetUserData.h"
#include "USDAttributeUtils.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDDynamicBindingResolverLibrary.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDGeomXformableTranslator.h"
#include "USDIntegrationUtils.h"
#include "USDLayerUtils.h"
#include "USDLightConversion.h"
#include "USDListener.h"
#include "USDMemory.h"
#include "USDObjectUtils.h"
#include "USDPrimConversion.h"
#include "USDPrimTwin.h"
#include "USDProjectSettings.h"
#include "USDSchemasModule.h"
#include "USDSkelSkeletonTranslator.h"
#include "USDStageModule.h"
#include "USDTransactor.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdGeomBBoxCache.h"
#include "UsdWrappers/UsdGeomXformable.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdStage.h"

#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "Components/AudioComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Components/LightComponent.h"
#include "Components/LightComponentBase.h"
#include "Components/PointLightComponent.h"
#include "Components/RectLightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/Light.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCacheComponent.h"
#include "HAL/IConsoleManager.h"
#include "LevelSequence.h"
#include "LiveLinkComponentController.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Widgets/Notifications/SNotificationList.h"

#if WITH_EDITOR
#include "BlueprintActionMenuItem.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Editor/TransBuffer.h"
#include "Editor/UnrealEdEngine.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ISequencerModule.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "MovieSceneDirectorBlueprintUtils.h"
#include "MovieSceneDynamicBindingUtils.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SequencerUtilities.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UnrealEdGlobals.h"
#include "USDClassesEditorModule.h"

#endif	  // WITH_EDITOR

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdMedia/tokens.h"
#include "pxr/usd/usdPhysics/tokens.h"
#include "USDIncludesEnd.h"
#endif	  // USE_USD_SDK

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageActor)

#define LOCTEXT_NAMESPACE "USDStageActor"

static bool GHandleNoticesImmediately = false;
static FAutoConsoleVariableRef CVarHandleNoticesImmediately(
	TEXT("USD.HandleNoticesImmediately"),
	GHandleNoticesImmediately,
	TEXT(
		"When this is false, we will collect all USD notices emitted during a UE transaction and handle them in one pass, potentially eliminating some unnecessary updates. When this is true, we respond to each USD notice immediately"
	)
);

static bool GRegenerateSkeletalAssetsOnControlRigBake = true;
static FAutoConsoleVariableRef CVarRegenerateSkeletalAssetsOnControlRigBake(
	TEXT("USD.RegenerateSkeletalAssetsOnControlRigBake"),
	GRegenerateSkeletalAssetsOnControlRigBake,
	TEXT("Whether to regenerate the assets associated with a SkelRoot (mesh, skeleton, anim sequence, etc.) whenever we modify Control Rig tracks. "
		 "The USD Stage itself is always updated however.")
);

static bool GTranslateOnlyUsedMaterialsWhenOpeningStage = true;
static FAutoConsoleVariableRef CVarTranslateOnlyUsedMaterialsWhenOpeningStage(
	TEXT("USD.TranslateOnlyUsedMaterialsWhenOpeningStage"),
	GTranslateOnlyUsedMaterialsWhenOpeningStage,
	TEXT("If enabled, only Material prims bound by at least one Mesh are translated into Unreal material assets. If disabled, all Material prims are "
		 "translated into Unreal material assets.")
);

static bool GDiscardUndoBufferOnStageOpenClose = false;
static FAutoConsoleVariableRef CVarDiscardUndoBufferOnStageOpenClose(
	TEXT("USD.DiscardUndoBufferOnStageOpenClose"),
	GDiscardUndoBufferOnStageOpenClose,
	TEXT(
		"Enabling this will prevent the recording of open/close stage transactions, but also discard the undo buffer after they happen. This can help load times and also reduce memory usage, as sometimes recording all created assets and actors in the undo buffer can be expensive."
	)
);

static const EObjectFlags DefaultObjFlag = EObjectFlags::RF_Transactional | EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;

AUsdStageActor::FOnActorLoaded AUsdStageActor::OnActorLoaded;
AUsdStageActor::FOnOpenStageEditorClicked AUsdStageActor::OnOpenStageEditorClicked;

struct FUsdStageActorImpl
{
	static TSharedRef<FUsdSchemaTranslationContext> CreateUsdSchemaTranslationContext(AUsdStageActor* StageActor, const FString& PrimPath)
	{
		TSharedRef<FUsdSchemaTranslationContext> TranslationContext = MakeShared<FUsdSchemaTranslationContext>(
			StageActor->GetOrOpenUsdStage(),
			*StageActor->AssetCache
		);

		TranslationContext->Level = StageActor->GetLevel();
		TranslationContext->ObjectFlags = DefaultObjFlag;
		TranslationContext->Time = StageActor->GetTime();
		TranslationContext->PurposesToLoad = (EUsdPurpose)StageActor->PurposesToLoad;
		TranslationContext->NaniteTriangleThreshold = StageActor->NaniteTriangleThreshold;
		TranslationContext->RenderContext = StageActor->RenderContext;
		TranslationContext->MaterialPurpose = StageActor->MaterialPurpose;
		TranslationContext->RootMotionHandling = StageActor->RootMotionHandling;
		TranslationContext->FallbackCollisionType = StageActor->FallbackCollisionType;
		TranslationContext->GeometryCacheImport = StageActor->GeometryCacheImport;
		TranslationContext->SubdivisionLevel = StageActor->SubdivisionLevel;
		TranslationContext->MetadataOptions = StageActor->MetadataOptions;
		TranslationContext->BlendShapesByPath = &StageActor->BlendShapesByPath;
		TranslationContext->UsdInfoCache = StageActor->UsdInfoCache ? &StageActor->UsdInfoCache->GetInner() : nullptr;
		TranslationContext->PrimLinkCache = StageActor->PrimLinkCache ? &StageActor->PrimLinkCache->GetInner() : nullptr;
		TranslationContext->BBoxCache = StageActor->BBoxCache;
		TranslationContext->bTranslateOnlyUsedMaterials = GTranslateOnlyUsedMaterialsWhenOpeningStage;

		// Its more convenient to toggle between variants using the USDStage window, as opposed to parsing LODs
		TranslationContext->bAllowInterpretingLODs = false;

		// We parse these even when opening the stage now, as they are used in the skeletal animation tracks
		TranslationContext->bAllowParsingSkeletalAnimations = true;

		TranslationContext->KindsToCollapse = (EUsdDefaultKind)StageActor->KindsToCollapse;
		TranslationContext->bUsePrimKindsForCollapsing = StageActor->bUsePrimKindsForCollapsing;
		TranslationContext->bMergeIdenticalMaterialSlots = StageActor->bMergeIdenticalMaterialSlots;
		TranslationContext->bShareAssetsForIdenticalPrims = StageActor->bShareAssetsForIdenticalPrims;

		UE::FSdfPath UsdPrimPath(*PrimPath);
		UUsdPrimTwin* ParentUsdPrimTwin = StageActor->GetRootPrimTwin()->Find(UsdPrimPath.GetParentPath().GetString());

		if (!ParentUsdPrimTwin)
		{
			ParentUsdPrimTwin = StageActor->RootUsdTwin;
		}

		TranslationContext->ParentComponent = ParentUsdPrimTwin ? ParentUsdPrimTwin->SceneComponent.Get() : nullptr;

		if (!TranslationContext->ParentComponent)
		{
			TranslationContext->ParentComponent = StageActor->RootComponent;
		}

		return TranslationContext;
	}

	// Workaround some issues where the details panel will crash when showing a property of a component we'll force-delete
	static void DeselectActorsAndComponents(AUsdStageActor* StageActor)
	{
#if WITH_EDITOR
		// If we're being BeginDestroyed (by GC) then it's not really safe to even *check* our prim twins because
		// they may have been fully destroyed before us, and could be just garbage memory at this point
		if (!StageActor || StageActor->HasAnyFlags(RF_BeginDestroyed))
		{
			return;
		}

		// This can get called when an actor is being destroyed due to GC.
		// Don't do this during garbage collecting if we need to delay-create the root twin (can't NewObject during garbage collection).
		// If we have no root twin we don't have any tracked spawned actors and components, so we don't need to deselect anything in the first place
		bool bDeselected = false;
		if (GEditor && !IsGarbageCollecting() && StageActor->RootUsdTwin && !StageActor->RootUsdTwin->HasAnyFlags(RF_BeginDestroyed))
		{
			TArray<UObject*> ActorsToDeselect;
			TArray<UObject*> ComponentsToDeselect;

			const bool bRecursive = true;
			StageActor->GetRootPrimTwin()->Iterate(
				[&ActorsToDeselect, &ComponentsToDeselect](UUsdPrimTwin& PrimTwin)
				{
					if (USceneComponent* ReferencedComponent = PrimTwin.SceneComponent.Get())
					{
						ComponentsToDeselect.Add(ReferencedComponent);

						AActor* Owner = ReferencedComponent->GetOwner();
						if (Owner && Owner->GetRootComponent() == ReferencedComponent)
						{
							ActorsToDeselect.Add(Owner);
						}
					}
				},
				bRecursive
			);

			if (USelection* SelectedComponents = GEditor->GetSelectedComponents())
			{
				for (UObject* Component : ComponentsToDeselect)
				{
					if (SelectedComponents->IsSelected(Component))
					{
						SelectedComponents->Deselect(Component);
						bDeselected = true;
					}
				}
			}

			if (USelection* SelectedActors = GEditor->GetSelectedActors())
			{
				for (UObject* Actor : ActorsToDeselect)
				{
					if (SelectedActors->IsSelected(Actor))
					{
						SelectedActors->Deselect(Actor);
						bDeselected = true;
					}
				}
			}

			if (bDeselected && GIsEditor)	 // Make sure we're not in standalone either
			{
				GEditor->NoteSelectionChange();
			}
		}
#endif	  // WITH_EDITOR
	}

	static void DiscardStage(const UE::FUsdStage& Stage, AUsdStageActor* DiscardingActor)
	{
		if (!Stage || !DiscardingActor)
		{
			return;
		}

		UE::FSdfLayer RootLayer = Stage.GetRootLayer();
		if (RootLayer && RootLayer.IsAnonymous())
		{
			// Erasing an anonymous stage would fully delete it. If we later undo/redo into a path that referenced
			// one of those anonymous layers, we wouldn't be able to load it back again.
			// To prevent that, for now we don't actually erase anonymous stages when discarding them. This shouldn't be
			// so bad as these stages are likely to be pretty small anyway... in the future we may have some better way of
			// undo/redoing USD operations that could eliminate this issue
			return;
		}

		TArray<UObject*> Instances;
		AUsdStageActor::StaticClass()->GetDefaultObject()->GetArchetypeInstances(Instances);
		for (UObject* Instance : Instances)
		{
			if (Instance == DiscardingActor || !Instance || !IsValidChecked(Instance) || Instance->IsTemplate())
			{
				continue;
			}

			// Need to use the const version here or we may inadvertently load the stage
			if (const AUsdStageActor* Actor = Cast<const AUsdStageActor>(Instance))
			{
				const UE::FUsdStage& OtherStage = Actor->GetUsdStage();
				if (OtherStage && Stage == OtherStage)
				{
					// Some other actor is still using our stage, so don't close it
					return;
				}
			}
		}

		UnrealUSDWrapper::EraseStageFromCache(Stage);
	}

	static bool ObjectNeedsMultiUserTag(UObject* Object, AUsdStageActor* StageActor)
	{
		// Don't need to tag non-transient stuff
		if (!Object->HasAnyFlags(RF_Transient))
		{
			return false;
		}

		// Object already has tag
		if (AActor* Actor = Cast<AActor>(Object))
		{
			if (Actor->Tags.Contains(UE::UsdTransactor::ConcertSyncEnableTag))
			{
				return false;
			}
		}
		else if (USceneComponent* Component = Cast<USceneComponent>(Object))
		{
			if (Component->ComponentTags.Contains(UE::UsdTransactor::ConcertSyncEnableTag))
			{
				return false;
			}
		}

		// Only care about objects that the same actor spawned
		bool bOwnedByStageActor = false;
		if (StageActor->ObjectsToWatch.Contains(Object))
		{
			bOwnedByStageActor = true;
		}
		if (AActor* Actor = Cast<AActor>(Object))
		{
			if (StageActor->ObjectsToWatch.Contains(Actor->GetRootComponent()))
			{
				bOwnedByStageActor = true;
			}
		}
		else if (AActor* Outer = Object->GetTypedOuter<AActor>())
		{
			if (StageActor->ObjectsToWatch.Contains(Outer->GetRootComponent()))
			{
				bOwnedByStageActor = true;
			}
		}
		if (!bOwnedByStageActor)
		{
			return false;
		}

		return bOwnedByStageActor;
	}

	static void AllowListComponentHierarchy(USceneComponent* Component, TSet<UObject*>& VisitedObjects)
	{
		if (!Component || VisitedObjects.Contains(Component))
		{
			return;
		}

		VisitedObjects.Add(Component);

		if (Component->HasAnyFlags(RF_Transient))
		{
			Component->ComponentTags.AddUnique(UE::UsdTransactor::ConcertSyncEnableTag);
		}

		if (AActor* Owner = Component->GetOwner())
		{
			if (!VisitedObjects.Contains(Owner) && Owner->HasAnyFlags(RF_Transient))
			{
				Owner->Tags.AddUnique(UE::UsdTransactor::ConcertSyncEnableTag);
			}

			VisitedObjects.Add(Owner);
		}

		// Iterate the attachment hierarchy directly because maybe some of those actors have additional components that aren't being
		// tracked by a prim twin
		for (USceneComponent* Child : Component->GetAttachChildren())
		{
			AllowListComponentHierarchy(Child, VisitedObjects);
		}
	}

	// Checks if a project-relative file path refers to a layer. It requires caution because anonymous layers need to be handled differently.
	// WARNING: This will break if FilePath is a relative path relative to anything else other than the Project directory (i.e. engine binary)
	static bool DoesPathPointToLayer(FString FilePath, const UE::FSdfLayer& Layer)
	{
#if USE_USD_SDK
		if (!Layer)
		{
			return false;
		}

		if (!FilePath.IsEmpty() && !FPaths::IsRelative(FilePath) && !FilePath.StartsWith(UnrealIdentifiers::IdentifierPrefix))
		{
			FilePath = UsdUtils::MakePathRelativeToProjectDir(FilePath);
		}

		// Special handling for anonymous layers as the RealPath is empty
		if (Layer.IsAnonymous())
		{
			// Something like "anon:0000022F9E194D50:tmp.usda"
			const FString LayerIdentifier = Layer.GetIdentifier();

			// Something like "@identifier:anon:0000022F9E194D50:tmp.usda" if we're also pointing at an anonymous layer
			if (FilePath.RemoveFromStart(UnrealIdentifiers::IdentifierPrefix))
			{
				// Same anonymous layers
				if (FilePath == LayerIdentifier)
				{
					return true;
				}
			}
			// RootLayer.FilePath is not an anonymous layer but the stage is
			else
			{
				return false;
			}
		}
		else
		{
			return FPaths::IsSamePath(UsdUtils::MakePathRelativeToProjectDir(Layer.GetRealPath()), FilePath);
		}
#endif	  // USE_USD_SDK

		return false;
	}

	/**
	 * Uses USD's MakeVisible to handle the visible/inherited update logic as it is a bit complex.
	 * Will update a potentially large chunk of the component hierarchy to having/not the `invisible` component tag, as well as the
	 * correct value of bHiddenInGame.
	 * Note that bHiddenInGame corresponds to computed visibility, and the component tags correspond to individual prim-level visibilities
	 */
	static void MakeVisible(UUsdPrimTwin& UsdPrimTwin, const UE::FUsdStage& Stage)
	{
		// Find the highest invisible prim parent: Nothing above this can possibly change with what we're doing
		UUsdPrimTwin* Iter = &UsdPrimTwin;
		UUsdPrimTwin* HighestInvisibleParent = nullptr;
		while (Iter)
		{
			if (USceneComponent* Component = Iter->GetSceneComponent())
			{
				if (Component->ComponentTags.Contains(UnrealIdentifiers::Invisible))
				{
					HighestInvisibleParent = Iter;
				}
			}

			Iter = Iter->GetParent();
		}

		// No parent (not even UsdPrimTwin's prim directly) was invisible, so we should already be visible and there's nothing to do
		if (!HighestInvisibleParent)
		{
			return;
		}

		UE::FUsdPrim Prim = Stage.GetPrimAtPath(UE::FSdfPath(*UsdPrimTwin.PrimPath));
		if (!Prim)
		{
			return;
		}
		UsdUtils::MakeVisible(Prim);

		TFunction<void(UUsdPrimTwin&, bool)> RecursiveResyncVisibility;
		RecursiveResyncVisibility = [&Stage, &RecursiveResyncVisibility](UUsdPrimTwin& PrimTwin, bool bPrimHasInvisibleParent)
		{
			USceneComponent* Component = PrimTwin.GetSceneComponent();
			if (!Component)
			{
				return;
			}

			UE::FUsdPrim CurrentPrim = Stage.GetPrimAtPath(UE::FSdfPath(*PrimTwin.PrimPath));
			if (!CurrentPrim)
			{
				return;
			}

			const bool bPrimHasInheritedVisibility = UsdUtils::HasInheritedVisibility(CurrentPrim);
			const bool bPrimIsVisible = bPrimHasInheritedVisibility && !bPrimHasInvisibleParent;

			const bool bComponentHasInvisibleTag = Component->ComponentTags.Contains(UnrealIdentifiers::Invisible);
			const bool bComponentIsVisible = !Component->bHiddenInGame;

			const bool bTagIsCorrect = bComponentHasInvisibleTag == !bPrimHasInheritedVisibility;
			const bool bComputedVisibilityIsCorrect = bPrimIsVisible == bComponentIsVisible;

			// Stop recursing as this prim's or its children couldn't possibly need to be updated
			if (bTagIsCorrect && bComputedVisibilityIsCorrect)
			{
				return;
			}

			if (!bTagIsCorrect)
			{
				if (bPrimHasInheritedVisibility)
				{
					Component->ComponentTags.Remove(UnrealIdentifiers::Invisible);
					Component->ComponentTags.AddUnique(UnrealIdentifiers::Inherited);
				}
				else
				{
					Component->ComponentTags.AddUnique(UnrealIdentifiers::Invisible);
					Component->ComponentTags.Remove(UnrealIdentifiers::Inherited);
				}
			}

			if (!bComputedVisibilityIsCorrect)
			{
				const bool bPropagateToChildren = false;
				Component->Modify();
				Component->SetHiddenInGame(!bPrimIsVisible, bPropagateToChildren);
			}

			for (const TPair<FString, TObjectPtr<UUsdPrimTwin>>& ChildPair : PrimTwin.GetChildren())
			{
				if (UUsdPrimTwin* ChildTwin = ChildPair.Value)
				{
					RecursiveResyncVisibility(*ChildTwin, !bPrimIsVisible);
				}
			}
		};

		const bool bHasInvisibleParent = false;
		RecursiveResyncVisibility(*HighestInvisibleParent, bHasInvisibleParent);
	}

	/**
	 * Sets this prim to 'invisible', and force all of the child components
	 * to bHiddenInGame = false. Leave their individual prim-level visibilities intact though.
	 * Note that bHiddenInGame corresponds to computed visibility, and the component tags correspond to individual prim-level visibilities
	 */
	static void MakeInvisible(UUsdPrimTwin& UsdPrimTwin)
	{
		USceneComponent* PrimSceneComponent = UsdPrimTwin.GetSceneComponent();
		if (!PrimSceneComponent)
		{
			return;
		}

		PrimSceneComponent->ComponentTags.AddUnique(UnrealIdentifiers::Invisible);
		PrimSceneComponent->ComponentTags.Remove(UnrealIdentifiers::Inherited);

		const bool bPropagateToChildren = true;
		const bool bNewHidden = true;
		PrimSceneComponent->SetHiddenInGame(bNewHidden, bPropagateToChildren);
	}

	static void SendAnalytics(
		AUsdStageActor* StageActor,
		double ElapsedSeconds,
		double NumberOfFrames,
		const FString& Extension,
		const TSet<FSoftObjectPath>& ActiveAssetPaths
	)
	{
		if (!StageActor)
		{
			return;
		}

		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;

			EventAttributes.Emplace(TEXT("InitialLoadSet"), LexToString((uint8)StageActor->InitialLoadSet));
			EventAttributes.Emplace(TEXT("InterpolationType"), LexToString((uint8)StageActor->InterpolationType));
			EventAttributes.Emplace(TEXT("KindsToCollapse"), LexToString(StageActor->KindsToCollapse));
			EventAttributes.Emplace(TEXT("bUsePrimKindsForCollapsing"), StageActor->bUsePrimKindsForCollapsing);
			EventAttributes.Emplace(TEXT("MergeIdenticalMaterialSlots"), LexToString(StageActor->bMergeIdenticalMaterialSlots));
			EventAttributes.Emplace(TEXT("bShareAssetsForIdenticalPrims"), StageActor->bShareAssetsForIdenticalPrims);
			EventAttributes.Emplace(TEXT("PurposesToLoad"), LexToString(StageActor->PurposesToLoad));
			EventAttributes.Emplace(TEXT("NaniteTriangleThreshold"), LexToString(StageActor->NaniteTriangleThreshold));
			EventAttributes.Emplace(TEXT("RenderContext"), StageActor->RenderContext.ToString());
			EventAttributes.Emplace(TEXT("MaterialPurpose"), StageActor->MaterialPurpose.ToString());
			EventAttributes.Emplace(TEXT("RootMotionHandling"), LexToString((uint8)StageActor->RootMotionHandling));
			EventAttributes.Emplace(TEXT("FallbackCollisionType"), (uint8)StageActor->FallbackCollisionType);
			EventAttributes.Emplace(TEXT("GeometryCacheImport"), LexToString((uint8)StageActor->GeometryCacheImport));
			EventAttributes.Emplace(TEXT("SubdivisionLevel"), LexToString(StageActor->SubdivisionLevel));

			UsdUtils::AddAnalyticsAttributes(StageActor->MetadataOptions, EventAttributes);

			TSet<UObject*> ActiveAssets;
			ActiveAssets.Reserve(ActiveAssetPaths.Num());
			for (const FSoftObjectPath& ActivePath : ActiveAssetPaths)
			{
				ActiveAssets.Add(ActivePath.TryLoad());
			}
			ActiveAssets.Remove(nullptr);
			IUsdClassesModule::AddAssetCountAttributes(ActiveAssets, EventAttributes);

			const bool bAutomated = false;
			IUsdClassesModule::SendAnalytics(MoveTemp(EventAttributes), TEXT("Open"), bAutomated, ElapsedSeconds, NumberOfFrames, Extension);
		}
	}

	// If we have any Sequencer opened with a persistent LevelSequence, this will refresh them so that if their LevelSequences had a binding
	// to one of our actors that was broken, it can be immediately repaired
	static void RepairExternalSequencerBindings()
	{
#if WITH_EDITOR
		IUsdStageModule& UsdStageModule = FModuleManager::Get().LoadModuleChecked<IUsdStageModule>(TEXT("UsdStage"));
		for (const TWeakPtr<ISequencer>& ExistingSequencer : UsdStageModule.GetExistingSequencers())
		{
			if (TSharedPtr<ISequencer> PinnedSequencer = ExistingSequencer.Pin())
			{
				if (UMovieSceneSequence* FocusedSequence = PinnedSequencer->GetFocusedMovieSceneSequence())
				{
					PinnedSequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
				}
			}
		}
#endif	  // WITH_EDITOR
	}

	static void GetDescendantMovieSceneSequences(UMovieSceneSequence* InSequence, TSet<UMovieSceneSequence*>& OutAllSequences)
	{
		if (InSequence == nullptr || OutAllSequences.Contains(InSequence))
		{
			return;
		}

		OutAllSequences.Add(InSequence);

		UMovieScene* MovieScene = InSequence->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		for (UMovieSceneSection* Section : MovieScene->GetAllSections())
		{
			UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
			if (SubSection != nullptr)
			{
				UMovieSceneSequence* SubSequence = SubSection->GetSequence();
				if (SubSequence != nullptr)
				{
					GetDescendantMovieSceneSequences(SubSequence, OutAllSequences);
				}
			}
		}
	}

	static void ShowTransformOnCameraComponentWarning(const UActorComponent* Component)
	{
		const UCineCameraComponent* CameraComponent = Cast<const UCineCameraComponent>(Component);
		if (!CameraComponent)
		{
			return;
		}
		const AActor* OwnerActor = CameraComponent->GetOwner();
		if (!OwnerActor)
		{
			return;
		}

		FObjectKey NewComponentKey{Component};
		static TSet<FObjectKey> WarnedComponents;
		if (WarnedComponents.Contains(NewComponentKey))
		{
			return;
		}
		WarnedComponents.Add(NewComponentKey);

		const FText Text = LOCTEXT("TransformOnCameraComponentText", "USD: Transform on camera component");

		const FText SubText = FText::Format(
			LOCTEXT(
				"TransformOnCameraComponentSubText",
				"The transform of camera component '{0}' was modified, but the new value will not be written out to the USD stage.\n\nIn order to "
				"write to the Camera prim "
				"transform, please modify the transform of the Cine Camera Actor (or its root Scene Component) instead."
			),
			FText::FromString(Component->GetName())
		);

		USD_LOG_USERWARNING(FText::FromString(SubText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (Settings && Settings->bShowTransformOnCameraComponentWarning)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(Text);
			Toast.SubText = SubText;
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.CheckBoxText = LOCTEXT("DontAskAgain", "Don't prompt again");
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;
			Toast.ButtonDetails.Emplace(
				LOCTEXT("OverridenOpinionMessageOk", "Ok"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);
			// This is flipped because the default checkbox message is "Don't prompt again"
			Toast.CheckBoxState = Settings->bShowTransformOnCameraComponentWarning ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
			Toast.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
				[](ECheckBoxState NewState)
				{
					if (UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>())
					{
						// This is flipped because the default checkbox message is "Don't prompt again"
						Settings->bShowTransformOnCameraComponentWarning = NewState == ECheckBoxState::Unchecked;
						Settings->SaveConfig();
					}
				}
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
	}

	// This function is in charge of writing out to USD the analogous metadata change that
	// we just received for ChangedUserData via the PropertyChangedEvent
	static void WriteOutAssetMetadataChange(
		const AUsdStageActor* StageActor,
		const UUsdAssetUserData* ChangedUserData,
		const FPropertyChangedEvent& PropertyChangedEvent
	)
	{
#if USE_USD_SDK
		if (!StageActor || !ChangedUserData)
		{
			return;
		}

		UE::FUsdStage Stage = StageActor->GetUsdStage();
		if (!Stage)
		{
			return;
		}

		const bool bChangeWasRemoval = PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove
									   || PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear;

		const bool bHasMetadataFilters = StageActor->MetadataOptions.BlockedPrefixFilters.Num() > 0;

		// For this call, we'll only care about metadata on prims that are from the stage opened by this stage actor.
		// If we need to modify other stages for this PropertyChangedEvent somehow, the other stage actors' call to this
		// function will handle them
		FString StageIdentifier = Stage.GetRootLayer().GetIdentifier();
		const FUsdCombinedPrimMetadata* StageMetadata = ChangedUserData->StageIdentifierToMetadata.Find(StageIdentifier);

		// This asset user data doesn't have any metadata for this particular stage, nothing to do
		if (!StageMetadata)
		{
			return;
		}

		for (const TPair<FString, FUsdPrimMetadata>& PrimPathToMetadata : StageMetadata->PrimPathToMetadata)
		{
			const FString& PrimPath = PrimPathToMetadata.Key;
			const FUsdPrimMetadata& PrimMetadata = PrimPathToMetadata.Value;

			const UE::FUsdPrim& Prim = Stage.GetPrimAtPath(UE::FSdfPath{*PrimPath});

			// If the change we need to write out is a removal, since we can't tell *what* was removed from the PropertyChangedEvent,
			// the only thing we can do is wipe all metadata on the prim and replace that with what we have on our AssetUserData
			if (bChangeWasRemoval)
			{
				// If the metadata we have was obtained with metadata filters, we're in trouble: We can't just clear everything
				// and write what we have, because we just have the stuff that passed the filter. What we'll do here then is
				// invert the filters and collect metadata again (which gives us the stuff that are *not* already in our
				// AssetUserData), then clear all metadata on the prim, write out that "inverted" dataset, and (later) also
				// write out our current AssetUserData
				if (bHasMetadataFilters)
				{
					// We are clearing/writing to a particular prim here, "collecting from subtrees" is an UE-concept
					const bool bCollectFromEntireSubtrees = false;
					FUsdCombinedPrimMetadata TempInvertedMetadata;

					const bool bSuccess = UsdToUnreal::ConvertMetadata(
						Prim,
						TempInvertedMetadata,
						StageActor->MetadataOptions.BlockedPrefixFilters,
						!StageActor->MetadataOptions.bInvertFilters,
						bCollectFromEntireSubtrees
					);

					// Don't clear anything if we failed to collect the inverted dataset
					if (bSuccess)
					{
						UsdUtils::ClearNonEssentialPrimMetadata(Prim);
						UnrealToUsd::ConvertMetadata(TempInvertedMetadata, Prim);
					}
				}
				// If what we have currently was obtained without any filters, we can be sure that what we have is a good
				// representation of all metadata on this prim, so we can just clear everything and write what we have
				else
				{
					UsdUtils::ClearNonEssentialPrimMetadata(Prim);
				}
			}

			UnrealToUsd::ConvertMetadata(PrimMetadata, Prim);
		}
#endif	  // USE_USD_SDK
	}

	static UsdUtils::FCaseSensitiveStringSet GetPointInstancerPrototypes(const UE::FUsdPrim& Prim)
	{
		UsdUtils::FCaseSensitiveStringSet PrototypePaths;

#if USE_USD_SDK
		static FString PrototypesStr = UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->prototypes);
		if (UE::FUsdRelationship Relationship = Prim.GetRelationship(*PrototypesStr))
		{
			TArray<UE::FSdfPath> Targets;
			if (Relationship.GetTargets(Targets))
			{
				PrototypePaths.Reserve(Targets.Num());
				for (const UE::FSdfPath& Path : Targets)
				{
					PrototypePaths.Add(Path.GetString());
				}
			}
		}
#endif	  // USE_USD_SDK

		return PrototypePaths;
	}
};

/**
 * Class that helps us know when a blueprint that derives from AUsdStageActor is being compiled.
 * Crucially this includes the process where existing instances of that blueprint are being reinstantiated and replaced.
 *
 * Recompiling a blueprint is not a transaction, which means we can't ever load a new stage during the process of
 * recompilation, or else the spawned assets/actors wouldn't be accounted for in the undo buffer and would lead to undo/redo bugs.
 *
 * This is a problem because we use PostActorCreated to load the stage whenever a blueprint is first placed on a level,
 * but that function also gets called during the reinstantiation process (where we can't load the stage). This means we need to be
 * able to tell from PostActorCreated when we're a new actor being dropped on the level, or just a reinstantiating actor
 * replacing an existing one, which is what this class provides.
 */
#if WITH_EDITOR
struct FRecompilationTracker
{
	static void SetupEvents()
	{
		if (bEventIsSetup || !GIsEditor || !GEditor)
		{
			return;
		}

		GEditor->OnBlueprintPreCompile().AddStatic(&FRecompilationTracker::OnCompilationStarted);
		bEventIsSetup = true;
	}

	static bool IsBeingCompiled(UBlueprint* BP)
	{
		return FRecompilationTracker::RecompilingBlueprints.Contains(BP);
	}

	static void OnCompilationStarted(UBlueprint* BP)
	{
		// We don't care if a BP is compiling on first load: It only matters to use if we're compiling one that already has loaded instances on the
		// level
		if (!BP || BP->bIsRegeneratingOnLoad || !BP->GeneratedClass || !BP->GeneratedClass->IsChildOf(AUsdStageActor::StaticClass())
			|| RecompilingBlueprints.Contains(BP))
		{
			return;
		}

		FDelegateHandle Handle = BP->OnCompiled().AddStatic(&FRecompilationTracker::OnCompilationEnded);
		FRecompilationTracker::RecompilingBlueprints.Add(BP, Handle);
	}

	static void OnCompilationEnded(UBlueprint* BP)
	{
		if (!BP)
		{
			return;
		}

		FDelegateHandle RemovedHandle;
		if (FRecompilationTracker::RecompilingBlueprints.RemoveAndCopyValue(BP, RemovedHandle))
		{
			BP->OnCompiled().Remove(RemovedHandle);
		}
	}

private:
	static bool bEventIsSetup;
	static TMap<UBlueprint*, FDelegateHandle> RecompilingBlueprints;
};
bool FRecompilationTracker::bEventIsSetup = false;
TMap<UBlueprint*, FDelegateHandle> FRecompilationTracker::RecompilingBlueprints;
#endif	  // WITH_EDITOR

AUsdStageActor::AUsdStageActor()
	: StageState(EUsdStageState::OpenedAndLoaded)
	, InitialLoadSet(EUsdInitialLoadSet::LoadAll)
	, InterpolationType(EUsdInterpolationType::Linear)
	, GeometryCacheImport(EGeometryCacheImport::Never)
	, bUsePrimKindsForCollapsing(true)
	, KindsToCollapse((int32)(EUsdDefaultKind::Component | EUsdDefaultKind::Subcomponent))
	, bMergeIdenticalMaterialSlots(true)
	, bShareAssetsForIdenticalPrims(true)
	, PurposesToLoad((int32)EUsdPurpose::Proxy)
	, NaniteTriangleThreshold((uint64)1000000)
	, RenderContext(UnrealIdentifiers::UnrealRenderContext)
	, MaterialPurpose(*UnrealIdentifiers::MaterialPreviewPurpose)
	, RootMotionHandling(EUsdRootMotionHandling::NoAdditionalRootMotion)
	, FallbackCollisionType(EUsdCollisionType::ConvexHull)
	, SubdivisionLevel(0)
	, MetadataOptions(FUsdMetadataImportOptions{
		  false, /* bCollectMetadata */
		  false, /* bCollectFromEntireSubtrees */
		  false, /* bCollectOnComponents */
		  {},	 /* BlockedPrefixFilters */
		  false	 /* bInvertFilters */
	  })
	, Time(0.0f)
	, bIsTransitioningIntoPIE(false)
	, bIsModifyingAProperty(false)
	, bIsUndoRedoing(false)
{
	SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent0"));
	SceneComponent->Mobility = EComponentMobility::Static;

	RootComponent = SceneComponent;

	const FName RootTwinName = MakeUniqueObjectName(this, UUsdPrimTwin::StaticClass(), TEXT("RootUsdTwin"));
	const FName InfoCacheName = MakeUniqueObjectName(this, UUsdInfoCache::StaticClass(), TEXT("InfoCache"));
	const FName PrimLinkCacheName = MakeUniqueObjectName(this, UUsdPrimLinkCache::StaticClass(), TEXT("LinkCache"));
	const FName TransactorName = MakeUniqueObjectName(this, UUsdTransactor::StaticClass(), TEXT("Transactor"));
	const bool bTransient = true;

	RootUsdTwin = CreateDefaultSubobject<UUsdPrimTwin>(RootTwinName, bTransient);
	UsdInfoCache = CreateDefaultSubobject<UUsdInfoCache>(InfoCacheName, bTransient);
	PrimLinkCache = CreateDefaultSubobject<UUsdPrimLinkCache>(PrimLinkCacheName, bTransient);
	Transactor = CreateDefaultSubobject<UUsdTransactor>(TransactorName, bTransient);
	Transactor->Initialize(this);

	// We never want to be without a valid BBoxCache or else we'll silently fail to compute bounds for all
	// draw mode components we end up spawning
	SetupBBoxCacheIfNeeded();

	if (!IsTemplate())
	{
#if WITH_EDITOR
		// Update the supported filetypes in our RootPath property
		for (TFieldIterator<FProperty> PropertyIterator(AUsdStageActor::StaticClass()); PropertyIterator; ++PropertyIterator)
		{
			FProperty* Property = *PropertyIterator;
			if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(AUsdStageActor, RootLayer))
			{
				TArray<FString> SupportedExtensions = UnrealUSDWrapper::GetNativeFileFormats();
				if (SupportedExtensions.Num() > 0)
				{
					// Note: Cannot have space after semicolon or else the parsing breaks on the Mac...
					FString JoinedWithSemicolon = FString::Join(SupportedExtensions, TEXT(";*."));
					FString JoinedWithComma = FString::Join(SupportedExtensions, TEXT(", *."));

					Property->SetMetaData(
						TEXT("FilePathFilter"),
						FString::Printf(TEXT("Universal Scene Description files (*.%s)|*.%s"), *JoinedWithComma, *JoinedWithSemicolon)
					);
				}
				break;
			}
		}

		FEditorDelegates::BeginPIE.AddUObject(this, &AUsdStageActor::OnBeginPIE);
		FEditorDelegates::PostPIEStarted.AddUObject(this, &AUsdStageActor::OnPostPIEStarted);

		FUsdDelegates::OnPostUsdImport.AddUObject(this, &AUsdStageActor::OnPostUsdImport);
		FUsdDelegates::OnPreUsdImport.AddUObject(this, &AUsdStageActor::OnPreUsdImport);

		GEngine->OnLevelActorDeleted().AddUObject(this, &AUsdStageActor::OnLevelActorDeleted);

		// When another client of a multi-user session modifies their version of this actor, the transaction will be replicated here.
		// The multi-user system uses "redo" to apply those transactions, so this is our best chance to respond to events as e.g. neither
		// PostTransacted nor Destroyed get called when the other user deletes the actor
		if (UTransBuffer* TransBuffer = GUnrealEd ? Cast<UTransBuffer>(GUnrealEd->Trans) : nullptr)
		{
			TransBuffer->OnTransactionStateChanged().AddUObject(this, &AUsdStageActor::HandleTransactionStateChanged);

			// We can't use AddUObject here as we may specifically want to respond *after* we're marked as pending kill
			OnRedoHandle = TransBuffer->OnRedo().AddLambda(
				[this](const FTransactionContext& TransactionContext, bool bSucceeded)
				{
					// This text should match the one in ConcertClientTransactionBridge.cpp
					if (this && HasAuthorityOverStage()
						&& TransactionContext.Title.EqualTo(LOCTEXT("ConcertTransactionEvent", "Concert Transaction Event"))
						&& !RootLayer.FilePath.IsEmpty())
					{
						// Other user deleted us
						if (!IsValidChecked(this))
						{
							Reset();
						}
						// We have a valid filepath but no objects/assets spawned, so it's likely we were just spawned on the
						// other client, and were replicated here with our RootLayer path already filled out, meaning we should just load that stage
						// Note that now our UUsdTransactor may have already caused the stage itself to be loaded, but we may still need to call
						// LoadUsdStage on our end.
						else if (ObjectsToWatch.Num() == 0 && (!AssetCache || AssetCache->GetNumAssets() == 0))
						{
							this->LoadUsdStage();
							AUsdStageActor::OnActorLoaded.Broadcast(this);
						}
					}
				}
			);
		}

		// Note that we subscribe to some additional events from PostInitProperties()

		LevelSequenceHelper.GetOnSkelAnimationBaked().AddUObject(this, &AUsdStageActor::OnSkelAnimationBaked);

#endif	  // WITH_EDITOR

		OnTimeChanged.AddUObject(this, &AUsdStageActor::AnimatePrims);

		UsdListener.GetOnObjectsChanged().AddUObject(this, &AUsdStageActor::OnUsdObjectsChanged);

		UsdListener.GetOnSdfLayersChanged().AddLambda(
			[&, this](const UsdUtils::FLayerToSdfChangeList& LayersToChangeList)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::OnSdfLayersChanged);

				if (!IsListeningToUsdNotices() || LayersToChangeList.Num() == 0)
				{
					return;
				}

				const UE::FUsdStage& Stage = GetUsdStage();
				if (!Stage)
				{
					return;
				}

				TSet<FString> ChangedLayers;

				// Check to see if any of the stage's layers reloaded, or if we added/removed any layer
				TSet<UE::FSdfLayer> UsedLayers{Stage.GetUsedLayers()};
				for (const TPair<UE::FSdfLayerWeak, UsdUtils::FSdfChangeList>& LayerToChangeList : LayersToChangeList)
				{
					if (!UsedLayers.Contains(LayerToChangeList.Key))
					{
						continue;
					}

					const FString ChangedLayerIdentifier = LayerToChangeList.Key.GetIdentifier();

					for (const TPair<UE::FSdfPath, UsdUtils::FSdfChangeListEntry>& Change : LayerToChangeList.Value)
					{
						if (Change.Value.Flags.bDidReloadContent)
						{
							// Luckily whenever USD emits a one of these events for a layer reload or sublayer being added/removed,
							// we also get an object changed notice right after it. This means that we really don't need to do anything
							// here except to flag that on the next HandleAccumulatedNotices call we really should make sure our
							// LevelSequence is reloaded, so that we generate subsections for these layers that were added/removed.
							bLayerReloaded = true;
							return;
						}

						for (const TPair<FString, UsdUtils::ESubLayerChangeType>& SubLayerChange : Change.Value.SubLayerChanges)
						{
							if (SubLayerChange.Value == UsdUtils::ESubLayerChangeType::SubLayerAdded
								|| SubLayerChange.Value == UsdUtils::ESubLayerChangeType::SubLayerRemoved)
							{
								bLayerReloaded = true;
								return;
							}
						}
					}
				}
			}
		);
	}
}

void AUsdStageActor::NewStage()
{
#if USE_USD_SDK
	UE::FUsdStage NewStage = UnrealUSDWrapper::NewStage();
	if (!NewStage)
	{
		return;
	}

	// We'll create an in-memory stage, and so the "RootLayer" path we'll use will be a
	// magic path that is guaranteed to never exist in a filesystem due to invalid characters.
	UE::FSdfLayer Layer = NewStage.GetRootLayer();
	if (!Layer)
	{
		return;
	}
	FString StagePath = FString(UnrealIdentifiers::IdentifierPrefix) + Layer.GetIdentifier();

	UE::FUsdPrim RootPrim = NewStage.DefinePrim(UE::FSdfPath{TEXT("/Root")}, TEXT("Xform"));
	ensure(UsdUtils::SetDefaultKind(RootPrim, EUsdDefaultKind::Assembly));

	NewStage.SetDefaultPrim(RootPrim);

	// Call OpenStage to intentionally put the new stage within the usdutils stage cache if we're on "Closed" state.
	// This is important because at least for now we want memory-only stages to always stick around after "closed",
	// so we can undo/redo back into them.
	// Yes, this is strange and will be removed eventually, but is needed now or else we'll get undo/redo crashes.
	//
	// Normally the new stage would naturally end up in the stage cache because SetRootLayer calls OpenUsdStage,
	// which also calls OpenStage on our RootLayer path. The issue here is that OpenUsdStage will (correctly) not
	// do anything if StageState is Closed, so here we need to cache that stage ourselves.
	//
	// This trick has the effect that clicking "New Stage" when StageState == Closed will stealthily open the stage
	// and put it into the stage cache, but not open it *on the stage actor itself*. When changing stage state
	// to e.g. "Opened", we'll try opening the stage on our RootLayer path and successfully end up opening that
	// memory-only stage, as the RootLayer will contain its identifier.
	if (StageState == EUsdStageState::Closed)
	{
		UnrealUSDWrapper::OpenStage(*StagePath, InitialLoadSet);
	}

	SetRootLayer(StagePath);
#endif	  // USE_USD_SDK
}

void AUsdStageActor::SetIsolatedRootLayer(const FString& IsolatedStageRootLayer)
{
	// Only clear the isolated layer if we intentionally pass an empty path
	if (IsolatedStageRootLayer.IsEmpty())
	{
		IsolateLayer(UE::FSdfLayer{});
	}
	else
	{
		if (UE::FSdfLayer LayerToIsolate = UE::FSdfLayer::FindOrOpen(*IsolatedStageRootLayer))
		{
			IsolateLayer(LayerToIsolate);
		}
		else
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT("FailIsolateMissingFile", "Failed to isolate layer '{0}': File does not exist or is not a valid USD layer"),
				FText::FromString(IsolatedStageRootLayer)
			));
		}
	}
}

FString AUsdStageActor::GetIsolatedRootLayer() const
{
	return IsolatedStage ? IsolatedStage.GetRootLayer().GetIdentifier() : FString{};
}

void AUsdStageActor::IsolateLayer(const UE::FSdfLayer& Layer, bool bLoadUsdStage)
{
	if (IsolatedStage && IsolatedStage.GetRootLayer() == Layer)
	{
		return;
	}

	// The USD Stage Editor listens to OnPreStageChanged and will use UE::UsdStageEditorModule::Private::SaveStageActorLayersForWorld
	// to show the "Save dirty layers" dialog as a response, if we have any dirty/memory-only layers. We're never really going to
	// discard unsaved changes by isolating/stopping isolation though, so we don't actually need to save anything in this case...
	// Let's temporarily tweak the project settings to disable automatic saving of dirty layers while we swap our isolated layer.
	UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>();
	if (!Settings)
	{
		return;
	}
	TGuardValue<EUsdSaveDialogBehavior> DisableDialogGuard{Settings->ShowSaveLayersDialogWhenClosing, EUsdSaveDialogBehavior::NeverSave};

	OnPreStageChanged.Broadcast();

	UE::FUsdStage NewIsolatedStage;
	UE::FUsdStage StageToListenTo;

	// Stop isolating
	if (!Layer || Layer == UsdStage.GetRootLayer())
	{
		NewIsolatedStage = UE::FUsdStage{};

		StageToListenTo = UsdStage;
	}
	else if (UsdStage)
	{
		// We should only be allowed to isolate a layer belonging to UsdStage's local layer stack, but checking for that
		// is not trivial given that layers can be muted.

		const bool bIncludeSessionLayers = true;
		TSet<UE::FSdfLayer> ValidLayers{UsdStage.GetLayerStack(bIncludeSessionLayers)};

		UE::FUsdStage FreshCurrentStage;
		if (!Layer.IsAnonymous() && !ValidLayers.Contains(Layer))
		{
			// If the layer has a file on disk but ValidLayers does not contain it, there's still a chance that this
			// is in fact part of the usual layer stack of the stage but is currently muted. To check for that we need
			// to reopen a fresh copy of the stage, as muted layers don't usually show up on the layer stack.
			// Note that we can't just check the list of muted layers either, as it's possible to mute *any* layer for
			// a given stage, not only the layers that are currently used by it.
			// We'll use an empty population mask though (which should prevent prim composition) and just use the layers
			// that are already opened on the current stage anyway, so this should be cheap
			FreshCurrentStage = UnrealUSDWrapper::OpenMaskedStage(*UsdStage.GetRootLayer().GetIdentifier(), EUsdInitialLoadSet::LoadNone, {});
			ensure(FreshCurrentStage);

			ValidLayers.Append(FreshCurrentStage.GetLayerStack(bIncludeSessionLayers));
		}

		if (!ValidLayers.Contains(Layer))
		{
			USD_LOG_USERWARNING(FText::Format(
				LOCTEXT(
					"FailIsolateNonLocal",
					"Failed to isolate layer '{0}' as it is not part of the currently opened USD Stage's local layer stack"
				),
				FText::FromString(Layer.GetIdentifier())
			));
			return;
		}

		// We really want our own stage for this and not something from the stage cache.
		// Plus, this means its easier to cleanup: Just drop our IsolatedStage
		const bool bUseStageCache = false;
		NewIsolatedStage = UnrealUSDWrapper::OpenStage(Layer, {}, EUsdInitialLoadSet::LoadAll, bUseStageCache);
		NewIsolatedStage.SetEditTarget(NewIsolatedStage.GetRootLayer());
		NewIsolatedStage.SetInterpolationType(InterpolationType);

		StageToListenTo = NewIsolatedStage;
	}

	// For now we're kind of using a small hack where we record our isolated layer before and after we change it.
	// When undo/redoing through the accumulated edits for this transaction, the transactor will play them out in the
	// correct direction (forwards when redoing, backwards when undoing), and switch the isolated layer when needed and
	// do nothing when the isolated layer is already correct, so we lose nothing by calling this twice per transaction
	// and get the benefit of the change application "direction" code automatically working.
	//
	// Later on we should hopefully get a command pattern undo/redo system, and we can get something cleaner than this
	auto RecordIsolatedLayer = [this]()
	{
		if (Transactor)
		{
			const static UsdUtils::FObjectChangesByPath Empty;
			Transactor->Update(Empty, Empty);
		}
	};

	RecordIsolatedLayer();
	{
		IsolatedStage = NewIsolatedStage;
	}
	RecordIsolatedLayer();

	UsdListener.Register(StageToListenTo);

	if (bLoadUsdStage)
	{
		LoadUsdStage();
	}

	// Fire this so that the USD Stage Editor knows to refresh.
	// Plus we kind of changed the active stage too
	OnStageChanged.Broadcast();
}

void AUsdStageActor::OnUsdObjectsChanged(const UsdUtils::FObjectChangesByPath& InfoChanges, const UsdUtils::FObjectChangesByPath& ResyncChanges)
{
#if USE_USD_SDK
	if (!IsListeningToUsdNotices() || !UsdInfoCache)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::OnUsdObjectsChanged);

	// If we're opened, we shouldn't have any actor, component or asset. We shouldn't even have a built info cache!
	// This means we don't really need to do anything here, except trigger the USD Stage Editor to refresh.
	// We'd otherwise go through a lot of work to find out the prims to update, but given that this is only for UI refresh
	// and that we don't even have an info cache anyway, let's just have a simple loop over all prims mentioned in the
	// notice and refresh the stage editor with them.
	// Note that the stage editor only refreshes once per tick anyway, so this shouldn't even cause any unnecessary refresh spam
	if (StageState == EUsdStageState::Opened)
	{
		TMap<UE::FSdfPath, bool> PrimsToUpdateOrResync;
		for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& Change : InfoChanges)
		{
			const bool bIsResync = false;
			PrimsToUpdateOrResync.Add({UE::FSdfPath(*Change.Key).StripAllVariantSelections(), bIsResync});
		}
		// Resyncs afterward so they overwrite
		for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& Change : ResyncChanges)
		{
			const bool bIsResync = true;
			PrimsToUpdateOrResync.Add({UE::FSdfPath(*Change.Key).StripAllVariantSelections(), bIsResync});
		}

		for (const TPair<UE::FSdfPath, bool>& PrimAndResync : PrimsToUpdateOrResync)
		{
			OnPrimChanged.Broadcast(PrimAndResync.Key.GetString(), PrimAndResync.Value);
		}
		return;
	}
	else if (StageState == EUsdStageState::Closed)
	{
		// If we're in the closed state we shouldn't have a stage, so we shouldn't ever get a notice
		ensure(false);
		return;
	}

	for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& InfoChange : InfoChanges)
	{
		AccumulatedInfoChanges.FindOrAdd(InfoChange.Key).Append(InfoChange.Value);
	}
	for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& ResyncChange : ResyncChanges)
	{
		AccumulatedResyncChanges.FindOrAdd(ResyncChange.Key).Append(ResyncChange.Value);
	}

	// We want to update our transactor right away, even if we may call HandleAccumulatedNotices() only at the end of the transaction.
	// This because we want to store the current edit target when these change notices were emitted, so that we can reply them
	// back to the right layer when undo/redoing. It's all going to always be on the same transaction anyway though
	//
	// Only update the transactor if we're listening to USD notices. Within OnObjectPropertyChanged we will stop listening when writing stage changes
	// from our component changes, and this will also make sure we're not duplicating the events we store and replicate via multi-user: If a
	// modification can be described purely via UObject changes, then those changes will be responsible for the whole modification and we won't record
	// the corresponding stage changes. The intent is that when undo/redo/replicating that UObject change, it will automatically generate the
	// corresponding stage changes
	if (Transactor)
	{
		Transactor->Update(InfoChanges, ResyncChanges);
	}

	// If we don't have a transaction currently, then HandleTransactionStateChanged will never be called in order to eventually
	// call HandleAccumulatedNotices, so we have no choice but to do it now. If we didn't do this, users would always need to put
	// their Python USD changes within a UE scoped transaction to get the stage actor to actually respond
	if (GHandleNoticesImmediately || !GUndo)
	{
		HandleAccumulatedNotices();
	}
#endif	  // USE_USD_SDK
}

void AUsdStageActor::HandleAccumulatedNotices()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::HandleAccumulatedNotices);

#if USE_USD_SDK

	if (AccumulatedInfoChanges.Num() == 0 && AccumulatedResyncChanges.Num() == 0 && !bLayerReloaded)
	{
		return;
	}

	const UE::FUsdStage& Stage = GetOrOpenUsdStage();
	if (!Stage)
	{
		return;
	}

	// If the stage was closed in a big transaction (e.g. undo open) a random UObject may be transacting before us and triggering USD changes,
	// and the UE::FUsdStage will still be opened and valid (even though we intend on closing/changing it when we transact). It could be
	// problematic/wasteful if we responded to those notices, so just early out here. We can do this check because our RootLayer property will already
	// have the new value
	{
		const UE::FUsdStage& BaseStage = GetBaseUsdStage();
		const UE::FSdfLayer& StageRoot = BaseStage.GetRootLayer();
		if (!StageRoot)
		{
			return;
		}

		if (!FUsdStageActorImpl::DoesPathPointToLayer(RootLayer.FilePath, StageRoot))
		{
			return;
		}
	}

	if (!AssetCache)
	{
		SetupAssetCacheIfNeeded();
	}
	if (!ensure(AssetCache))
	{
		// This should never happen now: We should always be able to find an asset cache
		CloseUsdStage();
		return;
	}
	UUsdAssetCache3::FUsdScopedReferencer ScopedReferencer{AssetCache, this};

	// Mark the level as dirty since we received a notice about our stage having changed in some way.
	// The main goal of this is to trigger the "save layers" dialog if we then save the UE level
	const bool bAlwaysMarkDirty = true;
	Modify(bAlwaysMarkDirty);

	// We may update our levelsequence objects (tracks, moviescene, sections, etc.) due to these changes. We definitely don't want to write anything
	// back to USD when these objects change though.
	FScopedBlockMonitoringChangesForTransaction BlockMonitoring{LevelSequenceHelper};

	bool bHasResync = AccumulatedResyncChanges.Num() > 0;

	// If any layer changed we'll need to regenerate the LevelSequences for sure (to make sure we update subsequence tracks to match the stage).
	// We don't have to worry about actually forcing a repopulate: Adding/removing/reloading layers always emits a root resync anyway, which will
	// already naturally repopulate the level sequences
	bool bNeedsAnimationReload = bLayerReloaded;

	// The most important thing here is to iterate in parent to child order, so build SortedPrimsChangedList
	TMap<UE::FSdfPath, bool> SortedPrimsChangedList;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FillSortedPrimsChangedList);

		for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& InfoChange : AccumulatedInfoChanges)
		{
			UE::FSdfPath ObjectPath{*InfoChange.Key};

			UE::FSdfPath PrimPath = ObjectPath.GetAbsoluteRootOrPrimPath().StripAllVariantSelections();

			FString PropertyName;
			if (ObjectPath.IsPropertyPath())
			{
				PropertyName = ObjectPath.GetName();
			}

			// Upgrade these to resync so that the prim twins are regenerated, which clears all the existing
			// animation tracks and adds new ones, automatically re-baking to control rig
			bool bIsResync = false;
			static const TSet<FString> PrimResyncProperties = {
				*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigPath),
				*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig),
				*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReduceKeys),
				*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReductionTolerance),
				// For now we need to do this as we need to refresh the material slot info on the info cache if these
				// update... this is of course way too aggressive, although it's unlikely people will be manually editing these.
				// TODO: More nuanced info cache updates
				UnrealIdentifiers::PrimvarsDisplayColor,
				UnrealIdentifiers::PrimvarsDisplayOpacity,
				UnrealIdentifiers::DoubleSided,
				// When we change these UsdGeomModelAPI attributes we may need to create a new component type
				// for the prim (as it may now need/stop needing an alternate draw mode component)
				UnrealIdentifiers::ModelDrawMode,
				UnrealIdentifiers::ModelApplyDrawMode,
				// Collapsing changes need to resync to build the new collapsed/uncollapsed meshes
				*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealCollapsingAttr),
				// Physics collision attribute change needs to rebuild the collision shape
				*UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->physicsCollisionEnabled),
				*UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->physicsApproximation),
				// We put all of the audio info directly on the section, so if any of these change then we need to re-add the prim
				// to the sequencer, and potentially generate a new audio asset
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->filePath),
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->auralMode),
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->playbackMode),
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->startTime),
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->endTime),
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->mediaOffset),
				*UsdToUnreal::ConvertToken(pxr::UsdMediaTokens->gain)
			};
			if (PrimPath.IsAbsoluteRootOrPrimPath() && PrimResyncProperties.Contains(PropertyName))
			{
				bIsResync = true;
				bHasResync = true;
				bNeedsAnimationReload = true;
			}

			// Upgrade some info changes into resync changes
			for (const UsdUtils::FSdfChangeListEntry& ObjectChange : InfoChange.Value)
			{
				// This is in charge of resyncing components (and so calling CreateComponents instead of just
				// UpdateComponents) whenever we change material assignments. This is important because
				// CreateComponents is where we set our material overrides
				if (ObjectChange.Flags.bDidChangeRelationshipTargets)
				{
					bIsResync = true;
					bHasResync = true;

					// If the material that was changed was on a geom subset, we need to instead pretend it
					// happened to its parent Mesh prim instead. If the mesh is collapsed this won't matter,
					// but if it's not collapsed we currently need this to make sure we resync the static mesh
					// component. The Mesh prim will declare that the subset is its aux prim, but we don't propagate
					// resyncs from aux to main prims now, so even if we resynced the UsdGeomSubset the Mesh
					// would not resync.
					// TODO: Better way of handling material override updates. Ideally we wouldn't be resyncing
					// the Mesh prim just to update material overrides...
					UE::FUsdPrim ChangedPrim = Stage.GetPrimAtPath(PrimPath);
					if (ChangedPrim && ChangedPrim.IsA(TEXT("GeomSubset")))
					{
						PrimPath = PrimPath.GetParentPath();
					}

					continue;
				}

				// Some stage info should trigger some resyncs because they should trigger reparsing of geometry
				if (PrimPath.IsAbsoluteRootPath())
				{
					for (const UsdUtils::FAttributeChange& FieldChange : ObjectChange.FieldChanges)
					{
						static const TSet<FString> StageResyncProperties = {TEXT("metersPerUnit"), TEXT("upAxis")};
						if (StageResyncProperties.Contains(FieldChange.Field))
						{
							bIsResync = true;
							bHasResync = true;
							break;
						}
					}

					// Any sublayer change (even offsets) means we need to regenerate our LevelSequence to add (or shift)
					// the corresponding subsequences
					if (ObjectChange.SubLayerChanges.Num() > 0)
					{
						bNeedsAnimationReload = true;
					}
				}
			}

			// We may need the full spec path with variant selections later, but for traversal and retrieving prims from the stage we always need
			// the prim path without any variant selections in it (i.e. GetPrimAtPath("/Root{Varset=Var}Child") doesn't work, we need
			// GetPrimAtPath("/Root/Child")), and USD sometimes emits changes with the variant selection path (like during renames).
			SortedPrimsChangedList.Add(PrimPath, bIsResync);
		}
	}
	// Do Resyncs after so that they overwrite pure info changes if we have any
	for (const TPair<FString, TArray<UsdUtils::FSdfChangeListEntry>>& ResyncChange : AccumulatedResyncChanges)
	{
		UE::FSdfPath PrimPath = UE::FSdfPath(*ResyncChange.Key).GetAbsoluteRootOrPrimPath().StripAllVariantSelections();

		const bool bIsResync = true;
		SortedPrimsChangedList.Add(PrimPath, bIsResync);
	}

	// During PIE, the PIE and the editor world will respond to notices. We have to prevent any PIE
	// objects from being added to the transaction however, or else it will be discarded when finalized.
	// We need to keep the transaction, or else we may end up with actors outside of the transaction
	// system that want to use assets that will be destroyed by it on an undo.
	// Note that we can't just make the spawned components/assets nontransactional because the PIE world will transact too
	TOptional<TGuardValue<ITransaction*>> SuppressTransaction;
	if (this->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor))
	{
		SuppressTransaction.Emplace(GUndo, nullptr);
	}

	FScopedSlowTask RefreshStageTask(4, LOCTEXT("ProcessingUSDStageUpdates", "Processing USD Stage updates"));
	RefreshStageTask.MakeDialogDelayed(0.25f);

	FScopedUsdMessageLog ScopedMessageLog;

	if (BBoxCache.IsValid() && bHasResync)
	{
		BBoxCache->Clear();
	}

	enum class EPrimUpdateType : uint8
	{
		MaterialBind,	 // "Weaker" than an info change: We just need to make sure we update material overrides for this prim's component
		Info,			 // An attribute/metadata value changed. We may need to regenerate assets, but at most update components for that one prim
		Resync,			 // Drastic change. We need to regenerate all assets and components for the prim's entire subtree
	};

	TFunction<void(TMap<UE::FSdfPath, EPrimUpdateType>&)> SortAndCleanPrimsToUpdate = [](TMap<UE::FSdfPath, EPrimUpdateType>& InOutMap)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SortAndCleanPrimsToUpdate);

		// Force traversal of parent before child (not needed for cleaning up, but very much needed later for reloading assets/components)
		InOutMap.KeySort(
			[](const UE::FSdfPath& A, const UE::FSdfPath& B) -> bool
			{
				return A < B;
			}
		);

		// Strip child paths for resyncs as processing a parent means we already process the children anyway.
		// This is not the same for info changes: We may have an info change for a parent and child component
		// in the same change block, and we really want to call UpdateComponents for both then
		TMap<UE::FSdfPath, EPrimUpdateType> CleanedPairs;
		CleanedPairs.Reserve(InOutMap.Num());
		TSet<UE::FSdfPath> ResyncedPaths;
		ResyncedPaths.Reserve(InOutMap.Num());
		for (const TPair<UE::FSdfPath, EPrimUpdateType>& Pair : InOutMap)
		{
			const UE::FSdfPath& ThisPrim = Pair.Key;
			bool bIsResync = Pair.Value == EPrimUpdateType::Resync;

			bool bRemoveThisPath = false;

			// Note: UE::FSdfPath::GetPrefixes() does not return the pseudoroot path "/"!
			for (const UE::FSdfPath& Prefix : ThisPrim.GetPrefixes())
			{
				if (ResyncedPaths.Contains(Prefix))
				{
					bRemoveThisPath = true;
					break;
				}
			}
			if (bRemoveThisPath)
			{
				continue;
			}

			CleanedPairs.Add(Pair);

			if (bIsResync)
			{
				ResyncedPaths.Add(ThisPrim);

				// Resyncing the root prim automatically means we have to resync the entire stage: There is no
				// point checking for anything else. Since we sort the keys this should always come first, if
				// present, so we can just break here.
				//
				// Note that this is how we reload stages, so it should happen relatively often
				if (ThisPrim == UE::FSdfPath::AbsoluteRootPath())
				{
					break;
				}
			}
		}
		Swap(CleanedPairs, InOutMap);
	};

	// Traverses the info caches to find out which prims we need to update
	TFunction<void(const UE::FSdfPath&, EPrimUpdateType, TMap<UE::FSdfPath, EPrimUpdateType>&, TMap<UE::FSdfPath, EPrimUpdateType>&)>
		RecursiveCollectPrimsToUpdate;
	RecursiveCollectPrimsToUpdate = [this, &RecursiveCollectPrimsToUpdate, &Stage](
										const UE::FSdfPath& PrimPath,
										EPrimUpdateType UpdateType,
										TMap<UE::FSdfPath, EPrimUpdateType>& OutPrimsToUpdate,
										TMap<UE::FSdfPath, EPrimUpdateType>& InOutVisitedPaths
									)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RecursiveCollectPrimsToUpdate);

		// Allow revisiting a prim if we're resyncing: This is a cheap way of making sure we mark the
		// resynced prims as resynced, or else we may recurse into them while traversing another prim and preemptively
		// mark them as visited, and because we always recurse with resync=false we'd assume those aren't resyncs.
		// By that same reason this shouldn't be that expensive, as we'll only ever potentially revisit the prims
		// that are the actual roots of the resyncs
		EPrimUpdateType* LastVisitedUpdateType = InOutVisitedPaths.Find(PrimPath);
		if (LastVisitedUpdateType && (uint8)*LastVisitedUpdateType >= (uint8)UpdateType)
		{
			return;
		}
		InOutVisitedPaths.Add(PrimPath, UpdateType);

		// In some cases USD sends us notices about prims that don't exist anymore: If you rename X to Y,
		// both X and Y will be on the notice change list, even though X doesn't exist on the stage anymore.
		// It's easy to just ignore those here by doing this, but we could also pay attention to the flags
		// on the notice and try to guess if a rename took place, if needed
		if (UsdInfoCache->GetInner().ContainsInfoAboutPrim(PrimPath))
		{
			// We always want the unwound path here. We'll take care to only ever register main prims that are
			// themselves uncollapsed or collapsed roots, but there's nothing stopping the user from manually
			// modifying directly a collapsed prim that is not an aux prim of it's collapsed root (e.g. some
			// parent prim of a point instancer prototype). If we retrieved a main prim for a prim like that,
			// we'd only get that prim itself, and assume we need to spawn assets/components for it (which we
			// really don't if it's collapsed)
			UE::FSdfPath UnwoundPath = UsdInfoCache->GetInner().UnwindToNonCollapsedPath(PrimPath, ECollapsingType::Assets);
			EPrimUpdateType& UnwoundUpdateType = OutPrimsToUpdate.FindOrAdd(UnwoundPath);
			UnwoundUpdateType = (EPrimUpdateType)FMath::Max((uint8)UnwoundUpdateType, (uint8)UpdateType);
		}

		// We don't need to recurse via material bind links: Nothing else is affected by a mesh component refreshing
		// its material overrides in response to an original material prim update
		if (UpdateType == EPrimUpdateType::MaterialBind)
		{
			return;
		}

		// Imagine we have a stage like this:
		// 		/parent/child1
		// 		/other
		// And "child1" is marked as an aux prim for "other". What happens if we resync "parent"? Since a resync means
		// the subtree is arbitrarily rebuilt, it means we probably want to update "other" too, which is what this does.
		// Note that parent and child could be fully independent, uncollapsed prims, without main/aux links between them.
		// We have to do this on both the stage and old info cache because the change may also have meant that aux/main links
		// have been modified (i.e. "other" could depend on "child" only now, or only on the old state of the stage, but
		// we'll still have those assets on the UE level either way, so we need to refresh them)
		if (UpdateType == EPrimUpdateType::Resync)
		{
			const TArray<UE::FSdfPath>& NewChildren = UsdInfoCache->GetInner().GetChildren(PrimPath);
			for (const UE::FSdfPath& Child : NewChildren)
			{
				RecursiveCollectPrimsToUpdate(Child, EPrimUpdateType::Info, OutPrimsToUpdate, InOutVisitedPaths);
			}
		}

		TSet<UE::FSdfPath> NewMainPrims = UsdInfoCache->GetInner().GetMainPrims(PrimPath);
		TSet<UE::FSdfPath> NewMaterialUsers = UsdInfoCache->GetInner().GetMaterialUsers(PrimPath);

		OutPrimsToUpdate.Reserve(OutPrimsToUpdate.Num() + NewMainPrims.Num() + NewMaterialUsers.Num());

		for (const UE::FSdfPath& NewPrimPath : NewMainPrims)
		{
			// If our original USD notice resyncs PrimPath, its subtree will need to be rebuilt, yes,
			// but external prims that depend on prim path (its "main prims") won't need to be *recursively* resynced.
			// Their hierarchies are fine, they just need to be updated to the fact that PrimPath changed. That is
			// at most a component update, or regenerating the asset for that particular main prim, but it's entire
			// hierarchy doesn't need to be rebuilt
			RecursiveCollectPrimsToUpdate(NewPrimPath, EPrimUpdateType::Info, OutPrimsToUpdate, InOutVisitedPaths);
		}
		for (const UE::FSdfPath& NewPrimPath : NewMaterialUsers)
		{
			RecursiveCollectPrimsToUpdate(NewPrimPath, EPrimUpdateType::MaterialBind, OutPrimsToUpdate, InOutVisitedPaths);
		}
	};

	// Collect all the paths to update from the old info cache
	TMap<UE::FSdfPath, EPrimUpdateType> PrimsToUpdate;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CollectOldPathsToUpdate);

		PrimsToUpdate.Reserve(SortedPrimsChangedList.Num());

		// Recursively append main prims to the list of PrimsToUpdate
		TMap<UE::FSdfPath, EPrimUpdateType> VisitedPaths;
		for (const TPair<UE::FSdfPath, bool>& PrimChangedInfo : SortedPrimsChangedList)
		{
			const UE::FSdfPath PrimPath = PrimChangedInfo.Key;
			const bool bIsResync = PrimChangedInfo.Value;

			// Note how we're not modifying SortedPrimsChangedList in-place and are instead adding to a new PrimsToUpdate list.
			// The intent is that we really only want to process uncollapsed/collapse root main prims, but what is actually on these notices is up to
			// USD, and could have anything
			RecursiveCollectPrimsToUpdate(PrimPath, bIsResync ? EPrimUpdateType::Resync : EPrimUpdateType::Info, PrimsToUpdate, VisitedPaths);
		}

		SortAndCleanPrimsToUpdate(PrimsToUpdate);
	}

	// Rebuild info cache if needed
	if (UsdInfoCache && bHasResync)
	{
		// The prim path doesn't matter here, it's only used for fetching the parent component (not used on the info cache rebuild)
		TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(this, TEXT("/"));

		TArray<UE::FSdfPath> ResyncPaths;
		ResyncPaths.Reserve(SortedPrimsChangedList.Num());
		for (const TPair<UE::FSdfPath, bool>& Pair : SortedPrimsChangedList)
		{
			if (Pair.Value)
			{
				ResyncPaths.Add(Pair.Key);
			}
		}

		ResyncedPrimsForThisTransaction = ResyncPaths;
		UsdInfoCache->GetInner().RebuildCacheForSubtrees(ResyncPaths, TranslationContext.Get());

		// Append the paths to update from the rebuilt info cache
		//
		// Note: We don't *reset* prims to update, because whatever assets/components we cleaned up we may also need to
		// regenerate. Alternatively, any *new* asset/component that we may end up generating also requires
		// that we cleanup the old asset/component in order to display it.
		//
		// The fact that it's a TMap and SortAndCleanPrimsToUpdate will prevent us from doing any extra
		// work anyway
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CollectNewPathsToUpdate);

			TMap<UE::FSdfPath, EPrimUpdateType> VisitedPaths;
			for (const TPair<UE::FSdfPath, bool>& PrimChangedInfo : SortedPrimsChangedList)
			{
				const UE::FSdfPath PrimPath = PrimChangedInfo.Key;
				const bool bIsResync = PrimChangedInfo.Value;
				RecursiveCollectPrimsToUpdate(PrimPath, bIsResync ? EPrimUpdateType::Resync : EPrimUpdateType::Info, PrimsToUpdate, VisitedPaths);
			}
			SortAndCleanPrimsToUpdate(PrimsToUpdate);
		}
	}

	if (bHasResync)
	{
		FUsdStageActorImpl::DeselectActorsAndComponents(this);
	}

	bool bHasLoadedOrAbandonedAssets = false;

	RefreshStageTask.EnterProgressFrame();
	FScopedSlowTask CleanUpAssetsTask(PrimsToUpdate.Num(), LOCTEXT("CleaningUpAssets", "Cleaning up old assets"));
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CleaningUpAssets);

		for (const TPair<UE::FSdfPath, EPrimUpdateType>& PrimChangedInfo : PrimsToUpdate)
		{
			CleanUpAssetsTask.EnterProgressFrame();

			const UE::FSdfPath& PrimPath = PrimChangedInfo.Key;
			const bool bIsResync = PrimChangedInfo.Value == EPrimUpdateType::Resync;

			const bool bForEntireSubtree = bIsResync;
			bHasLoadedOrAbandonedAssets |= UnloadAssets(PrimPath, bForEntireSubtree);
		}
	}

	RefreshStageTask.EnterProgressFrame();
	FScopedSlowTask CleanUpComponentsTask(PrimsToUpdate.Num(), LOCTEXT("CleaningUpComponents", "Cleaning up actors and components"));
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CleaningUpComponents);

		for (const TPair<UE::FSdfPath, EPrimUpdateType>& PrimChangedInfo : PrimsToUpdate)
		{
			CleanUpComponentsTask.EnterProgressFrame();

			const UE::FSdfPath PrimPath = PrimChangedInfo.Key;
			const bool bIsResync = PrimChangedInfo.Value == EPrimUpdateType::Resync;

			if (bIsResync && PrimPath.IsAbsoluteRootOrPrimPath())
			{
				if (UUsdPrimTwin* UsdPrimTwin = GetRootPrimTwin()->Find(PrimPath.GetString()))
				{
					UsdPrimTwin->Clear();
				}
			}
		}
	}

	// Recreate our LevelSequences before we regenerate components and want to add bindings/tracks
	// back onto it
	bool bSequenceWasOpened = false;
	if (bNeedsAnimationReload)
	{
		bSequenceWasOpened = RegenerateLevelSequence();
	}

	// Reset our translated prototypes only here, and reuse them for all individual changes. This because some types of operations
	// (e.g. reloading a reference used in an instanceable) will cause USD to emit a resync notice for every single instance
	// of the prototype: By keeping track of which prototypes we translated across all those changes we can do the actual translation
	//  only once
	if (UsdInfoCache)
	{
		UsdInfoCache->GetInner().ResetTranslatedPrototypes();
	}

	RefreshStageTask.EnterProgressFrame();
	FScopedSlowTask RegenerateAssetsTask(PrimsToUpdate.Num(), LOCTEXT("RegeneratingAssets", "Regenerating assets"));
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RegeneratingAssets);

		for (const TPair<UE::FSdfPath, EPrimUpdateType>& PrimChangedInfo : PrimsToUpdate)
		{
			RegenerateAssetsTask.EnterProgressFrame();

			const UE::FSdfPath& PrimPath = PrimChangedInfo.Key;
			const bool bIsResync = PrimChangedInfo.Value == EPrimUpdateType::Resync;

			UE::FUsdPrim PrimToUpdate = Stage.GetPrimAtPath(PrimPath);

			// It's OK to not have info about a prim if it's an old prim that only exists on the old info cache.
			// If the new info cache has info about this prim then it must exist on the stage right now
			if (!UsdInfoCache->GetInner().ContainsInfoAboutPrim(PrimPath))
			{
				ensure(!PrimToUpdate);
				continue;
			}
			ensure(PrimToUpdate);

			TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(
				this,
				PrimPath.GetString()
			);

			bool bThisPrimLoadedAssets = false;
			if (bIsResync)
			{
				bThisPrimLoadedAssets |= LoadAssets(*TranslationContext, PrimToUpdate);
			}
			else
			{
				bThisPrimLoadedAssets |= LoadAsset(*TranslationContext, PrimToUpdate);
			}
			bHasLoadedOrAbandonedAssets |= bThisPrimLoadedAssets;
		}
	}

	RefreshStageTask.EnterProgressFrame();
	FScopedSlowTask RegenerateComponentsTask(PrimsToUpdate.Num(), LOCTEXT("RegeneratingComponents", "Regenerating components"));
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RegeneratingComponents);

		for (const TPair<UE::FSdfPath, EPrimUpdateType>& PrimChangedInfo : PrimsToUpdate)
		{
			RegenerateComponentsTask.EnterProgressFrame();

			const UE::FSdfPath PrimPath = PrimChangedInfo.Key;
			const bool bIsResync = PrimChangedInfo.Value == EPrimUpdateType::Resync;

			if (!UsdInfoCache->GetInner().ContainsInfoAboutPrim(PrimPath))
			{
				continue;
			}

			TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(
				this,
				PrimPath.GetString()
			);

			UpdatePrim(PrimPath, bIsResync, *TranslationContext);
			TranslationContext->CompleteTasks();
		}
	}

	// Separate pass because we need to update the stage editor even if we don't have info about this prim
	// anymore, as that's how it refreshes whenever we delete a prim
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OnPrimChangedBroadcast);

		for (const TPair<UE::FSdfPath, EPrimUpdateType>& PrimChangedInfo : PrimsToUpdate)
		{
			const bool bIsResync = PrimChangedInfo.Value == EPrimUpdateType::Resync;
			OnPrimChanged.Broadcast(PrimChangedInfo.Key.GetString(), bIsResync);
		}
	}

	if (bHasResync)
	{
		FUsdStageActorImpl::RepairExternalSequencerBindings();
	}

	if (bSequenceWasOpened)
	{
		OpenLevelSequence();
	}

#if WITH_EDITOR
	if (GIsEditor && GEditor && !IsGarbageCollecting())	   // Make sure we're not in standalone either
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BroadcastingEditorEvents);

		if (bHasResync)
		{
			GEditor->BroadcastLevelActorListChanged();
		}

		GEditor->RedrawLevelEditingViewports();
	}
#endif	  // WITH_EDITOR

#endif	  // USE_USD_SDK

	AccumulatedInfoChanges.Reset();
	AccumulatedResyncChanges.Reset();
	bLayerReloaded = false;
}

USDSTAGE_API void AUsdStageActor::Reset()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::Reset);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Modify);
		Modify();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Reset);
		Super::Reset();
	}

	bool bUnloadIfNeeded = true;
	CloseUsdStage(bUnloadIfNeeded);

	Time = 0.f;
	RootLayer.FilePath.Empty();
}

void AUsdStageActor::StopListeningToUsdNotices()
{
	IsBlockedFromUsdNotices.Increment();
}

void AUsdStageActor::ResumeListeningToUsdNotices()
{
	IsBlockedFromUsdNotices.Decrement();
}

bool AUsdStageActor::IsListeningToUsdNotices() const
{
	return IsBlockedFromUsdNotices.GetValue() == 0;
}

void AUsdStageActor::StopMonitoringLevelSequence()
{
	LevelSequenceHelper.StopMonitoringChanges();
}

void AUsdStageActor::ResumeMonitoringLevelSequence()
{
	LevelSequenceHelper.StartMonitoringChanges();
}

void AUsdStageActor::BlockMonitoringLevelSequenceForThisTransaction()
{
	LevelSequenceHelper.BlockMonitoringChangesForThisTransaction();
}

UUsdPrimTwin* AUsdStageActor::GetOrCreatePrimTwin(const UE::FSdfPath& UsdPrimPath)
{
	const FString PrimPath = UsdPrimPath.GetString();
	const FString ParentPrimPath = UsdPrimPath.GetParentPath().GetString();

	UUsdPrimTwin* RootTwin = GetRootPrimTwin();
	UUsdPrimTwin* UsdPrimTwin = RootTwin->Find(PrimPath);
	UUsdPrimTwin* ParentUsdPrimTwin = RootTwin->Find(ParentPrimPath);

	const UE::FUsdPrim Prim = GetOrOpenUsdStage().GetPrimAtPath(UsdPrimPath);

	if (!Prim)
	{
		return nullptr;
	}

	if (!ParentUsdPrimTwin)
	{
		ParentUsdPrimTwin = RootUsdTwin;
	}

	if (!UsdPrimTwin)
	{
		UsdPrimTwin = &ParentUsdPrimTwin->AddChild(*PrimPath);

		UsdPrimTwin->OnDestroyed.AddUObject(this, &AUsdStageActor::OnUsdPrimTwinDestroyed);
	}

	return UsdPrimTwin;
}

UUsdPrimTwin* AUsdStageActor::ExpandPrim(
	const UE::FUsdPrim& Prim,
	bool bResync,
	FUsdSchemaTranslationContext& TranslationContext,
	TOptional<bool> bParentHasAnimatedVisibility
)
{
	UUsdPrimTwin* UsdPrimTwin = nullptr;
#if USE_USD_SDK
	// "Active" is the non-destructive deletion used in USD. Sometimes when we rename/remove a prim in a complex stage it may remain in
	// an inactive state, but its otherwise effectively deleted.
	//
	// We check IsDefined() because we need to consider the possibility that we've been called directly for this prim
	// (e.g. when handling an update notice). During regular traversal when opening the stage, the Prim.GetFilteredChildren() call
	// within this same function will naturally strip all "pure over" prims
	if (!Prim || !Prim.IsActive() || !Prim.IsDefined())
	{
		return nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::ExpandPrim);

	UsdPrimTwin = GetOrCreatePrimTwin(Prim.GetPrimPath());

	if (!UsdPrimTwin)
	{
		return nullptr;
	}

	bool bExpandChildren = true;

	if (TranslationContext.bIsJustRepopulatingLevelSequence)
	{
		// For the repopulate, let's only visit the prim twins that already have children and so may actually have
		// components. We don't want to create brand new components here
		bExpandChildren = UsdPrimTwin->GetChildren().Num() > 0;
	}
	else if (!TranslationContext.bIsJustRepopulatingLevelSequence)
	{
		if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get()
																	.CreateTranslatorForSchema(TranslationContext.AsShared(), UE::FUsdTyped(Prim)))
		{
			if (bResync && !UsdPrimTwin->SceneComponent.IsValid())
			{
				UsdPrimTwin->SceneComponent = SchemaTranslator->CreateComponents();
			}
			else
			{
				USceneComponent* TwinSceneComponent = UsdPrimTwin->SceneComponent.Get();

				ObjectsToWatch.Remove(TwinSceneComponent);
				if (Prim.IsA(TEXT("Camera")))
				{
					if (ACineCameraActor* CameraActor = Cast<ACineCameraActor>(SceneComponent->GetOwner()))
					{
						ObjectsToWatch.Remove(CameraActor->GetCineCameraComponent());
					}
				}
				else if (Prim.IsA(TEXT("PointInstancer")))
				{
					for (const TObjectPtr<USceneComponent>& Child : TwinSceneComponent->GetAttachChildren())
					{
						UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(Child.Get());
						ObjectsToWatch.Remove(ISMComponent);
					}
				}
				SchemaTranslator->UpdateComponents(TwinSceneComponent);
			}

			bExpandChildren = bResync && !SchemaTranslator->CollapsesChildren(ECollapsingType::Components);
		}
	}

	// Check for parents with animated visibility.
	//
	// When opening the stage we'll propagate this down already, but we may be just updating a random prim in the middle
	// of the hierarchy from an update notice, so we may need to check our parents right here.
	// After we do this here we can propagate this value to our children though
	if (!bParentHasAnimatedVisibility.IsSet())
	{
		bool bHasVisibilityAnimationParent = false;
		UE::FUsdPrim ParentPrim = Prim.GetParent();
		while (ParentPrim && !ParentPrim.IsPseudoRoot())
		{
			if (UsdUtils::HasAnimatedVisibility(ParentPrim))
			{
				bHasVisibilityAnimationParent = true;
				break;
			}

			ParentPrim = ParentPrim.GetParent();
		}

		bParentHasAnimatedVisibility = bHasVisibilityAnimationParent;
	}

	if (bExpandChildren)
	{
		// Unfortunately if we have animated visibility we need to be ready to update the visibility
		// of all components that we spawned for child prims whenever this prim's visibility updates.
		// We can't just have this prim's FUsdGeomXformableTranslator::UpdateComponents ->
		// -> UsdToUnreal::ConvertXformable call use SetHiddenInGame recursively, because we may have
		// child prims that are themselves also invisible, and so their own subtrees should be invisible
		// even if this prim goes visible. Also keep in mind that technically we'll always update each
		// prim in the order that they are within PrimsToAnimate, but that order is not strictly enforced
		// to be e.g. a breadth first traversal on the prim tree or anything like this, so these updates
		// need to be order-independent, which means we really should add the entire subtree to the list
		// and have UpdateComponents called on all components.
		bParentHasAnimatedVisibility = bParentHasAnimatedVisibility.GetValue() || UsdUtils::HasAnimatedVisibility(Prim);

		USceneComponent* ContextParentComponent = TranslationContext.ParentComponent;

		if (UsdPrimTwin->SceneComponent.IsValid())
		{
			ContextParentComponent = UsdPrimTwin->SceneComponent.Get();
		}

		TGuardValue<USceneComponent*> ParentComponentGuard(TranslationContext.ParentComponent, ContextParentComponent);

		const bool bTraverseInstanceProxies = true;
		const TArray<UE::FUsdPrim> PrimChildren = Prim.GetFilteredChildren(bTraverseInstanceProxies);

		for (const UE::FUsdPrim& ChildPrim : PrimChildren)
		{
			ExpandPrim(ChildPrim, bResync, TranslationContext, bParentHasAnimatedVisibility);
		}
	}

	USceneComponent* TwinSceneComponent = UsdPrimTwin->SceneComponent.Get();
	if (TwinSceneComponent && !TranslationContext.bIsJustRepopulatingLevelSequence)
	{
#if WITH_EDITOR
		if (TwinSceneComponent->IsA<UGeometryCacheComponent>() || TwinSceneComponent->IsA<USkyLightComponent>())
		{
			TwinSceneComponent->PostEditChange();
		}
#endif	  // WITH_EDITOR

		if (!TwinSceneComponent->IsRegistered())
		{
			TwinSceneComponent->RegisterComponent();
		}

		ObjectsToWatch.Add(TwinSceneComponent, UsdPrimTwin->PrimPath);

		// Make sure we monitor direct changes to camera properties on the component as well as the actor
		if (Prim.IsA(TEXT("Camera")))
		{
			if (ACineCameraActor* CameraActor = Cast<ACineCameraActor>(TwinSceneComponent->GetOwner()))
			{
				ObjectsToWatch.Add(CameraActor->GetCineCameraComponent(), UsdPrimTwin->PrimPath);
			}
		}
		else if (Prim.IsA(TEXT("PointInstancer")))
		{
			// Collect all the known prototype paths for this PointInstancer
			UsdUtils::FCaseSensitiveStringSet PrototypePaths = FUsdStageActorImpl::GetPointInstancerPrototypes(Prim);

			const TArray<TObjectPtr<USceneComponent>>& ChildComponents = TwinSceneComponent->GetAttachChildren();
			for (const TObjectPtr<USceneComponent>& Child : ChildComponents)
			{
				UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(Child.Get());
				if (!ISMComponent)
				{
					continue;
				}

				UStaticMesh* ISMMesh = ISMComponent->GetStaticMesh();
				if (!ISMMesh)
				{
					continue;
				}

				UUsdAssetUserData* UserData = UsdUnreal::ObjectUtils::GetAssetUserData(ISMMesh);
				if (!UserData)
				{
					continue;
				}

				for (const FString& Path : UserData->PrimPaths)
				{
					if (PrototypePaths.Contains(Path))
					{
						ObjectsToWatch.Add(ISMComponent, Path);
					}
				}
			}
		}
	}

	// Check if the prim should have Sequencer tracks or not
	bool bIsAnimated = bParentHasAnimatedVisibility.GetValue();

	// We know we're animated if we have skeletal animation of course
	if (!bIsAnimated)
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(TwinSceneComponent))
		{
			if (SkeletalMeshComponent->AnimationData.AnimToPlay)
			{
				bIsAnimated = true;
			}
		}
	}

	// Always consider SpatialAudio prims as animated so that we can create LevelSequence tracks for the audio itself.
	// We exclusively handle the audio stuff via the Sequencer and LevelSequence tracks because there's no way to play audio via Time animation,
	// and the audio component is not meant to be a fully featured audio player with start/end play times and animated volume controls. In
	// other words, if we placed our SoundWave asset on the component, the audio component would instantly play it when going into PIE, which is
	// not what we want. The audio component and actor are only really used for their transforms on the level whenever we'ret trying to play
	// spatial audio
	bIsAnimated = bIsAnimated || Prim.IsA(TEXT("SpatialAudio")) || UsdUtils::IsAnimated(Prim);

	TOptional<bool> bHasAnimatedBounds;
	if (EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(Prim); DrawMode != EUsdDrawMode::Default)
	{
		const bool bUseExtentsHint = true;
		const bool bIgnoreVisibility = false;
		bHasAnimatedBounds = UsdUtils::HasAnimatedBounds(Prim, static_cast<EUsdPurpose>(PurposesToLoad), bUseExtentsHint, bIgnoreVisibility);

		const bool bDefault = false;
		if (bHasAnimatedBounds.Get(bDefault))
		{
			bIsAnimated = true;

			// Mark the component as animated right away because HasAnimatedBounds is expensive to call and
			// we don't want to have to re-do it when creating the component
			if (USceneComponent* PrimTwinsComponent = UsdPrimTwin->SceneComponent.Get())
			{
				PrimTwinsComponent->SetMobility(EComponentMobility::Movable);
			}
		}
	}

	const FString HashedPrimPath = UsdUtils::AddHashToPrimPath(UsdPrimTwin->PrimPath);

	if (bIsAnimated)
	{
		const bool bForceVisibilityTracks = bParentHasAnimatedVisibility.GetValue();
		LevelSequenceHelper.AddPrim(*UsdPrimTwin, bForceVisibilityTracks, bHasAnimatedBounds);
		PrimsToAnimate.Add(HashedPrimPath);
	}
	else
	{
		PrimsToAnimate.Remove(HashedPrimPath);
		LevelSequenceHelper.RemovePrim(*UsdPrimTwin);
	}

	// Setup Control Rig tracks if we need to. This must be done after adding regular skeletal animation tracks
	// if we have any as if will properly deactivate them like the usual "Bake to Control Rig" workflow.
	if (Prim.IsA(TEXT("Skeleton")))
	{
		UE::FUsdPrim PrimWithSchema;
		if (UsdUtils::PrimHasSchema(Prim, UnrealIdentifiers::ControlRigAPI))
		{
			PrimWithSchema = Prim;
		}

		if (PrimWithSchema)
		{
			LevelSequenceHelper.UpdateControlRigTracks(*UsdPrimTwin);

			// If our prim wasn't originally considered animated and we just added a new track, it should be
			// considered animated too, so lets add it to the proper locations. This will also ensure that
			// we can close the sequencer after creating a new animation in this way and see it animate on
			// the level
			PrimsToAnimate.Add(HashedPrimPath);
			LevelSequenceHelper.AddPrim(*UsdPrimTwin);

			// Prevent register/unregister spam when calling FUsdGeomXformableTranslator::UpdateComponents later
			// during sequencer animation (which can cause the Sequencer UI to glitch out a bit)
			UsdPrimTwin->SceneComponent->SetMobility(EComponentMobility::Movable);
		}
	}

#endif	  // USE_USD_SDK
	return UsdPrimTwin;
}

void AUsdStageActor::UpdatePrim(const UE::FSdfPath& InUsdPrimPath, bool bResync, FUsdSchemaTranslationContext& TranslationContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::UpdatePrim);

	FScopedSlowTask SlowTask(1.f, LOCTEXT("UpdatingUSDPrim", "Updating USD Prim"));
	SlowTask.MakeDialogDelayed(0.25f);
	SlowTask.EnterProgressFrame();

	UE::FSdfPath UsdPrimPath = InUsdPrimPath;

	if (!UsdPrimPath.IsAbsoluteRootOrPrimPath())
	{
		UsdPrimPath = UsdPrimPath.GetAbsoluteRootOrPrimPath();
	}

	if (UsdPrimPath.IsAbsoluteRootOrPrimPath())
	{
		UE::FUsdPrim PrimToExpand = GetOrOpenUsdStage().GetPrimAtPath(UsdPrimPath);
		ExpandPrim(PrimToExpand, bResync, TranslationContext);
	}
}

const UE::FUsdStage& AUsdStageActor::GetUsdStage() const
{
	return IsolatedStage ? IsolatedStage : UsdStage;
}

const UE::FUsdStage& AUsdStageActor::GetBaseUsdStage() const
{
	return UsdStage;
}

const UE::FUsdStage& AUsdStageActor::GetIsolatedUsdStage() const
{
	return IsolatedStage;
}

void AUsdStageActor::SetUsdStage(const UE::FUsdStage& NewStage)
{
	if (UsdStage == NewStage)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	// Fire this in case CloseUsdStage is not going to
	if (!UsdStage)
	{
		OnPreStageChanged.Broadcast();
	}

	UnloadUsdStage();
	CloseUsdStage();

	FString RelativeFilePath = NewStage.GetRootLayer().GetIdentifier();
#if USE_USD_SDK
	if (!RelativeFilePath.IsEmpty() && !FPaths::IsRelative(RelativeFilePath) && !RelativeFilePath.StartsWith(UnrealIdentifiers::IdentifierPrefix))
	{
		RelativeFilePath = UsdUtils::MakePathRelativeToProjectDir(RelativeFilePath);
	}
#endif	  // USE_USD_SDK
	RootLayer.FilePath = RelativeFilePath;

	UsdStage = NewStage;
	IsolatedStage = UE::FUsdStage{};

	if (UsdStage)
	{
		UsdStage.SetEditTarget(UsdStage.GetRootLayer());

		UsdStage.SetInterpolationType(InterpolationType);

		UsdListener.Register(UsdStage);

#if USE_USD_SDK
		// Try loading a UE-state session layer if we can find one
		const bool bCreateIfNeeded = false;
		UsdUtils::GetUEPersistentStateSublayer(UsdStage, bCreateIfNeeded);
#endif	  // #if USE_USD_SDK
	}

	LoadUsdStage();
	OnStageChanged.Broadcast();
}

UE::FUsdStage& AUsdStageActor::GetOrOpenUsdStage()
{
	OpenUsdStage();

	return IsolatedStage ? IsolatedStage : UsdStage;
}

void AUsdStageActor::SetRootLayer(const FString& RootFilePath)
{
	FString RelativeFilePath = RootFilePath;
#if USE_USD_SDK
	if (!RelativeFilePath.IsEmpty() && !FPaths::IsRelative(RelativeFilePath) && !RelativeFilePath.StartsWith(UnrealIdentifiers::IdentifierPrefix))
	{
		RelativeFilePath = UsdUtils::MakePathRelativeToProjectDir(RootFilePath);
	}
#endif	  // USE_USD_SDK

	// See if we're talking about the stage that is already loaded
	if (UsdStage)
	{
		const UE::FSdfLayer& StageRootLayer = UsdStage.GetRootLayer();
		if (StageRootLayer)
		{
			if (FUsdStageActorImpl::DoesPathPointToLayer(RelativeFilePath, StageRootLayer))
			{
				return;
			}
		}
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	UnloadUsdStage();
	CloseUsdStage();
	RootLayer.FilePath = RelativeFilePath;

	// Don't call OpenUsdStage directly so that we can abort opening the stage in case the user cancels
	// out of the missing asset cache dialog
	const bool bOpenIfNeeded = true;
	LoadUsdStage(bOpenIfNeeded);

	// Do this here instead of on OpenUsdStage/LoadUsdStage as those also get called when changing any of
	// our properties, like render context, material purpose, etc.
	UsdUnreal::Analytics::CollectSchemaAnalytics(UsdStage, TEXT("Open"));
}

void AUsdStageActor::SetStageState(EUsdStageState NewState)
{
	if (NewState == StageState)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	StageState = NewState;
	if (StageState == EUsdStageState::Closed)
	{
		UnloadUsdStage();
		CloseUsdStage();
	}
	else if (StageState == EUsdStageState::Opened)
	{
		UnloadUsdStage();
		OpenUsdStage();
	}
	else if (StageState == EUsdStageState::OpenedAndLoaded)
	{
		// Don't call OpenUsdStage directly so that we can abort opening the stage in case the user cancels
		// out of the missing asset cache dialog
		const bool bOpenIfNeeded = true;
		LoadUsdStage(bOpenIfNeeded);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void AUsdStageActor::SetAssetCache(UUsdAssetCache2* NewCache)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void AUsdStageActor::SetUsdAssetCache(UUsdAssetCache3* NewCache)
{
	if (NewCache == AssetCache)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	if (AssetCache)
	{
		AssetCache->RemoveAllReferencerAssets(this);
		AssetCache->RequestDelayedAssetAutoCleanup();
	}

	AssetCache = NewCache;
}

void AUsdStageActor::SetInitialLoadSet(EUsdInitialLoadSet NewLoadSet)
{
	if (NewLoadSet == InitialLoadSet)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	InitialLoadSet = NewLoadSet;
	LoadUsdStage();
}

void AUsdStageActor::SetInterpolationType(EUsdInterpolationType NewType)
{
	if (NewType == InterpolationType)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	InterpolationType = NewType;
	LoadUsdStage();
}

void AUsdStageActor::SetGeometryCacheImport(EGeometryCacheImport ImportOption)
{
	if (ImportOption == GeometryCacheImport)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	GeometryCacheImport = ImportOption;
	LoadUsdStage();
}

void AUsdStageActor::SetUsePrimKindsForCollapsing(bool bUse)
{
	if (bUse == bUsePrimKindsForCollapsing)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	bUsePrimKindsForCollapsing = bUse;
	LoadUsdStage();
}

void AUsdStageActor::SetKindsToCollapse(int32 NewKindsToCollapse)
{
	const EUsdDefaultKind NewEnum = (EUsdDefaultKind)NewKindsToCollapse;
	EUsdDefaultKind Result = NewEnum;

	// If we're collapsing all 'model's, then we must collapse all of its derived kinds
	if (EnumHasAnyFlags(NewEnum, EUsdDefaultKind::Model))
	{
		Result |= (EUsdDefaultKind::Component | EUsdDefaultKind::Group | EUsdDefaultKind::Assembly);
	}

	// If we're collapsing all 'group's, then we must collapse all of its derived kinds
	if (EnumHasAnyFlags(NewEnum, EUsdDefaultKind::Group))
	{
		Result |= (EUsdDefaultKind::Assembly);
	}

	if ((int32)Result == KindsToCollapse)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	KindsToCollapse = (int32)Result;
	LoadUsdStage();
}

void AUsdStageActor::SetMergeIdenticalMaterialSlots(bool bMerge)
{
	if (bMerge == bMergeIdenticalMaterialSlots)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	bMergeIdenticalMaterialSlots = bMerge;
	LoadUsdStage();
}

void AUsdStageActor::SetShareAssetsForIdenticalPrims(bool bShare)
{
	if (bShare == bShareAssetsForIdenticalPrims)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	bShareAssetsForIdenticalPrims = bShare;
	LoadUsdStage();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void AUsdStageActor::SetReuseIdenticalAssets(bool bReuse)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void AUsdStageActor::SetCollapseTopLevelPointInstancers(bool bCollapse)
{
}

void AUsdStageActor::SetPurposesToLoad(int32 NewPurposesToLoad)
{
	if (NewPurposesToLoad == PurposesToLoad)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	PurposesToLoad = NewPurposesToLoad;
	if (BBoxCache.IsValid())
	{
		BBoxCache->SetIncludedPurposes(static_cast<EUsdPurpose>(PurposesToLoad));
	}
	LoadUsdStage();
}

void AUsdStageActor::SetNaniteTriangleThreshold(int32 NewNaniteTriangleThreshold)
{
	if (NewNaniteTriangleThreshold == NaniteTriangleThreshold)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	NaniteTriangleThreshold = NewNaniteTriangleThreshold;
	LoadUsdStage();
}

void AUsdStageActor::SetRenderContext(const FName& NewRenderContext)
{
	if (NewRenderContext == RenderContext)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	RenderContext = NewRenderContext;
	LoadUsdStage();
}

void AUsdStageActor::SetMaterialPurpose(const FName& NewMaterialPurpose)
{
	if (NewMaterialPurpose == MaterialPurpose)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	MaterialPurpose = NewMaterialPurpose;
	LoadUsdStage();
}

void AUsdStageActor::SetRootMotionHandling(EUsdRootMotionHandling NewHandlingStrategy)
{
	if (NewHandlingStrategy == RootMotionHandling)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	RootMotionHandling = NewHandlingStrategy;
	LoadUsdStage();
}

void AUsdStageActor::SetFallbackCollisionType(EUsdCollisionType NewCollisionType)
{
	if (NewCollisionType == FallbackCollisionType)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	FallbackCollisionType = NewCollisionType;
	LoadUsdStage();
}

void AUsdStageActor::SetSubdivisionLevel(int32 NewSubdivisionLevel)
{
	if (NewSubdivisionLevel == SubdivisionLevel)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	SubdivisionLevel = NewSubdivisionLevel;
	LoadUsdStage();
}

void AUsdStageActor::SetCollectMetadata(bool bNewCollectValue)
{
	if (bNewCollectValue == MetadataOptions.bCollectMetadata)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	MetadataOptions.bCollectMetadata = bNewCollectValue;
	LoadUsdStage();
}

void AUsdStageActor::SetCollectFromEntireSubtrees(bool bNewCollectValue)
{
	if (bNewCollectValue == MetadataOptions.bCollectFromEntireSubtrees)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	MetadataOptions.bCollectFromEntireSubtrees = bNewCollectValue;
	LoadUsdStage();
}

void AUsdStageActor::SetCollectOnComponents(bool bNewCollectValue)
{
	if (bNewCollectValue == MetadataOptions.bCollectOnComponents)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	MetadataOptions.bCollectOnComponents = bNewCollectValue;
	LoadUsdStage();
}

void AUsdStageActor::SetBlockedPrefixFilters(const TArray<FString>& NewFilters)
{
	if (NewFilters == MetadataOptions.BlockedPrefixFilters)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	MetadataOptions.BlockedPrefixFilters = NewFilters;
	LoadUsdStage();
}

void AUsdStageActor::SetInvertFilters(bool bNewInvertValue)
{
	if (bNewInvertValue == MetadataOptions.bInvertFilters)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	MetadataOptions.bInvertFilters = bNewInvertValue;
	LoadUsdStage();
}

float AUsdStageActor::GetTime() const
{
	return Time;
}

void AUsdStageActor::SetTime(float InTime)
{
	if (InTime == Time)
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	Time = InTime;
	if (BBoxCache.IsValid())
	{
		BBoxCache->SetTime(Time);
	}

	Refresh();
}

ULevelSequence* AUsdStageActor::GetLevelSequence() const
{
	return LevelSequence;
}

USceneComponent* AUsdStageActor::GetGeneratedComponent(const FString& PrimPath) const
{
	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();
	if (!CurrentStage || StageState != EUsdStageState::OpenedAndLoaded)
	{
		return nullptr;
	}

	// We can't query our UsdInfoCache with invalid paths, as we're using ensures to track when we miss the cache (which shouldn't ever happen)
	UE::FSdfPath UsdPath{*PrimPath};
	if (!CurrentStage.GetPrimAtPath(UsdPath))
	{
		return nullptr;
	}

	FString UncollapsedPath = PrimPath;
	if (UsdInfoCache)
	{
		UncollapsedPath = UsdInfoCache->GetInner().UnwindToNonCollapsedPath(UsdPath, ECollapsingType::Components).GetString();
	}

	if (UUsdPrimTwin* UsdPrimTwin = GetRootPrimTwin()->Find(UncollapsedPath))
	{
		return UsdPrimTwin->GetSceneComponent();
	}

	return nullptr;
}

TArray<UObject*> AUsdStageActor::GetGeneratedAssets(const FString& PrimPath) const
{
	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();
	if (!CurrentStage || StageState != EUsdStageState::OpenedAndLoaded)
	{
		return {};
	}

	// We can't query our UsdInfoCache with invalid paths, as we're using ensures to track when we miss the cache (which shouldn't ever happen)
	UE::FSdfPath UsdPath{*PrimPath};
	if (!CurrentStage.GetPrimAtPath(UsdPath))
	{
		return {};
	}

	if (!PrimLinkCache || !UsdInfoCache)
	{
		return {};
	}

	// Prefer checking the prim directly, but also check its collapsed root if it is collapsed.
	TArray<TWeakObjectPtr<UObject>> AssetsPtrs = PrimLinkCache->GetInner().GetAllAssetsForPrim(UsdPath);
	if (AssetsPtrs.Num() == 0 && UsdInfoCache->GetInner().IsPathCollapsed(UsdPath, ECollapsingType::Assets))
	{
		UsdPath = UsdInfoCache->GetInner().UnwindToNonCollapsedPath(UsdPath, ECollapsingType::Assets);
		AssetsPtrs = PrimLinkCache->GetInner().GetAllAssetsForPrim(UsdPath);
	}

	TArray<UObject*> Assets;
	Assets.Reserve(AssetsPtrs.Num());
	for (const TWeakObjectPtr<UObject>& Asset : AssetsPtrs)
	{
		Assets.Add(Asset.Get());
	}
	return Assets;
}

FString AUsdStageActor::GetSourcePrimPath(const UObject* Object) const
{
	UUsdPrimTwin* RootUsdPrimTwin = GetRootPrimTwin();

	const USceneComponent* Component = Cast<const USceneComponent>(Object);
	if (!Component)
	{
		// We always bind the root component and actor itself to the same prim anyway, so let's
		// just decay to the component in case we've been given an actor
		if (const AActor* Actor = Cast<AActor>(Object))
		{
			Component = Actor->GetRootComponent();
		}
	}

	if (Component && RootUsdTwin)
	{
		if (UUsdPrimTwin* UsdPrimTwin = RootUsdPrimTwin->Find(Component))
		{
			return UsdPrimTwin->PrimPath;
		}
	}
	else if (PrimLinkCache)
	{
		const TArray<UE::FSdfPath> FoundPaths = PrimLinkCache->GetInner().GetPrimsForAsset(Object);
		if (FoundPaths.Num() > 0)
		{
			return FoundPaths[0].GetString();
		}
	}

	return FString();
}

void AUsdStageActor::OpenUsdStage()
{
	// Early exit if stage is already opened, or if we shouldn't be opening anything anyway
	if (UsdStage || RootLayer.FilePath.IsEmpty() || StageState == EUsdStageState::Closed)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::OpenUsdStage);

	TOptional<TGuardValue<ITransaction*>> SuppressTransaction;
	if (GDiscardUndoBufferOnStageOpenClose)
	{
		SuppressTransaction.Emplace(GUndo, nullptr);
		RequestDelayedTransactorReset();
	}

	FScopedUsdMessageLog ScopedLog;

	FString AbsPath;
	if (!RootLayer.FilePath.StartsWith(UnrealIdentifiers::IdentifierPrefix) && FPaths::IsRelative(RootLayer.FilePath))
	{
		// The RootLayer property is marked as RelativeToGameDir, and FUsdStageViewModel::OpenStage will also give us paths relative to the project's
		// directory
		FString ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
		AbsPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(ProjectDir, RootLayer.FilePath));
	}
	else
	{
		AbsPath = RootLayer.FilePath;
	}

	if (UsdStage && (FPaths::IsSamePath(UsdStage.GetRootLayer().GetRealPath(), AbsPath)))
	{
		return;
	}

	OnPreStageChanged.Broadcast();

	UsdStage = UnrealUSDWrapper::OpenStage(*AbsPath, InitialLoadSet);
	IsolatedStage = UE::FUsdStage{};

	if (UsdStage)
	{
		if (!UsdStage.IsEditTargetValid())
		{
			UsdStage.SetEditTarget(UsdStage.GetRootLayer());
		}

		UsdStage.SetInterpolationType(InterpolationType);

		UsdListener.Register(UsdStage);

#if USE_USD_SDK
		// Try loading a UE-state session layer if we can find one
		const bool bCreateIfNeeded = false;
		UsdUtils::GetUEPersistentStateSublayer(UsdStage, bCreateIfNeeded);
#endif	  // #if USE_USD_SDK
	}
	else
	{
		// We failed to produce a valid UsdStage from this RootLayer.
		// Reset the RootLayer back to empty so that subsequent calls to OpenUsdStage don't repeatedly try
		// opening the same stage over and over
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("FailedToOpenStage", "Failed to open USD stage '{0}'"),
			FText::FromString(*RootLayer.FilePath)
		));
		RootLayer = FFilePath{};
	}

	OnStageChanged.Broadcast();
}

void AUsdStageActor::CloseUsdStage(bool bUnloadIfNeeded)
{
	TOptional<TGuardValue<ITransaction*>> SuppressTransaction;
	if (GDiscardUndoBufferOnStageOpenClose)
	{
		SuppressTransaction.Emplace(GUndo, nullptr);
		RequestDelayedTransactorReset();
	}

	const bool bStageWasOpened = static_cast<bool>(UsdStage);
	if (bStageWasOpened)
	{
		OnPreStageChanged.Broadcast();

		if (bUnloadIfNeeded)
		{
			UnloadUsdStage();
		}
	}

	FUsdStageActorImpl::DiscardStage(UsdStage, this);
	UsdStage = UE::FUsdStage();
	IsolatedStage = UE::FUsdStage();			  // We don't keep our isolated stages on the stage cache
	LevelSequenceHelper.Init(UE::FUsdStage());	  // Drop the helper's reference to the stage

	if (bStageWasOpened)
	{
		OnStageChanged.Broadcast();
	}
}

#if WITH_EDITOR
void AUsdStageActor::OnBeginPIE(bool bIsSimulating)
{
	// Remove transient flag from our spawned actors and components so they can be duplicated for PIE
	const bool bTransient = false;
	UpdateSpawnedObjectsTransientFlag(bTransient);

	bIsTransitioningIntoPIE = true;
}

void AUsdStageActor::OnPostPIEStarted(bool bIsSimulating)
{
	// Restore transient flags to our spawned actors and components so they aren't saved otherwise
	const bool bTransient = true;
	UpdateSpawnedObjectsTransientFlag(bTransient);

	bIsTransitioningIntoPIE = false;

	// Setup for the very first frame when we duplicate into PIE, or else we will display skeletal mesh components on their
	// StartTimeCode state. We have to do this here (after duplicating) as we need the calls to FUsdSkelSkeletonTranslator::UpdateComponents
	// to actually animate the components, and they will only be able to do anything after they have been registered (which
	// needs to be done by the engine when going into PIE)
	AnimatePrims();
}

void AUsdStageActor::OnObjectsReplaced(const TMap<UObject*, UObject*>& ObjectReplacementMap)
{
	UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass());
	if (!BPClass)
	{
		return;
	}

	UBlueprint* BP = Cast<UBlueprint>(BPClass->ClassGeneratedBy);
	if (!BP)
	{
		return;
	}

	// We are a replacement actor: Anything that is a property was already copied over,
	// and the spawned actors and components are still alive. We just need to move over any remaining non-property data
	if (AUsdStageActor* NewActor = Cast<AUsdStageActor>(ObjectReplacementMap.FindRef(this)))
	{
		if (FRecompilationTracker::IsBeingCompiled(BP))
		{
			// Can't just move out of this one as TUsdStore expects its TOptional to always contain a value, and we may
			// still need to use the bool operator on it to test for validity
			NewActor->UsdStage = UsdStage;
			NewActor->IsolatedStage = IsolatedStage;

			NewActor->LevelSequenceHelper = MoveTemp(LevelSequenceHelper);
			NewActor->LevelSequence = LevelSequence;
			NewActor->BlendShapesByPath = MoveTemp(BlendShapesByPath);

			NewActor->UsdListener.Register(NewActor->UsdStage);

			// This does not look super safe...
			NewActor->OnActorDestroyed = OnActorDestroyed;
			NewActor->OnActorLoaded = OnActorLoaded;
			NewActor->OnStageChanged = OnStageChanged;
			NewActor->OnPreStageChanged = OnPreStageChanged;
			NewActor->OnPrimChanged = OnPrimChanged;

			NewActor->AssetCache = AssetCache;

			// We used to just move our subobjects to the new actor here, but from the outside that doesn't look very good:
			// We'd have the new instance of the actor pointing at the subobjects of the old instance...
			// Even though we used to rename/reparent the subobjects properly, the code that calls this function
			// (UEngine::CopyPropertiesForUnrelatedObjects) builds a map from remapped old objects to new objects before it calls us.
			// Later on, the engine will use stuff like FindAndReplaceReferences to always remap from old subobjects to new subobjects,
			// undoing any UObject swapping we did here.
			//
			// Even if it were possible to influence/prepare for CopyPropertiesForUnrelatedObjects, it's probably a bad idea anyway:
			// We just want to be a "well behaved UObject" here and not go against the grain. So instead we will now
			// duplicate/stomp the new objects using the old objects. It seems slightly more aggressive, but from the outside
			// looking in it should seem better behaved as we're just keeping our new subobject. It should be slower of course,
			// but recompiling a blueprint deriving from the stage actor is not exactly a hot path
			NewActor->RootUsdTwin = DuplicateObject(RootUsdTwin, NewActor, NewActor->RootUsdTwin.GetFName());
			NewActor->UsdInfoCache = DuplicateObject(UsdInfoCache, NewActor, NewActor->UsdInfoCache.GetFName());
			NewActor->PrimLinkCache = DuplicateObject(PrimLinkCache, NewActor, NewActor->PrimLinkCache.GetFName());

			// We can just keep NewActor's transactor though

			NewActor->BBoxCache = BBoxCache;
			BBoxCache = nullptr;

			NewActor->ResyncedPrimsForThisTransaction = ResyncedPrimsForThisTransaction;

			// It could be that we're automatically recompiling when going into PIE because our blueprint was dirty.
			// In that case we also need bIsTransitioningIntoPIE to be true to prevent us from calling LoadUsdStage from PostRegisterAllComponents
			NewActor->bIsTransitioningIntoPIE = bIsTransitioningIntoPIE;
			NewActor->bIsModifyingAProperty = bIsModifyingAProperty;
			NewActor->bIsUndoRedoing = bIsUndoRedoing;

			NewActor->IsBlockedFromUsdNotices.Set(IsBlockedFromUsdNotices.GetValue());
			NewActor->OldRootLayer = OldRootLayer;

			// Close our stage or else it will remain open forever. NewActor has a a reference to it now so it won't actually close.
			// Don't discard our spawned actors and components though, as they will be used by the replacement
			const bool bUnloadIfNeeded = false;
			CloseUsdStage(bUnloadIfNeeded);
		}
	}
}

void AUsdStageActor::OnLevelActorDeleted(AActor* DeletedActor)
{
	// Check for this here because it could be that we tried to delete this actor before changing any of its
	// properties, in which case our similar check within OnObjectPropertyChange hasn't had the chance to tag this actor
	if (RootLayer.FilePath == OldRootLayer.FilePath && FUsdStageActorImpl::ObjectNeedsMultiUserTag(DeletedActor, this))
	{
		// DeletedActor is already detached from our hierarchy, so we must tag it directly
		TSet<UObject*> VisitedObjects;
		FUsdStageActorImpl::AllowListComponentHierarchy(DeletedActor->GetRootComponent(), VisitedObjects);
	}
}

#endif	  // WITH_EDITOR

void AUsdStageActor::LoadUsdStage(bool bOpenIfNeeded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::LoadUsdStage);

	// We should always have an asset cache when opening a stage now, as IUsdClassesModule::GetAssetCacheForProject
	// should never fail
	if (!RootLayer.FilePath.IsEmpty())
	{
		SetupAssetCacheIfNeeded();
		if (!ensure(AssetCache))
		{
			RootLayer.FilePath.Empty();
			return;
		}
	}

	// Make sure the asset cache tries picking up any existing asset on its UsdAssets folder before we try
	// creating new assets. We don't want to do this *too* often as we may have thousands of assets here,
	// but once before each load should be fine and could potentially save on stage load times
	if (AssetCache)
	{
		AssetCache->RescanAssetDirectory();
	}

	TOptional<TGuardValue<ITransaction*>> SuppressTransaction;
	if (GDiscardUndoBufferOnStageOpenClose)
	{
		SuppressTransaction.Emplace(GUndo, nullptr);
		RequestDelayedTransactorReset();
	}

	if (!UsdStage && bOpenIfNeeded)
	{
		OpenUsdStage();
		if (!UsdStage)
		{
			return;
		}
	}

	if (StageState != EUsdStageState::OpenedAndLoaded)
	{
		return;
	}

	double StartTime = FPlatformTime::Cycles64();

	FScopedSlowTask SlowTask(1.f, LOCTEXT("LoadingUDStage", "Loading USD Stage"));
	SlowTask.MakeDialogDelayed(0.25f);

	// Block writing level sequence changes back to the USD stage until we finished this transaction, because once we do
	// the movie scene and tracks will all trigger OnObjectTransacted. We listen for those on FUsdLevelSequenceHelperImpl::OnObjectTransacted,
	// and would otherwise end up writing all of the data we just loaded back to the USD stage
	BlockMonitoringLevelSequenceForThisTransaction();

	ObjectsToWatch.Reset();

	FUsdStageActorImpl::DeselectActorsAndComponents(this);

	UUsdPrimTwin* RootTwin = GetRootPrimTwin();
	RootTwin->Clear();
	RootTwin->PrimPath = TEXT("/");

	FScopedUsdMessageLog ScopedMessageLog;

	UUsdAssetCache3::FUsdScopedReferencer ScopedReferencer{AssetCache, this};
	AssetCache->MarkAssetsAsStale();

	UE::FUsdStage StageToLoad = GetUsdStage();

	if (UsdInfoCache)
	{
		UsdInfoCache->GetInner().ResetTranslatedPrototypes();
	}

	const bool bSequenceWasOpened = RegenerateLevelSequence();

	TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(this, RootTwin->PrimPath);

	SlowTask.EnterProgressFrame(0.1f);
	PrimLinkCache->Modify();
	PrimLinkCache->GetInner().RemoveAllAssetPrimLinks();
	UsdInfoCache->Modify();
	UsdInfoCache->GetInner().RebuildCacheForSubtrees({UE::FSdfPath::AbsoluteRootPath()}, TranslationContext.Get());

	SlowTask.EnterProgressFrame(0.7f);
	const bool bLoadedOrAbandonedAssets = LoadAssets(*TranslationContext, StageToLoad.GetPseudoRoot());

	SlowTask.EnterProgressFrame(0.2f);
	UpdatePrim(StageToLoad.GetPseudoRoot().GetPrimPath(), true, *TranslationContext);

	TranslationContext->CompleteTasks();

	// Keep our old Time value if we're loading the stage during initialization, so that we can save/load Time values
	if (StageToLoad.GetRootLayer() && IsActorInitialized())
	{
		SetTime(StageToLoad.GetRootLayer().GetStartTimeCode());

		// If we're an instance of a blueprint that derives the stage actor and we're in the editor preview world, it means we're the
		// blueprint preview actor. We (the instance) will load the stage and update our Time to StartTimeCode, but our CDO will not.
		// The blueprint editor shows the property values of the CDO however, so our Time may desync with the CDO's. If that happens, setting the Time
		// value in the blueprint editor won't be propagated to the instance, so we wouldn't be able to animate the preview actor at all.
		// Here we fix that by updating our CDO to our new Time value. Note how we just do this if we're the preview instance though, we don't
		// want other instances driving the CDO like this
		if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass()))
		{
			UWorld* World = GetWorld();
			if (World && World->WorldType == EWorldType::EditorPreview)
			{
				// Note: CDO is an instance of a BlueprintGeneratedClass here and this is just a base class pointer. We're not changing the actual
				// AUsdStageActor's CDO
				if (AUsdStageActor* CDO = Cast<AUsdStageActor>(GetClass()->GetDefaultObject()))
				{
					CDO->SetTime(GetTime());
				}
			}
		}
	}

	FUsdStageActorImpl::RepairExternalSequencerBindings();

#if WITH_EDITOR
	if (GIsEditor && GEditor && !IsGarbageCollecting())	   // Make sure we're not in standalone either
	{
		GEditor->BroadcastLevelActorListChanged();
		GEditor->RedrawLevelEditingViewports();
	}
#endif	  // WITH_EDITOR

	OnStageLoaded.Broadcast();

	if (bSequenceWasOpened)
	{
		OpenLevelSequence();
	}

	// Log time spent to load the stage
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;

	USD_LOG_INFO(TEXT("%s %s in [%d min %.3f s]"), TEXT("Stage loaded"), *FPaths::GetBaseFilename(RootLayer.FilePath), ElapsedMin, ElapsedSeconds);

#if USE_USD_SDK
	TSet<FSoftObjectPath> ActiveAssetPaths = AssetCache->GetActiveAssets();
	FUsdStageActorImpl::SendAnalytics(
		this,
		ElapsedSeconds,
		UsdUtils::GetUsdStageNumFrames(StageToLoad),
		FPaths::GetExtension(RootLayer.FilePath),
		ActiveAssetPaths
	);
#endif	  // #if USE_USD_SDK
}

void AUsdStageActor::UnloadUsdStage()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::UnloadUsdStage);

	// No point doing any of this if we're unloading because we're exiting the engine altogether
	if (IsEngineExitRequested())
	{
		return;
	}

	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	TOptional<TGuardValue<ITransaction*>> SuppressTransaction;
	if (GDiscardUndoBufferOnStageOpenClose)
	{
		SuppressTransaction.Emplace(GUndo, nullptr);
		RequestDelayedTransactorReset();
	}

	FUsdStageActorImpl::DeselectActorsAndComponents(this);

	// Stop listening because we'll discard LevelSequence assets, which may trigger transactions
	// and could lead to stage changes
	BlockMonitoringLevelSequenceForThisTransaction();

	if (LevelSequence)
	{
#if WITH_EDITOR
		// CloseAllEditorsForAsset crashes if called when the engine is closing
		if (GEditor && !IsEngineExitRequested())
		{
			// We'll only close the Sequencer via a delayed task. This because the Sequencer can't itself
			// close from the callstack of its LevelSequence being evaluated (for example, imagine we had a track to
			// set StateState to Closed: Sequencer evaluates the track -> Calls SetStageState -> Ends up here -> We try
			// destroying the Sequencer -> Crash).
			//
			// Originally we did this in an AsyncTask so that it would hopefully run later in the same frame and avoid
			// drawing the Sequencer with broken bindings. Unfortunately the task graph system can fast-forward and run these
			// async tasks at any time, which may be e.g. before other Sequencer callbacks intended to run on this same frame.
			// That could lead to a crash, as the LevelSequence would have been closed/destroyed at that point
			TWeakObjectPtr<ULevelSequence> LevelSequencePtr = LevelSequence;
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[LevelSequencePtr](float DelayTime)
				{
					if (ULevelSequence* ValidSequence = LevelSequencePtr.Get())
					{
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(ValidSequence);
					}
					return false;
				}
			));
		}
#endif	  // WITH_EDITOR

		LevelSequence = nullptr;
	}
	LevelSequenceHelper.Clear();

	// Resetting ObjectsToWatch before dropping the info cache, as the Sequencer closing may
	// trigger one last SetTime call on the stage actor (to revert to the preanimated state),
	// and if we try animating things and calling UpdateComponents we may need the info cache
	ObjectsToWatch.Reset();
	BlendShapesByPath.Reset();
	MaterialToPrimvarToUVIndex.Reset();

	if (RootUsdTwin)
	{
		RootUsdTwin->Clear();
		RootUsdTwin->PrimPath = TEXT("/");
	}

#if WITH_EDITOR
	// We can't emit this when garbage collecting as it may lead to objects being created
	// (we may unload stage when going into PIE or other sensitive transitions)
	if (GIsEditor && GEditor && !IsGarbageCollecting())
	{
		GEditor->BroadcastLevelActorListChanged();
		GEditor->RedrawLevelEditingViewports();
	}
#endif	  // WITH_EDITOR

	if (AssetCache)
	{
		AssetCache->RemoveAllReferencerAssets(this);
		AssetCache->RequestDelayedAssetAutoCleanup();
	}

	if (PrimLinkCache)
	{
		PrimLinkCache->Modify();
		PrimLinkCache->GetInner().Clear();
	}

	if (UsdInfoCache)
	{
		UsdInfoCache->Modify();
		UsdInfoCache->GetInner().Clear();
	}

	if (BBoxCache)
	{
		BBoxCache->Clear();
	}

	OnStageUnloaded.Broadcast();
}

void AUsdStageActor::SetupAssetCacheIfNeeded()
{
	if (!AssetCache)
	{
		FNotificationInfo Toast(LOCTEXT("MustHaveCache3", "Must have an Asset Cache"));
		Toast.SubText = LOCTEXT(
			"MustHaveCache3_Subtext",
			"The Stage Actor must always have an Asset Cache while a stage is loaded, so a new cache will be created."
		);
		Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		Toast.bUseLargeFont = false;
		Toast.bFireAndForget = true;
		Toast.FadeOutDuration = 1.0f;
		Toast.ExpireDuration = 8.0f;
		Toast.bUseThrobber = false;
		Toast.bUseSuccessFailIcons = false;
		FSlateNotificationManager::Get().AddNotification(Toast);

		TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};
		AssetCache = IUsdClassesModule::GetAssetCacheForProject();
	}
}

void AUsdStageActor::SetupBBoxCacheIfNeeded()
{
	if (BBoxCache.IsValid())
	{
		return;
	}

	const bool bUseExtentsHint = true;
	const bool bIgnoreVisibility = false;
	BBoxCache = MakeShared<UE::FUsdGeomBBoxCache>(Time, static_cast<EUsdPurpose>(PurposesToLoad), bUseExtentsHint, bIgnoreVisibility);
}

void AUsdStageActor::RebuildInfoCacheFromStoredChanges()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::RebuildInfoCacheFromStoredChanges);

	if (UsdInfoCache)
	{
		TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(this, TEXT("/"));
		UsdInfoCache->GetInner().RebuildCacheForSubtrees(ResyncedPrimsForThisTransaction, TranslationContext.Get());
	}
}

UUsdPrimTwin* AUsdStageActor::GetRootPrimTwin()
{
	return RootUsdTwin;
}

UUsdPrimTwin* AUsdStageActor::GetRootPrimTwin() const
{
	return RootUsdTwin;
}

void AUsdStageActor::Refresh() const
{
	OnTimeChanged.Broadcast();
}

bool AUsdStageActor::RegenerateLevelSequence()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::RegenerateLevelSequence);

	// Don't check for full authority here because even if we can't write back to the stage (i.e. during PIE) we still
	// want to listen to it and have valid level sequences
	if (IsTemplate())
	{
		return false;
	}

	// If we're using some property editor that can trigger a stage reload (like the Nanite threshold spinbox),
	// applying a value may trigger RegenerateLevelSequence -> Can trigger asset editors to open/close/change focus ->
	// -> Can trigger focus to drop from the property editors -> Can cause the values to be applied from the
	// property editors when releasing focus -> Can trigger another call to RegenerateLevelSequence.
	// CloseAllEditorsForAsset in particular is problematic for this because it will destroy the asset editor
	// (which is TSharedFromThis) and the reentrant call will try use AsShared() internally and assert, as it
	// hasn't finished being destroyed.
	// In that case we only want the outer call to change the level sequence, so a reentrant guard does what we need
	static bool bIsReentrant = false;
	if (bIsReentrant)
	{
		return false;
	}
	TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

	const UE::FUsdStage& CurrentStage = GetOrOpenUsdStage();
	if (!CurrentStage)
	{
		return false;
	}

	// We will update our levelsequence objects (tracks, moviescene, sections, etc.) due to these changes.
	// We definitely don't want to write anything back to USD when these objects change though.
	FScopedBlockMonitoringChangesForTransaction BlockMonitoring{LevelSequenceHelper};

	// Don't check for full authority here because even if we can't write back to the stage (i.e. during PIE) we still
	// want to listen to it and have valid level sequences
	bool bSequencerWasOpened = false;
#if WITH_EDITOR
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
	bSequencerWasOpened = AssetEditorSubsystem && AssetEditorSubsystem->FindEditorsForAssetAndSubObjects(LevelSequence).Num() > 0;
	if (LevelSequence && AssetEditorSubsystem)
	{
		AssetEditorSubsystem->CloseAllEditorsForAsset(LevelSequence);
	}
#endif	  // WITH_EDITOR

	// We need to guarantee we'll record our change of LevelSequence into the transaction, as Init() will create a new one
	const bool bMarkDirty = false;
	Modify(bMarkDirty);

	LevelSequence = LevelSequenceHelper.Init(CurrentStage);
	LevelSequenceHelper.BindToUsdStageActor(this);

	return bSequencerWasOpened;
}

void AUsdStageActor::RepopulateLevelSequence()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::RepopulateLevelSequence);

	TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(
		this,
		UE::FSdfPath::AbsoluteRootPath().GetString()
	);
	TranslationContext->bIsJustRepopulatingLevelSequence = true;

	// We will update our levelsequence objects (tracks, moviescene, sections, etc.) due to these changes.
	// We definitely don't want to write anything back to USD when these objects change though.
	FScopedBlockMonitoringChangesForTransaction BlockMonitoring{LevelSequenceHelper};

	const bool bIsResync = false;
	UpdatePrim(UE::FSdfPath::AbsoluteRootPath(), bIsResync, *TranslationContext);
}

void AUsdStageActor::OpenLevelSequence()
{
#if WITH_EDITOR
	if (!LevelSequence)
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
	if (AssetEditorSubsystem)
	{
		AssetEditorSubsystem->OpenEditorForAsset(LevelSequence);
	}
#endif	  // WITH_EDITOR
}

TSharedPtr<UE::FUsdGeomBBoxCache> AUsdStageActor::GetBBoxCache()
{
	return BBoxCache;
}

TMap<FString, TMap<FString, int32>> AUsdStageActor::GetMaterialToPrimvarToUVIndex()
{
	return MaterialToPrimvarToUVIndex;
}

const UsdUtils::FBlendShapeMap& AUsdStageActor::GetBlendShapeMap()
{
	return BlendShapesByPath;
}

FUsdListener& AUsdStageActor::GetUsdListener()
{
	return UsdListener;
}

const FUsdListener& AUsdStageActor::GetUsdListener() const
{
	return UsdListener;
}

#if WITH_EDITOR
void AUsdStageActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// For handling root layer changes via direct changes to properties we want to go through OnObjectPropertyChanged -> HandlePropertyChangedEvent ->
	// -> SetRootLayer (which checks whether this stage is already opened or not) -> PostRegisterAllComponents.
	// We need to intercept PostEditChangeProperty too because in the editor any call to PostEditChangeProperty can also *directly* trigger
	// PostRegister/UnregisterAllComponents which would have sidestepped our checks in SetRootLayer.
	// Note that any property change event would also end up calling our intended path via OnObjectPropertyChanged, this just prevents us from loading
	// the same stage again if we don't need to.
	TGuardValue<bool> ModifyingPropertyGuard{bIsModifyingAProperty, true};
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void AUsdStageActor::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	const TArray<FName>& ChangedProperties = TransactionEvent.GetChangedProperties();

	// TODO: This check is broken... we care whether the stage actor has had a pending kill change,
	// but we are checking whether *anything* has a pending kill change
	if (TransactionEvent.HasPendingKillChange())
	{
		// Fires when being deleted in editor, redo delete
		if (!IsValidChecked(this))
		{
			CloseUsdStage();
		}
		// This fires when being spawned in an existing level, undo delete, redo spawn
		else
		{
			OpenUsdStage();
		}
	}

	// If we're in the persistent level don't do anything, because hiding/showing the persistent level doesn't
	// cause actors to load/unload like it does if they're in sublevels
	ULevel* CurrentLevel = GetLevel();
	if (CurrentLevel && !CurrentLevel->IsPersistentLevel())
	{
		// If we're in a sublevel that is hidden, we'll respond to the generated PostUnregisterAllComponent call
		// and unload our spawned actors/assets, so let's close/open the stage too
		if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(AActor, bHiddenEdLevel))
			|| ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(AActor, bHiddenEdLayer))
			|| ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(AActor, bHiddenEd)))
		{
			if (IsHiddenEd())
			{
				CloseUsdStage();
			}
			else
			{
				OpenUsdStage();
			}
		}
	}

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		// PostTransacted marks the end of the undo/redo cycle, so reset this bool so that we can resume
		// listening to PostRegister/PostUnregister calls
		bIsUndoRedoing = false;

		// UsdStageStore can't be a UPROPERTY, so we have to make sure that it
		// is kept in sync with the state of RootLayer, because LoadUsdStage will
		// do the job of clearing our instanced actors/components if the path is empty
		if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(AUsdStageActor, RootLayer)))
		{
			// Changed the path, so we need to reopen the correct stage
			// Note: We don't unload/load here, as that would wipe the spawned actors and components that were
			// potentially recreated with the transaction
			const bool bUnloadIfNeeded = false;
			CloseUsdStage(bUnloadIfNeeded);
			OpenUsdStage();

			// Keep the info cache up to date whenever we undo/redo opening/closing/changing the root layer,
			// as we don't put the filled in cache into the transaction anymore
			if (StageState != EUsdStageState::Closed && UsdInfoCache)
			{
				TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(this, TEXT("/"));
				UsdInfoCache->GetInner().RebuildCacheForSubtrees({UE::FSdfPath::AbsoluteRootPath()}, TranslationContext.Get());
			}

			const bool bSequenceWasOpened = RegenerateLevelSequence();
			RepopulateLevelSequence();
			if (bSequenceWasOpened)
			{
				OpenLevelSequence();
			}
		}
		else if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(AUsdStageActor, StageState)))
		{
			// Partially copied from SetStageState, except that in here we don't want to call the
			// Load/UnloadUsdStage functions. Firstly because we'll already have the assets/actors/components in place
			// since they came along with us for the transaction, and secondly because PostTransacted is itself
			// outside of a transaction: Any change done in here (creating/destroying/modifying UObjects) is outside of
			// the transaction system and would cause chaos if we were to hit Undo/Redo afterwards
			if (StageState == EUsdStageState::Closed)
			{
				CloseUsdStage();
			}
			else if (StageState == EUsdStageState::Opened)
			{
				OpenUsdStage();
			}
			else if (StageState == EUsdStageState::OpenedAndLoaded)
			{
				OpenUsdStage();
			}
		}
		else if (ChangedProperties.Contains(GET_MEMBER_NAME_CHECKED(AUsdStageActor, Time)))
		{
			Refresh();

			// Sometimes when we undo/redo changes that modify SkinnedMeshComponents, their render state is not correctly updated which can show some
			// very garbled meshes. Here we workaround that by recreating all those render states manually
			const bool bRecurive = true;
			GetRootPrimTwin()->Iterate(
				[](UUsdPrimTwin& PrimTwin)
				{
					if (USkinnedMeshComponent* Component = Cast<USkinnedMeshComponent>(PrimTwin.GetSceneComponent()))
					{
						FRenderStateRecreator RecreateRenderState{Component};
					}
				},
				bRecurive
			);
		}
	}

	// Fire OnObjectTransacted so that multi-user can track our transactions
	Super::PostTransacted(TransactionEvent);
}

void AUsdStageActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	// If we're just editing some other actor property like Time or anything else, we will get
	// PostRegister/Unregister calls in the editor due to AActor::PostEditChangeProperty *and* AActor::PreEditChange.
	// Here we determine in which cases we should ignore those PostRegister/Unregister calls by using the
	// bIsModifyingAProperty flag
	TOptional<TGuardValue<bool>> ModifyingPropertyGuard;
	if (!IsActorBeingDestroyed())
	{
		if ((GEditor && GEditor->bIsSimulatingInEditor && GetWorld() != nullptr) || ReregisterComponentsWhenModified())
		{
			// PreEditChange gets called for actor lifecycle functions too (like if the actor transacts on undo/redo).
			// In those cases we will have nullptr PropertyThatWillChange, and we don't want to block our PostRegister/Unregister
			// functions. We only care about blocking the calls triggered by AActor::PostEditChangeProperty and AActor::PreEditChange
			if (PropertyThatWillChange)
			{
				ModifyingPropertyGuard.Emplace(bIsModifyingAProperty, true);
			}
		}
	}

	Super::PreEditChange(PropertyThatWillChange);
}

void AUsdStageActor::PreEditUndo()
{
	bIsUndoRedoing = true;

	Super::PreEditUndo();
}

void AUsdStageActor::HandleTransactionStateChanged(
	const FTransactionContext& InTransactionContext,
	const ETransactionStateEventType InTransactionState
)
{
	// Handle any accumulated USD notices we received during the transaction
	if (!GHandleNoticesImmediately && GIsEditor)
	{
		if (InTransactionState == ETransactionStateEventType::PreTransactionFinalized)
		{
			HandleAccumulatedNotices();
		}
		else if (InTransactionState == ETransactionStateEventType::TransactionFinalized)
		{
			ensureAlways(AccumulatedInfoChanges.Num() == 0 && AccumulatedResyncChanges.Num() == 0 && !bLayerReloaded);
			AccumulatedInfoChanges.Reset();
			AccumulatedResyncChanges.Reset();
			bLayerReloaded = false;
		}
	}

	// Hack for solving UE-127253
	// When we Reload (or open a new stage), we call ReloadAnimations which will close the Sequencer (if opened), recreate our LevelSequence, and get
	// the Sequencer to show that one instead. If we undo the Reload, that new LevelSequence will be deleted and the Sequencer will be left open
	// trying to display it, which leads to crashes. Here we try detecting for that case and close/reopen the sequencer to show the correct one.
	if (GIsEditor && LevelSequence
		&& (InTransactionState == ETransactionStateEventType::UndoRedoStarted || InTransactionState == ETransactionStateEventType::UndoRedoFinalized))
	{
		if (UTransactor* Trans = GEditor->Trans)
		{
			static TMap<AUsdStageActor*, TArray<TWeakPtr<ISequencer>>> ActorsToSequencers;

			int32 CurrentTransactionIndex = Trans->FindTransactionIndex(InTransactionContext.TransactionId);
			if (const FTransaction* Transaction = Trans->GetTransaction(CurrentTransactionIndex))
			{
				TArray<UObject*> TransactionObjects;
				Transaction->GetTransactionObjects(TransactionObjects);

				// We really just want the transactions that contain *our* LevelSequence, but it seems like when we swap LevelSequences the newly
				// created LevelSequence is not in the TransactionObjects, so we would fail to detect the right transaction on redo (as our "current
				// LevelSequence" would have been this new one, that is not part of TransactionObjects)
				bool bTransactionContainsLevelSequence = false;
				bool bTransactionContainsThis = false;
				for (UObject* TransactionObject : TransactionObjects)
				{
					if (TransactionObject == this)
					{
						bTransactionContainsThis = true;
					}
					else if (Cast<ULevelSequence>(TransactionObject))
					{
						bTransactionContainsLevelSequence = true;
					}
				}

				if (bTransactionContainsLevelSequence && bTransactionContainsThis)
				{
					if (InTransactionState == ETransactionStateEventType::UndoRedoStarted)
					{
						if (IUsdStageModule* UsdStageModule = FModuleManager::Get().GetModulePtr<IUsdStageModule>(TEXT("UsdStage")))
						{
							TArray<TWeakPtr<ISequencer>> SequencersToReset;
							for (const TWeakPtr<ISequencer>& ExistingSequencer : UsdStageModule->GetExistingSequencers())
							{
								if (TSharedPtr<ISequencer> PinnedSequencer = ExistingSequencer.Pin())
								{
									if (PinnedSequencer->GetRootMovieSceneSequence() == LevelSequence)
									{
										SequencersToReset.Add(PinnedSequencer);

										// Hack for solving UE-171596
										// In this transaction we will switch LevelSequences, and have a Sequencer opened displaying our current
										// Sequence.
										// - We cannot leave this Sequencer displaying our old LevelSequence, because it will go PendingKill, and as
										// the
										//   Sequencer fetches it through WeakPtrs it will not find a valid LevelSequence and crash (this was the
										//   reason for the original UE-127253 hack above). This means on UndoRedoStarted we *must* do something;
										// - We cannot set our new LevelSequence into it yet of course, because it hasn't been created yet (it will be
										// spawned
										//   by the undo system after UndoRedoStarted);
										// - We cannot close this Sequencer, because of this "DeferredModify" mechanism that pushes some updates to
										// the end
										//   of the transaction (to UndoRedoFinalized). If one of those updates executes after we close the Sequencer
										//   and before we fix things up (which it can always do as the order of execution of the delegates is not
										//   deterministic), it will crash (this is the issue at UE-171596);
										//
										// This means we're forced to give *some valid LevelSequence* to the Sequencer for the split second while we
										// switch our actual generated LevelSequence.
										static TStrongObjectPtr<ULevelSequence> DummySequencePtr = nullptr;
										ULevelSequence* DummySequence = DummySequencePtr.Get();
										if (!DummySequence)
										{
											DummySequence = NewObject<ULevelSequence>();
											DummySequence->Initialize();

											DummySequencePtr.Reset(DummySequence);
										}
										PinnedSequencer->ResetToNewRootSequence(*DummySequence);
									}
								}
							}
							ActorsToSequencers.Add(this, SequencersToReset);
						}
					}

					if (InTransactionState == ETransactionStateEventType::UndoRedoFinalized)
					{
						if (TArray<TWeakPtr<ISequencer>>* FoundSequencers = ActorsToSequencers.Find(this))
						{
							for (TWeakPtr<ISequencer>& Sequencer : *FoundSequencers)
							{
								if (TSharedPtr<ISequencer> PinnedSequencer = Sequencer.Pin())
								{
									if (LevelSequence && PinnedSequencer->GetRootMovieSceneSequence() != LevelSequence)
									{
										PinnedSequencer->ResetToNewRootSequence(*LevelSequence);
									}
								}
							}
							ActorsToSequencers.Remove(this);
						}
					}
				}
			}
		}
	}

	if (InTransactionState == ETransactionStateEventType::TransactionFinalized || InTransactionState == ETransactionStateEventType::UndoRedoFinalized
		|| InTransactionState == ETransactionStateEventType::TransactionCanceled)
	{
		OldRootLayer = RootLayer;
	}
}
#endif	  // WITH_EDITOR

void AUsdStageActor::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (bDuplicateForPIE)
	{
		OpenUsdStage();

		// We always want our own LevelSequence though, otherwise we could end up with some strange behavior, like:
		// PIE -> Change LevelSequence -> PIE actor is inside of a FScopedBlockNoticeListening from HandleTrackChange,
		// but the actor back in the editor is not -> Editor actor writes changes back to the stage anyway
		const bool bSequenceWasOpened = RegenerateLevelSequence();
		RepopulateLevelSequence();
		if (bSequenceWasOpened)
		{
			OpenLevelSequence();
		}
	}
	else
	{
		// When duplicating for PIE the engine will duplicate all our spawned actors and components too.
		// When duplicating directly (e.g. via Ctrl+D), it will not. This means we need to reload the stage in order
		// to generate our own duplicate actors and components. It's probably possible to just traverse the attach
		// hiearachy here and call DuplicateObject() on our spawns instead, but this should be more well-behaved
		LoadUsdStage();
	}
}

void AUsdStageActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		// We want to duplicate these properties for PIE only, as they are required to animate and listen to notices
		Ar << LevelSequence;
		Ar << RootUsdTwin;
		Ar << PrimsToAnimate;
		Ar << ObjectsToWatch;
		Ar << BlendShapesByPath;
		Ar << MaterialToPrimvarToUVIndex;
		Ar << bIsTransitioningIntoPIE;
		Ar << UsdInfoCache;
		Ar << PrimLinkCache;
	}

	if ((Ar.GetPortFlags() & PPF_DuplicateForPIE) || Ar.IsTransacting())
	{
		LevelSequenceHelper.Serialize(Ar);

		// For regular transactions we don't need to serialize the info cache: We'll do partial builds
		// after the change and when undo/redoing it
		Ar << ResyncedPrimsForThisTransaction;
	}
}

void AUsdStageActor::Destroyed()
{
	// This is fired before the actor is actually deleted or components/actors are detached.
	// We modify our child actors here because they will be detached by UWorld::DestroyActor before they're modified. Later,
	// on AUsdStageActor::Reset (called from PostTransacted), we would Modify() these actors, but if their first modify is in
	// this detached state, they're saved to the transaction as being detached from us. If we undo that transaction,
	// they will be restored as detached, which we don't want, so here we make sure they are first recorded as attached.

	TArray<AActor*> ChildActors;
	GetAttachedActors(ChildActors);

	for (AActor* Child : ChildActors)
	{
		Child->Modify();
	}

	Super::Destroyed();
}

void AUsdStageActor::PostActorCreated()
{
	Super::PostActorCreated();
}

void AUsdStageActor::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// Update the binding to this actor on the level sequence. This happens consistently when placing a BP-derived
	// stage actor with a set root layer onto the stage: We will call ReloadAnimations() before something else calls SetActorLabel()
	// and changes the actor's name, which means the level sequence would never be bound to the actor
	LevelSequenceHelper.OnStageActorRenamed();
}

void AUsdStageActor::BeginDestroy()
{
#if WITH_EDITOR
	if (!IsEngineExitRequested() && HasAuthorityOverStage())
	{
		FEditorDelegates::BeginPIE.RemoveAll(this);
		FEditorDelegates::PostPIEStarted.RemoveAll(this);
		FUsdDelegates::OnPostUsdImport.RemoveAll(this);
		FUsdDelegates::OnPreUsdImport.RemoveAll(this);
		if (UTransBuffer* TransBuffer = GUnrealEd ? Cast<UTransBuffer>(GUnrealEd->Trans) : nullptr)
		{
			TransBuffer->OnTransactionStateChanged().RemoveAll(this);
			TransBuffer->OnRedo().Remove(OnRedoHandle);
		}

		GEngine->OnLevelActorDeleted().RemoveAll(this);
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}

	// This clears the SUSDStage window whenever the level we're currently in gets destroyed.
	// Note that this is not called when deleting from the Editor, as the actor goes into the undo buffer.
	OnActorDestroyed.Broadcast();
	CloseUsdStage();

	// If our prims are already destroyed then likely the entire map has been destroyed anyway, so don't need to clear it
	if (RootUsdTwin && !RootUsdTwin->HasAnyFlags(RF_BeginDestroyed))
	{
		RootUsdTwin->Clear();
	}
#endif	  // WITH_EDITOR

	Super::BeginDestroy();
}

void AUsdStageActor::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	// Don't register FCoreUObjectDelegates from the constructor as that can lead to a race condition when async loading (see UE-294491)
	if (!IsTemplate())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
		FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &AUsdStageActor::OnObjectPropertyChanged);

		// Also prevent standalone from doing this
		if (GIsEditor && GEditor)
		{
			if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass()))
			{
				FRecompilationTracker::SetupEvents();
				FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
				FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &AUsdStageActor::OnObjectsReplaced);
			}
		}
	}
#endif	  // WITH_EDITOR
}

void AUsdStageActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	// This may fail if our stage happened to not spawn any components, actors or assets, but by that
	// point "being loaded" doesn't really mean anything anyway
	const bool bStageIsLoaded = GetBaseUsdStage()
								&& ((RootUsdTwin && RootUsdTwin->GetSceneComponent() != nullptr) || (AssetCache && AssetCache->GetNumAssets() > 0));

	// Blocks loading stage when going into PIE, if we already have something loaded (we'll want to duplicate stuff instead).
	// We need to allow loading when going into PIE when we have nothing loaded yet because the MovieRenderQueue (or other callers)
	// may directly trigger PIE sessions providing an override world. Without this exception a map saved with a loaded stage
	// wouldn't load it at all when opening the level in that way
	UWorld* World = GetWorld();
	if (bIsTransitioningIntoPIE && bStageIsLoaded && (!World || World->WorldType == EWorldType::PIE))
	{
		return;
	}

	// We get an inactive world when dragging a ULevel asset
	// This is just hiding though, so we shouldn't actively load/unload anything
	if (!World || World->WorldType == EWorldType::Inactive)
	{
		return;
	}

#if WITH_EDITOR
	// Prevent loading on bHiddenEdLevel because PostRegisterAllComponents gets called in the process of hiding our level, if we're in the persistent
	// level.
	if (bIsEditorPreviewActor || bHiddenEdLevel)
	{
		return;
	}

	if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass()))
	{
		// We can't load stage when recompiling our blueprint because blueprint recompilation is not a transaction. We're forced
		// to reuse the existing spawned components, actors and prim twins instead ( which we move over on OnObjectsReplaced ), or
		// we'd get tons of undo/redo bugs.
		if (FRecompilationTracker::IsBeingCompiled(Cast<UBlueprint>(BPClass->ClassGeneratedBy)))
		{
			return;
		}

		// For blueprints that derive from the stage actor, any property change on the blueprint preview window will trigger a full
		// PostRegisterAllComponents. We don't want to reload the stage when e.g. changing the Time property, so we have to return here
		if (World && World->WorldType == EWorldType::EditorPreview && bStageIsLoaded)
		{
			return;
		}
	}
#endif	  // WITH_EDITOR

	// When we add a sublevel the very first time (i.e. when it is associating) it may still be invisible, but we should load the stage anyway because
	// by default it will become visible shortly after this call. On subsequent postregisters, if our level is invisible there is no point to loading
	// our stage, as our spawned actors/components should be invisible too
	ULevel* Level = GetLevel();
	const bool bIsLevelHidden = !Level || (!Level->bIsVisible && !Level->bIsAssociatingLevel);
	if (bIsLevelHidden)
	{
		return;
	}

	if (IsTemplate() || bIsModifyingAProperty || bIsUndoRedoing)
	{
		return;
	}

	// Send this before we load the stage so that we know SUSDStage is synced to a potential OnStageChanged broadcast
	OnActorLoaded.Broadcast(this);

	LoadUsdStage();
}

void AUsdStageActor::UnregisterAllComponents(bool bForReregister)
{
	Super::UnregisterAllComponents(bForReregister);

	if (bForReregister || bIsModifyingAProperty || bIsUndoRedoing)
	{
		return;
	}

#if WITH_EDITOR
	if (bIsEditorPreviewActor)
	{
		return;
	}

	// We can't unload stage when recompiling our blueprint because blueprint recompilation is not a transaction.
	// After recompiling we will reuse these already spawned actors and assets.
	if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(GetClass()))
	{
		if (FRecompilationTracker::IsBeingCompiled(Cast<UBlueprint>(BPClass->ClassGeneratedBy)))
		{
			return;
		}
	}
#endif	  // WITH_EDITOR

	const bool bStageIsLoaded = GetBaseUsdStage()
								&& ((RootUsdTwin && RootUsdTwin->GetSceneComponent() != nullptr) || (AssetCache && AssetCache->GetNumAssets() > 0));

	UWorld* World = GetWorld();
	if (bIsTransitioningIntoPIE && bStageIsLoaded && (!World || World->WorldType == EWorldType::PIE))
	{
		return;
	}

	// We get an inactive world when dragging a ULevel asset
	// Unlike on PostRegister, we still want to unload our stage if our world is nullptr, as that likely means we were in
	// a sublevel that got unloaded
	if (World && World->WorldType == EWorldType::Inactive)
	{
		return;
	}

	if (IsTemplate() || IsEngineExitRequested())
	{
		return;
	}

	UnloadUsdStage();
	CloseUsdStage();
}

void AUsdStageActor::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();
}

void AUsdStageActor::OnPreUsdImport(FString FilePath)
{
	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();
	if (!CurrentStage || !HasAuthorityOverStage())
	{
		return;
	}

	// Stop listening to events because a USD import may temporarily modify the stage (e.g. when importing with
	// a different MetersPerUnit value), and we don't want to respond to the notices in the meantime
	FString RootPath = CurrentStage.GetRootLayer().GetRealPath();
	FPaths::NormalizeFilename(RootPath);
	if (RootPath == FilePath)
	{
		StopListeningToUsdNotices();
	}
}

void AUsdStageActor::OnPostUsdImport(FString FilePath)
{
	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();
	if (!CurrentStage || !HasAuthorityOverStage())
	{
		return;
	}

	// Resume listening to events
	FString RootPath = CurrentStage.GetRootLayer().GetRealPath();
	FPaths::NormalizeFilename(RootPath);
	if (RootPath == FilePath)
	{
		ResumeListeningToUsdNotices();
	}
}

void AUsdStageActor::UpdateSpawnedObjectsTransientFlag(bool bTransient)
{
	if (!RootUsdTwin)
	{
		return;
	}

	EObjectFlags Flag = bTransient ? EObjectFlags::RF_Transient : EObjectFlags::RF_NoFlags;
	TFunction<void(UUsdPrimTwin&)> UpdateTransient = [=](UUsdPrimTwin& PrimTwin)
	{
		if (USceneComponent* Component = PrimTwin.SceneComponent.Get())
		{
			Component->ClearFlags(EObjectFlags::RF_Transient);
			Component->SetFlags(Flag);

			if (AActor* ComponentOwner = Component->GetOwner())
			{
				ComponentOwner->ClearFlags(EObjectFlags::RF_Transient);
				ComponentOwner->SetFlags(Flag);
			}
		}
	};

	const bool bRecursive = true;
	GetRootPrimTwin()->Iterate(UpdateTransient, bRecursive);
}

void AUsdStageActor::RequestDelayedTransactorReset()
{
#if WITH_EDITOR
	bIsPendingTransactorReset = true;

	// Wait for the next tick because many of our functions may all try to
	// get the transactor reset
	TWeakObjectPtr<AUsdStageActor> WeakThis{this};
	ExecuteOnGameThread(
		UE_SOURCE_LOCATION,
		[WeakThis]()
		{
			if (AUsdStageActor* Actor = WeakThis.Get())
			{
				if (Actor->bIsPendingTransactorReset && GEditor)
				{
					Actor->bIsPendingTransactorReset = false;

					if (UTransactor* EditorTransactor = GEditor->Trans)
					{
						const FText Reason = LOCTEXT(
							"DiscardTransactionReason",
							"Resetting because USD.DiscardUndoBufferOnStageOpenClose is enabled"
						);
						EditorTransactor->Reset(Reason);
					}
				}
			}
		}
	);
#endif	  // WITH_EDITOR
}

void AUsdStageActor::OnUsdPrimTwinDestroyed(const UUsdPrimTwin& UsdPrimTwin)
{
	PrimsToAnimate.Remove(UsdUtils::AddHashToPrimPath(UsdPrimTwin.PrimPath));
	ObjectsToWatch.Remove(UsdPrimTwin.SceneComponent.Get());
	LevelSequenceHelper.RemovePrim(UsdPrimTwin);
}

void AUsdStageActor::OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ObjectBeingModified == this)
	{
		HandlePropertyChangedEvent(PropertyChangedEvent);
		return;
	}

	// Don't modify the stage if we're in PIE
	if (!HasAuthorityOverStage())
	{
		return;
	}

	// This transient object is owned by us but it doesn't have the multi user tag. If we're not in a transaction
	// where we're spawning objects and components, traverse our hierarchy and tag everything that needs it.
	// We avoid the RootLayer change transaction because if we tagged our spawns then the actual spawning would be
	// replicated, and we want other clients to spawn their own actors and components instead
	if (RootLayer.FilePath == OldRootLayer.FilePath && FUsdStageActorImpl::ObjectNeedsMultiUserTag(ObjectBeingModified, this))
	{
		TSet<UObject*> VisitedObjects;
		FUsdStageActorImpl::AllowListComponentHierarchy(GetRootComponent(), VisitedObjects);
	}

	// If the user is just setting metadata on one of our transient UAssets, then try to author the metadata back
	// out to the relevant prims
	if (UUsdAssetUserData* UserData = Cast<UUsdAssetUserData>(ObjectBeingModified))
	{
		if (AssetCache->IsAssetTrackedByCache(ObjectBeingModified->GetOuter()->GetPathName()))
		{
			FScopedBlockNoticeListening BlockNoticeListening(this);
			FUsdStageActorImpl::WriteOutAssetMetadataChange(this, UserData, PropertyChangedEvent);
			return;
		}
	}

	// We have to accept actor and component events here, because actor transform changes do not trigger root component
	// transform property events, and component property changes don't trigger actor property change events
	bool bIsActorEvent = false;
	UActorComponent* ComponentBeingModified = Cast<UActorComponent>(ObjectBeingModified);
	if (!ComponentBeingModified || !ObjectsToWatch.Contains(ComponentBeingModified))
	{
		if (AActor* ActorBeingModified = Cast<AActor>(ObjectBeingModified))
		{
			bIsActorEvent = true;

			if (!ObjectsToWatch.Contains(ActorBeingModified->GetRootComponent()))
			{
				return;
			}
			else
			{
				ComponentBeingModified = ActorBeingModified->GetRootComponent();
			}
		}
		else
		{
			return;
		}
	}

	// So that we can detect when the user enables/disables live link properties on a ULiveLinkComponentController that may
	// be controlling a component that we *do* care about
	ULiveLinkComponentController* Controller = Cast<ULiveLinkComponentController>(ComponentBeingModified);
	if (Controller)
	{
		if (UActorComponent* ControlledComponent = Controller->GetControlledComponent(ULiveLinkTransformRole::StaticClass()))
		{
			ComponentBeingModified = ControlledComponent;
		}
	}

	const static TSet<FName> TransformProperties = {
		USceneComponent::GetRelativeLocationPropertyName(),
		USceneComponent::GetRelativeRotationPropertyName(),
		USceneComponent::GetRelativeScale3DPropertyName()
	};
	const bool bIsTransformChange = TransformProperties.Contains(PropertyChangedEvent.GetPropertyName());

	// When we change an actor property that is just a mirror of a component property (e.g. light intensity, or camera aperture)
	// UE will emit a property changed event on the actual component with the expected PropertyChangedEvent, and also emit a strange
	// property changed event for the actor, with the PropertyChangedEvent claiming the component property changed (it didn't, it's still
	// pointing at the same component). We can *almost* fully ignore these events where the object modified is an actor then, so we
	// don't have false positives/negatives due to these strange events, except that changing the actor transform doesn't seem to
	// fire a component transform property changed event... so we allow that case to pass through
	if (bIsActorEvent && !bIsTransformChange)
	{
		return;
	}

	// Try to suppress writing anything to the stage if we're modifying a property that is animated with a track
	// on a persistent LevelSequence currently opened in the sequencer. Otherwise we'd be constantly writing out
	// default (non-animated) opinions for attributes that the user is trying to animate on their persistent LevelSequences.
	// This is also important because whenever the user closes that Sequence, the modifed properties will be reverted
	// on the UE level, but not on the stage
#if WITH_EDITOR
	{
		UObject* Context = ComponentBeingModified->GetOwner();
		AActor* OwnerActor = ComponentBeingModified->GetOwner();
		UObject* ActorContext = OwnerActor->GetWorld();

		const bool bIsRootComponent = ComponentBeingModified->GetOwner()->GetRootComponent() == ComponentBeingModified;

		IUsdStageModule& UsdStageModule = FModuleManager::Get().LoadModuleChecked<IUsdStageModule>(TEXT("UsdStage"));
		for (const TWeakPtr<ISequencer>& ExistingSequencer : UsdStageModule.GetExistingSequencers())
		{
			if (TSharedPtr<ISequencer> PinnedSequencer = ExistingSequencer.Pin())
			{
				if (UMovieSceneSequence* RootSequence = PinnedSequencer->GetRootMovieSceneSequence())
				{
					TSet<UMovieSceneSequence*> AllSequences;
					FUsdStageActorImpl::GetDescendantMovieSceneSequences(RootSequence, AllSequences);

					for (UMovieSceneSequence* Sequence : AllSequences)
					{
						UMovieScene* MovieScene = Sequence->GetMovieScene();
						if (!MovieScene)
						{
							continue;
						}

						TArray<FGuid> BindingsToCheck;
						BindingsToCheck.Add(Sequence->FindBindingFromObject(ComponentBeingModified, PinnedSequencer->GetSharedPlaybackState()));
						if (bIsRootComponent)
						{
							// Maybe all the sequence has is a track directly on the actor. That's still enough to
							// suppress a root component animation in case the property is just mirrored on the actor,
							// so let's try checking for that
							BindingsToCheck.Add(Sequence->FindBindingFromObject(OwnerActor, PinnedSequencer->GetSharedPlaybackState()));
						}

						for (const FGuid& BindingGuid : BindingsToCheck)
						{
							FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingGuid);
							if (!Binding)
							{
								continue;
							}

							for (const UMovieSceneTrack* Track : Binding->GetTracks())
							{
								// Ignore muted tracks
								if (Track->IsEvalDisabled())
								{
									continue;
								}

								if (bIsTransformChange && Track->IsA<UMovieScene3DTransformTrack>())
								{
									return;
								}

								if (const UMovieScenePropertyTrack* PropertyTrack = Cast<const UMovieScenePropertyTrack>(Track))
								{
									if (PropertyTrack->GetPropertyName() == PropertyChangedEvent.GetPropertyName())
									{
										return;
									}
								}
							}
						}
					}
				}
			}
		}
	}
#endif	  // WITH_EDITOR

	// We spawn Cine Camera Actors for Camera prims, but those have two components by default. Our convention is to place
	// camera stuff on the camera component (not much choice there), but use the transform of the scene (root) component.
	// Here we ignore transform changes of the camera component, emitting a warning if appropriate
	if (bIsTransformChange && ComponentBeingModified->IsA<UCineCameraComponent>())
	{
		FUsdStageActorImpl::ShowTransformOnCameraComponentWarning(ComponentBeingModified);
		return;
	}

	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();

	FString PrimPath = ObjectsToWatch[ComponentBeingModified];

	// Not all of our spawned components will have prim twins (e.g. ISM components for PointInstancers)
	USceneComponent* PrimSceneComponent = Cast<USceneComponent>(ComponentBeingModified);
	UUsdPrimTwin* UsdPrimTwin = GetRootPrimTwin()->Find(PrimPath);
	if (UsdPrimTwin)
	{
		PrimSceneComponent = UsdPrimTwin->SceneComponent.Get();
	}

	// Update prim from UE
	if (PrimSceneComponent && CurrentStage)
	{
		// This block is important, as it not only prevents us from getting into infinite loops with the USD notices,
		// but it also guarantees that if we have an object property change, the corresponding stage notice is not also
		// independently saved to the transaction via the UUsdTransactor, which would be duplication
		FScopedBlockNoticeListening BlockNotices(this);

		UE::FUsdPrim UsdPrim = CurrentStage.GetPrimAtPath(UE::FSdfPath(*PrimPath));

		// We want to keep component visibilities in sync with USD, which uses inherited visibilities
		// To accomplish that while blocking notices we must always propagate component visibility changes manually.
		// This part is effectively the same as calling pxr::UsdGeomImageable::MakeVisible/Invisible.
		// TODO: Allow writing out visibility without needing a prim twin
		if (UsdPrimTwin && PropertyChangedEvent.GetPropertyName() == UnrealIdentifiers::HiddenInGamePropertyName)
		{
			PrimSceneComponent->Modify();

			if (PrimSceneComponent->bHiddenInGame)
			{
				FUsdStageActorImpl::MakeInvisible(*UsdPrimTwin);
			}
			else
			{
				FUsdStageActorImpl::MakeVisible(*UsdPrimTwin, CurrentStage);
			}
		}

#if USE_USD_SDK

		// We can author material overrides even in instance proxies now, so we should always be able to do this
		if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(PrimSceneComponent))
		{
			UnrealToUsd::ConvertMeshComponent(CurrentStage, MeshComponent, UsdPrim);
		}

		if (UsdPrim.IsInstanceProxy())
		{
			if (PropertyChangedEvent.GetPropertyName() != GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials))
			{
				UsdUtils::NotifyIfInstanceProxy(UsdPrim);
			}
		}
		else
		{
			UnrealToUsd::ConvertLiveLinkProperties(Controller ? Cast<UActorComponent>(Controller) : PrimSceneComponent, UsdPrim);

			UnrealToUsd::ConvertSceneComponent(CurrentStage, PrimSceneComponent, UsdPrim);

			// When we parse a Gprim like a Cube or a Cylinder, we'll always generate some "default" meshes (e.g. Cylinder with
			// height always equal 1), and combine the Xform and the effect of the prim's attributes (e.g. height/width) into
			// a SINGLE transform, and put that on the component (this approach allows attribute animation purely with Sequencer tracks).
			// When we modify any property and want to write back out to USD however, we'll write that combined transform as the prim's
			// transform. This means we must also "reset" the (e.g. height/width) attributes, so that the combined transform stays
			// consistent
			const bool bDefaultValues = true;
			const bool bTimeSampleValues = false;
			UsdUtils::AuthorIdentityTransformGprimAttributes(UsdPrim, bDefaultValues, bTimeSampleValues);

			if (UUsdDrawModeComponent* DrawModeComponent = Cast<UUsdDrawModeComponent>(PrimSceneComponent))
			{
				const static TSet<FName> BoundsProperties = {
					GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMin),
					GET_MEMBER_NAME_CHECKED(UUsdDrawModeComponent, BoundsMax),
				};

				// If we just manually tweaked the extents, also author those back out to USD as extents opinions
				const bool bWriteExtents = BoundsProperties.Contains(PropertyChangedEvent.GetMemberPropertyName());
				const double UsdTimeCode = UsdUtils::GetDefaultTimeCode();
				UnrealToUsd::ConvertDrawModeComponent(*DrawModeComponent, UsdPrim, bWriteExtents, UsdTimeCode);
			}
			else if (UsdPrim && UsdPrim.IsA(TEXT("Camera")))
			{
				// Our component may be pointing directly at a camera component in case we recreated an exported
				// ACineCameraActor (see UE-120826)
				if (UCineCameraComponent* RecreatedCameraComponent = Cast<UCineCameraComponent>(PrimSceneComponent))
				{
					UnrealToUsd::ConvertCameraComponent(*RecreatedCameraComponent, UsdPrim);
				}
				// Or it could have been just a generic Camera prim, at which case we'll have spawned an entire new
				// ACineCameraActor for it. In this scenario our prim twin is pointing at the root component, so we need
				// to dig to the actual UCineCameraComponent to write out the camera data.
				// We should only do this when the Prim actually corresponds to the Camera though, or else we'll also catch
				// the prim/component pair that corresponds to the root scene component in case we recreated an exported
				// ACineCameraActor.
				else if (ACineCameraActor* CameraActor = Cast<ACineCameraActor>(PrimSceneComponent->GetOwner()))
				{
					if (UCineCameraComponent* CameraComponent = CameraActor->GetCineCameraComponent())
					{
						UnrealToUsd::ConvertCameraComponent(*CameraComponent, UsdPrim);
					}
				}
			}
			else if (ALight* LightActor = Cast<ALight>(PrimSceneComponent->GetOwner()))
			{
				if (ULightComponent* LightComponent = LightActor->GetLightComponent())
				{
					UnrealToUsd::ConvertLightComponent(*LightComponent, UsdPrim, UsdUtils::GetDefaultTimeCode());

					if (UDirectionalLightComponent* DirectionalLight = Cast<UDirectionalLightComponent>(LightComponent))
					{
						UnrealToUsd::ConvertDirectionalLightComponent(*DirectionalLight, UsdPrim, UsdUtils::GetDefaultTimeCode());
					}
					else if (URectLightComponent* RectLight = Cast<URectLightComponent>(LightComponent))
					{
						UnrealToUsd::ConvertRectLightComponent(*RectLight, UsdPrim, UsdUtils::GetDefaultTimeCode());
					}
					else if (UPointLightComponent* PointLight = Cast<UPointLightComponent>(LightComponent))
					{
						UnrealToUsd::ConvertPointLightComponent(*PointLight, UsdPrim, UsdUtils::GetDefaultTimeCode());

						if (USpotLightComponent* SpotLight = Cast<USpotLightComponent>(LightComponent))
						{
							UnrealToUsd::ConvertSpotLightComponent(*SpotLight, UsdPrim, UsdUtils::GetDefaultTimeCode());
						}
					}
				}
			}
			// In contrast to the other light types, the USkyLightComponent is the root component of the ASkyLight
			else if (USkyLightComponent* SkyLightComponent = Cast<USkyLightComponent>(PrimSceneComponent))
			{
				UnrealToUsd::ConvertLightComponent(*SkyLightComponent, UsdPrim, UsdUtils::GetDefaultTimeCode());
				UnrealToUsd::ConvertSkyLightComponent(*SkyLightComponent, UsdPrim, UsdUtils::GetDefaultTimeCode());
			}
		}
#endif	  // #if USE_USD_SDK

		// Update stage window in case any of our component changes trigger USD stage changes
		this->OnPrimChanged.Broadcast(PrimPath, false);
	}
}

void AUsdStageActor::HandlePropertyChangedEvent(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Handle property changed events with this function (called from our OnObjectPropertyChanged delegate) instead of overriding
	// PostEditChangeProperty because replicated multi-user transactions directly broadcast OnObjectPropertyChanged on the properties that were
	// changed, instead of making PostEditChangeProperty events. Note that UObject::PostEditChangeProperty ends up broadcasting
	// OnObjectPropertyChanged anyway, so this works just the same as before. see ConcertClientTransactionBridge.cpp, function
	// ConcertClientTransactionBridgeUtil::ProcessTransactionEvent.

	// Note that in here we'll delegate to these setter functions (like SetRootLayer) to actually set the new property values.
	// We want our setter functions to be able to automatically refresh the stage (both for simplicity, since we have a single
	// code path for changing them that gets reused everywhere) and also due to the fact that the Sequencer uses these setters
	// when we create Sequencer tracks for these properties: If we make a track for e.g. "PurposesToLoad", we want the stage to
	// refresh as soon as we hit a keyframe to change the chosen purposes. We don't want to need some separate track to "refresh
	// the stage" or something like that.
	//
	// An issue, however, is the fact that the Sequencer can repeatedly call these setters with the same value over and over in
	// case it is just e.g. stopped at some frame. We don't want that to keep reloading the stage, so we need the setters to have
	// an "early out" and not do anything in case they're receiving the same value that was previously set.
	//
	// With an "early out" mechanism though, we end up with a problem: This function (called from OnObjectPropertyChanged) is only
	// called *after* these properties have already been set with their new values. So if we naively delegated to the setters now
	// they would all just "early out" and do nothing. We do still need to respond from the OnObjectPropertyChanged code path though,
	// due to the fact that ConcertClientTransactionBridgeUtil::ProcessTransactionEvent calls OnObjectPropertyChanged directly in
	// order to replicate the multiuser property value changes. We want the stage to automatically refresh when that happens,
	// meaning we need to do exactly what the setters do anyway and may as well call them. TL;DR: We need this function and for it
	// to call the setters.
	//
	// We can't rely on any other additional event (like OnPreObjectPropertyChanged) because that doesn't tell us the new value
	// that will be changed anyway, so we'd need some complicated mechanism to store the property values at e.g.
	// OnPreObjectPropertyChanged time and compare our current values to them to know if something changed...
	//
	// This explains the CorrectValues (e.g. CorrectTime, CorretRootLayer, etc.) you'll see below: We will temporarly put a
	// different value on the properties before calling them to prevent the setters from earlying out. We don't want to record
	// these spoofed values into the transaction though (otherwise if we hit Undo we would end up with those set), so we Modify()
	// before we do that.
	const bool bAlwaysMarkAsDirty = false;
	Modify(bAlwaysMarkAsDirty);

	// If we're changing a property inside a struct, like "bCollectMetadata" inside our MetadataOptions, then
	// "MemberProperty" will point to "MetadataOptions", and "Property" is the thing that will point to "bCollectMetadata"
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, RootLayer))
	{
		// Technically we don't need this guard value for the root layer itself, since SetRootLayer can compare
		// RootLayer with the path of the current stage's root layer, but let's just do this for consistency.
		const FString CorrectRootLayer = RootLayer.FilePath;
		RootLayer.FilePath = RootLayer.FilePath + TEXT("dummy");
		SetRootLayer(CorrectRootLayer);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, StageState))
	{
		const EUsdStageState CorrectState = StageState;
		StageState = (EUsdStageState) !((uint8)StageState);
		SetStageState(CorrectState);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, Time))
	{
		const float CorrectTime = Time;
		Time = Time + 1.0f;
		SetTime(CorrectTime);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, InitialLoadSet))
	{
		const EUsdInitialLoadSet CorrectLoadSet = InitialLoadSet;
		InitialLoadSet = (EUsdInitialLoadSet) !((uint8)InitialLoadSet);
		SetInitialLoadSet(CorrectLoadSet);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, InterpolationType))
	{
		const EUsdInterpolationType CorrectInterpolationType = InterpolationType;
		InterpolationType = (EUsdInterpolationType) !((uint8)InterpolationType);
		SetInterpolationType(CorrectInterpolationType);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, GeometryCacheImport))
	{
		const EGeometryCacheImport CorrectImportOption = GeometryCacheImport;
		GeometryCacheImport = (EGeometryCacheImport) !((uint8)GeometryCacheImport);
		SetGeometryCacheImport(CorrectImportOption);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, bUsePrimKindsForCollapsing))
	{
		const bool bCorrectUsePrimKindsForCollapsing = bUsePrimKindsForCollapsing;
		bUsePrimKindsForCollapsing = !bUsePrimKindsForCollapsing;
		SetUsePrimKindsForCollapsing(bCorrectUsePrimKindsForCollapsing);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, KindsToCollapse))
	{
		const int32 CorrectKindsToCollapse = KindsToCollapse;
		KindsToCollapse += 1;
		SetKindsToCollapse(CorrectKindsToCollapse);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, bMergeIdenticalMaterialSlots))
	{
		const bool bCorrectMergeMaterialSlots = bMergeIdenticalMaterialSlots;
		bMergeIdenticalMaterialSlots = !bMergeIdenticalMaterialSlots;
		SetMergeIdenticalMaterialSlots(bCorrectMergeMaterialSlots);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, bShareAssetsForIdenticalPrims))
	{
		const bool bCorrectShare = bShareAssetsForIdenticalPrims;
		bShareAssetsForIdenticalPrims = !bShareAssetsForIdenticalPrims;
		SetShareAssetsForIdenticalPrims(bCorrectShare);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, PurposesToLoad))
	{
		const int32 CorrectPurposesToLoad = PurposesToLoad;
		PurposesToLoad += 1;
		SetPurposesToLoad(CorrectPurposesToLoad);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, NaniteTriangleThreshold))
	{
		const int32 CorrectNaniteThreshold = NaniteTriangleThreshold;
		NaniteTriangleThreshold += 1;
		SetNaniteTriangleThreshold(CorrectNaniteThreshold);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, RenderContext))
	{
		const FName CorrectRenderContext = RenderContext;
		RenderContext = *(RenderContext.ToString() + TEXT("dummy"));
		SetRenderContext(CorrectRenderContext);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, MaterialPurpose))
	{
		const FName CorrectMaterialPurpose = MaterialPurpose;
		MaterialPurpose = *(MaterialPurpose.ToString() + TEXT("dummy"));
		SetMaterialPurpose(CorrectMaterialPurpose);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, RootMotionHandling))
	{
		const EUsdRootMotionHandling CorrectHandling = RootMotionHandling;
		RootMotionHandling = (EUsdRootMotionHandling) !((uint8)RootMotionHandling);
		SetRootMotionHandling(CorrectHandling);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, FallbackCollisionType))
	{
		const EUsdCollisionType CorrectCollisionType = FallbackCollisionType;
		FallbackCollisionType = (EUsdCollisionType) !((uint8)FallbackCollisionType);
		SetFallbackCollisionType(CorrectCollisionType);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, SubdivisionLevel))
	{
		int32 CorrectSubdivisionLevel = SubdivisionLevel;
		SubdivisionLevel = !SubdivisionLevel;
		SetSubdivisionLevel(CorrectSubdivisionLevel);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FUsdMetadataImportOptions, bCollectMetadata))
	{
		bool bCorrectCollectValue = MetadataOptions.bCollectMetadata;
		MetadataOptions.bCollectMetadata = !bCorrectCollectValue;
		SetCollectMetadata(bCorrectCollectValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FUsdMetadataImportOptions, bCollectFromEntireSubtrees))
	{
		bool bCorrectCollectValue = MetadataOptions.bCollectFromEntireSubtrees;
		MetadataOptions.bCollectFromEntireSubtrees = !bCorrectCollectValue;
		SetCollectFromEntireSubtrees(bCorrectCollectValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FUsdMetadataImportOptions, bCollectOnComponents))
	{
		bool bCorrectCollectValue = MetadataOptions.bCollectOnComponents;
		MetadataOptions.bCollectOnComponents = !bCorrectCollectValue;
		SetCollectOnComponents(bCorrectCollectValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FUsdMetadataImportOptions, BlockedPrefixFilters))
	{
		TArray<FString> CorrectFilters = MetadataOptions.BlockedPrefixFilters;
		MetadataOptions.BlockedPrefixFilters.Add(TEXT("dummy"));
		SetBlockedPrefixFilters(CorrectFilters);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FUsdMetadataImportOptions, bInvertFilters))
	{
		bool bCorrectInvertValue = MetadataOptions.bInvertFilters;
		MetadataOptions.bInvertFilters = !bCorrectInvertValue;
		SetInvertFilters(bCorrectInvertValue);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(AUsdStageActor, AssetCache))
	{
		UUsdAssetCache3* CorrectCache = AssetCache.Get();
		AssetCache = AssetCache ? nullptr : NewObject<UUsdAssetCache3>();

		SetUsdAssetCache(CorrectCache);
	}
}

bool AUsdStageActor::HasAuthorityOverStage() const
{
#if WITH_EDITOR
	if (GIsEditor)	  // Don't check for world in Standalone: The game world is the only one there, so it's OK if we have authority while in it
	{
		// In the editor we have to prevent actors in PIE worlds from having authority
		return !IsTemplate() && (!GetWorld() || !GetWorld()->IsGameWorld());
	}
#endif	  // WITH_EDITOR

	return !IsTemplate();
}

void AUsdStageActor::OnSkelAnimationBaked(const FString& SkeletonPrimPath)
{
#if USE_USD_SDK
	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();
	if (!CurrentStage || !GRegenerateSkeletalAssetsOnControlRigBake)
	{
		return;
	}

	UE::FUsdPrim SkeletonPrim = CurrentStage.GetPrimAtPath(UE::FSdfPath{*SkeletonPrimPath});
	if (!SkeletonPrim || !SkeletonPrim.IsA(TEXT("Skeleton")))
	{
		return;
	}

	UUsdPrimTwin* RootTwin = GetRootPrimTwin();
	if (!RootTwin)
	{
		return;
	}

	UUsdPrimTwin* Twin = RootTwin->Find(SkeletonPrimPath);
	if (!Twin)
	{
		return;
	}

	USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Twin->GetSceneComponent());
	if (!SkeletalMeshComponent)
	{
		return;
	}

	TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(this, SkeletonPrimPath);
	// The only way we could have baked a skel animation is via the sequencer, so we know its playing
	TranslationContext->bSequencerIsAnimating = true;

	if (UsdInfoCache)
	{
		UsdInfoCache->GetInner().ResetTranslatedPrototypes();
	}

	if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get()
																.CreateTranslatorForSchema(TranslationContext, UE::FUsdTyped(SkeletonPrim)))
	{
		if (TSharedPtr<FUsdSkelSkeletonTranslator> SkelRootTranslator = StaticCastSharedPtr<FUsdSkelSkeletonTranslator>(SchemaTranslator))
		{
			// For now we're regenerating all asset types (including skeletal meshes) but we could
			// eventually just split off the anim sequence generation and call exclusively that from
			// here
			SkelRootTranslator->CreateAssets();
			TranslationContext->CompleteTasks();

			// Have to update the components to assign the new assets
			SkelRootTranslator->UpdateComponents(SkeletalMeshComponent);
		}
	}
#endif	  // #if USE_USD_SDK
}

bool AUsdStageActor::UnloadAssets(const UE::FSdfPath& StartPrimPath, bool bForEntireSubtree)
{
	// Note that whenever we change a stage option (like render context, kinds to collapse, etc.) we may generate
	// new assets for a prim but we won't call this function, which means we will still temporarily keep the old
	// assets in the asset cache, and they will count as "referenced". That is not great, although they will *still*
	// be tracked via the info cache asset prim links, so if at any time they resync the old assets will still be
	// found below when iterating the prim links, and we will discard them either way.

	if (!AssetCache || !PrimLinkCache)
	{
		return false;
	}

	AssetCache->Modify();
	PrimLinkCache->Modify();

	bool bAssetsDiscarded = false;

	TSet<UE::FSdfPath> PrimPathsToRemove;
	if (bForEntireSubtree)
	{
		for (const TPair<UE::FSdfPath, TArray<TWeakObjectPtr<UObject>>>& PrimPathToAssetIt : PrimLinkCache->GetInner().GetAllAssetPrimLinks())
		{
			const UE::FSdfPath& LinkPrimPath = PrimPathToAssetIt.Key;
			if (LinkPrimPath.HasPrefix(StartPrimPath) || LinkPrimPath == StartPrimPath)
			{
				PrimPathsToRemove.Add(LinkPrimPath);
			}
		}
	}
	else
	{
		PrimPathsToRemove.Add(StartPrimPath);
	}

	for (const UE::FSdfPath& PrimPathToRemove : PrimPathsToRemove)
	{
		TArray<TWeakObjectPtr<UObject>> OldAssets = PrimLinkCache->GetInner().RemoveAllAssetPrimLinks(PrimPathToRemove);
		for (const TWeakObjectPtr<UObject>& OldAsset : OldAssets)
		{
			// If there are any other prim paths linked to this asset that we *won't* be removing/reparsing
			// in here, it means our stage actor as a whole is still "referencing" that asset
			bool bAssetStillReferenced = false;
			for (const UE::FSdfPath& LinkedPrim : PrimLinkCache->GetInner().GetPrimsForAsset(OldAsset.Get()))
			{
				if (!PrimPathsToRemove.Contains(LinkedPrim))
				{
					bAssetStillReferenced = true;
					break;
				}
			}
			if (bAssetStillReferenced)
			{
				continue;
			}

			// If we're going to delete, just remove our reference but keep tracking: We need to be tracking an
			// unreferenced asset in order to be able to delete it.
			// Note: We could make it so that we fully stop tracking the asset if we're not going to delete it,
			// although that doesn't really get us anything
			bAssetsDiscarded |= AssetCache->RemoveAssetReferencer(OldAsset.Get(), this);
		}
	}

	return bAssetsDiscarded;
}

bool AUsdStageActor::LoadAsset(FUsdSchemaTranslationContext& TranslationContext, const UE::FUsdPrim& Prim)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::LoadAsset);

	if (!AssetCache)
	{
		return false;
	}

	int32 StartNumAssets = AssetCache->GetNumAssets();

	AssetCache->Modify();
	PrimLinkCache->Modify();
	{
		// Suppress transaction while we're creating assets.
		// c.f. the big comment on the analogous position within AUsdStageActor::LoadAssets
		TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};

		if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(	  //
				TranslationContext.AsShared(),
				UE::FUsdTyped(Prim)
			))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::CreateAssetsForPrim);
			SchemaTranslator->CreateAssets();
		}

		TranslationContext.CompleteTasks();	   // Finish the asset tasks before moving on
	}

	return AssetCache->GetNumAssets() != StartNumAssets;
}

bool AUsdStageActor::LoadAssets(FUsdSchemaTranslationContext& TranslationContext, const UE::FUsdPrim& StartPrim)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::LoadAssets);

	if (!AssetCache)
	{
		return false;
	}

	int32 StartNumAssets = AssetCache->GetNumAssets();

	auto CreateAssetsForPrims = [&TranslationContext](const TArray<UE::FUsdPrim>& AllPrimAssets, FSlowTask& Progress)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::CreateAssetsForPrims);

		for (const UE::FUsdPrim& UsdPrim : AllPrimAssets)
		{
			Progress.EnterProgressFrame(1.f);

			if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(
					TranslationContext.AsShared(),
					UE::FUsdTyped(UsdPrim)
				))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::CreateAssetsForPrim);
				SchemaTranslator->CreateAssets();
			}
		}

		TranslationContext.CompleteTasks();	   // Finish the assets tasks before moving on
	};

	auto PruneChildren = [&TranslationContext](const UE::FUsdPrim& UsdPrim) -> bool
	{
		if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get()
																	.CreateTranslatorForSchema(TranslationContext.AsShared(), UE::FUsdTyped(UsdPrim)))
		{
			return SchemaTranslator->CollapsesChildren(ECollapsingType::Assets);
		}

		return false;
	};

	AssetCache->Modify();
	PrimLinkCache->Modify();
	{
		// Suppress current transaction, as we never want assets to be put into the transaction buffer. This because these
		// will be exposed to the content browser now, so that "opening the stage" essentially acts as a full import.
		// We don't want to rip these assets up when pressing undo after they've been created. The engine should act
		// essentially as if they've always been there.
		//
		// Note that we have tried achieving that by just creating these assets without the RF_Transactional flag, but that is not enough:
		// Some assets create subobjects that are transactional anyway (StaticMeshes), and some other assets have much
		// more complicated logic that can even spawn some transient Worlds, actors and components, and can put them all
		// into the transaction buffer (Skeletal assets), causing havoc if we try to make sense of object referencers
		// when it's time to clean up the asset.
		TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};

		// Load materials first since meshes are referencing them
		TArray<UE::FUsdPrim> AllPrimAssets = UsdUtils::GetAllPrimsOfType(StartPrim, TEXT("UsdShadeMaterial"));
		{
			FScopedSlowTask MaterialsProgress(AllPrimAssets.Num(), LOCTEXT("CreateMaterials", "Creating materials"));
			CreateAssetsForPrims(AllPrimAssets, MaterialsProgress);
		}

		// Load everything else (including meshes)
		AllPrimAssets = UsdUtils::GetAllPrimsOfType(StartPrim, TEXT("UsdSchemaBase"), PruneChildren, {TEXT("UsdShadeMaterial")});
		{
			FScopedSlowTask AssetsProgress(AllPrimAssets.Num(), LOCTEXT("CreateAssets", "Creating assets"));
			CreateAssetsForPrims(AllPrimAssets, AssetsProgress);
		}
	}

	return AssetCache->GetNumAssets() != StartNumAssets;
}

void AUsdStageActor::AnimatePrims()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AUsdStageActor::AnimatePrims);

	// Don't try to animate if we don't have a stage opened
	const UE::FUsdStage& CurrentStage = static_cast<const AUsdStageActor*>(this)->GetUsdStage();
	if (!CurrentStage)
	{
		return;
	}

	if (UsdInfoCache)
	{
		UsdInfoCache->GetInner().ResetTranslatedPrototypes();
	}

	TSharedRef<FUsdSchemaTranslationContext> TranslationContext = FUsdStageActorImpl::CreateUsdSchemaTranslationContext(
		this,
		GetRootPrimTwin()->PrimPath
	);

	// For performance reasons we don't want to try computing material overrides on every animation frame.
	// Material bindings don't change with time code anyway
	TranslationContext->bAllowRecomputingMaterialOverrides = false;

	// c.f. comment on bSequencerIsAnimating's declaration
#if WITH_EDITOR
	if (GEditor)
	{
		const bool bFocusIfOpen = false;
		IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(LevelSequence, bFocusIfOpen);
		if (ILevelSequenceEditorToolkit* LevelSequenceEditor = static_cast<ILevelSequenceEditorToolkit*>(AssetEditor))
		{
			TranslationContext->bSequencerIsAnimating = true;
		}
	}
#endif	  // WITH_EDITOR

	for (const FString& PrimPathWithHash : PrimsToAnimate)
	{
		const FString PrimPathWithoutHash = UsdUtils::RemoveHashFromPrimPath(PrimPathWithHash);
		UE::FSdfPath PrimPath{*PrimPathWithoutHash};

		if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(
				TranslationContext,
				UE::FUsdTyped(CurrentStage.GetPrimAtPath(PrimPath))
			))
		{
			if (UUsdPrimTwin* UsdPrimTwin = GetRootPrimTwin()->Find(PrimPathWithoutHash))
			{
				SchemaTranslator->UpdateComponents(UsdPrimTwin->SceneComponent.Get());
			}
		}
	}

	TranslationContext->CompleteTasks();
}

FScopedBlockNoticeListening::FScopedBlockNoticeListening(AUsdStageActor* InStageActor)
{
	StageActor = InStageActor;
	if (InStageActor)
	{
		StageActor->StopListeningToUsdNotices();
	}
}

FScopedBlockNoticeListening::~FScopedBlockNoticeListening()
{
	if (AUsdStageActor* StageActorPtr = StageActor.Get())
	{
		StageActorPtr->ResumeListeningToUsdNotices();
	}
}

#undef LOCTEXT_NAMESPACE
