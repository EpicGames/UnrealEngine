// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "USDStageActorCustomization.h"

#include "USDMaterialUtils.h"
#include "USDProjectSettings.h"
#include "Objects/USDSchemaTranslator.h"
#include "USDStageActor.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "UsdStageActorCustomization"

FUsdStageActorCustomization::FUsdStageActorCustomization()
{
}

TSharedRef<IDetailCustomization> FUsdStageActorCustomization::MakeInstance()
{
	return MakeShared<FUsdStageActorCustomization>();
}

void FUsdStageActorCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];
	if (!SelectedObject.IsValid())
	{
		return;
	}

	CurrentActor = Cast<AUsdStageActor>(SelectedObject.Get());
	if (!CurrentActor)
	{
		return;
	}

	CurrentActor->OnStageChanged.AddSP(this, &FUsdStageActorCustomization::ForceRefreshDetails);

	RenderContextComboBoxItems.Reset();
	TSharedPtr<FString> InitiallySelectedContext;
	for (const FName& Context : UsdUnreal::MaterialUtils::GetRegisteredRenderContexts())
	{
		TSharedPtr<FString> ContextStr;
		if (Context == UnrealIdentifiers::UniversalRenderContext)
		{
			ContextStr = MakeShared<FString>(UnrealIdentifiers::UniversalRenderContextDisplayString);
		}
		else
		{
			ContextStr = MakeShared<FString>(Context.ToString());
		}

		if (Context == CurrentActor->RenderContext)
		{
			InitiallySelectedContext = ContextStr;
		}

		RenderContextComboBoxItems.Add(ContextStr);
	}

	IDetailCategoryBuilder& CatBuilder = DetailLayoutBuilder.EditCategory(TEXT("USD"));

	// clang-format off
	TWeakObjectPtr<AUsdStageActor> WeakActor = CurrentActor;
	CatBuilder.AddCustomRow(FText())
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SButton)
			.Text(LOCTEXT("OpenInStageEditorText", "Open in USD Stage Editor"))
			.OnClicked_Lambda([WeakActor]()
			{
				if(AUsdStageActor* StageActor = WeakActor.Get())
				{
					AUsdStageActor::OnOpenStageEditorClicked.Broadcast(StageActor);
				}

				return FReply::Handled();
			})
		]
	];

	if (TSharedPtr<IPropertyHandle> RenderContextProperty = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AUsdStageActor, RenderContext)))
	{
		DetailLayoutBuilder.HideProperty(RenderContextProperty);

		CatBuilder.AddCustomRow(FText::FromString(TEXT("RenderContextCustomization")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Render context")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(RenderContextProperty->GetToolTipText())
		]
		.ValueContent()
		[
			SAssignNew(RenderContextComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&RenderContextComboBoxItems)
			.InitiallySelectedItem(InitiallySelectedContext)
			.OnSelectionChanged(this, &FUsdStageActorCustomization::OnComboBoxSelectionChanged)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
			{
				return SNew(STextBlock)
					.Text(Item.IsValid() ? FText::FromString(*Item) : FText::GetEmpty())
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FUsdStageActorCustomization::GetComboBoxSelectedOptionText)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];
	}

	if (TSharedPtr<IPropertyHandle> MaterialPurposeProperty = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(AUsdStageActor, MaterialPurpose)))
	{
		DetailLayoutBuilder.HideProperty(MaterialPurposeProperty);

		CatBuilder.AddCustomRow(FText::FromString(TEXT("MaterialPurposeCustomization")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Material purpose")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(MaterialPurposeProperty->GetToolTipText())
		]
		.ValueContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(SComboBox< TSharedPtr<FString> >)
				.OptionsSource(&MaterialPurposeComboBoxItems)
				.OnComboBoxOpening_Lambda([this]()
				{
					MaterialPurposeComboBoxItems = {
						MakeShared<FString>(UnrealIdentifiers::MaterialAllPurpose),
						MakeShared<FString>(UnrealIdentifiers::MaterialPreviewPurpose),
						MakeShared<FString>(UnrealIdentifiers::MaterialFullPurpose)
					};

					// Add additional purposes from project settings
					if (const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>())
					{
						MaterialPurposeComboBoxItems.Reserve(MaterialPurposeComboBoxItems.Num() + ProjectSettings->AdditionalMaterialPurposes.Num());

						TSet<FString> ExistingEntries = {
							UnrealIdentifiers::MaterialAllPurpose,
							UnrealIdentifiers::MaterialPreviewPurpose,
							UnrealIdentifiers::MaterialFullPurpose
						};

						for (const FName& AdditionalPurpose : ProjectSettings->AdditionalMaterialPurposes)
						{
							FString AdditionalPurposeStr = AdditionalPurpose.ToString();

							if (!ExistingEntries.Contains(AdditionalPurposeStr))
							{
								ExistingEntries.Add(AdditionalPurposeStr);
								MaterialPurposeComboBoxItems.AddUnique(MakeShared<FString>(AdditionalPurposeStr));
							}
						}
					}
				})
				.OnGenerateWidget_Lambda([&](TSharedPtr<FString> Option)
				{
					TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
					if (Option)
					{
						Widget = SNew(STextBlock)
							.Text(FText::FromString((*Option) == UnrealIdentifiers::MaterialAllPurpose
								? UnrealIdentifiers::MaterialAllPurposeText
								: *Option
							))
							.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
					}

					return Widget.ToSharedRef();
				})
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo)
				{
					if (ChosenOption && CurrentActor)
					{
						CurrentActor->SetMaterialPurpose(**ChosenOption);
					}
				})
				[
					SNew(SEditableTextBox)
					.Text_Lambda([this]() -> FText
					{
						if (CurrentActor)
						{
							return FText::FromString(CurrentActor->MaterialPurpose == *UnrealIdentifiers::MaterialAllPurpose
								? UnrealIdentifiers::MaterialAllPurposeText
								: CurrentActor->MaterialPurpose.ToString()
							);
						}

						return FText::GetEmpty();
					})
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
					{
						if (CommitType != ETextCommit::OnEnter)
						{
							return;
						}

						FString NewPurposeString = NewText.ToString();
						FName NewPurpose = *NewPurposeString;

						bool bIsNew = true;
						for (const TSharedPtr<FString>& Purpose : MaterialPurposeComboBoxItems)
						{
							if (Purpose && *Purpose == NewPurposeString)
							{
								bIsNew = false;
								break;
							}
						}

						if (bIsNew)
						{
							if (UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>())
							{
								ProjectSettings->AdditionalMaterialPurposes.AddUnique(NewPurpose);
								ProjectSettings->SaveConfig();
							}
						}

						if (CurrentActor)
						{
							CurrentActor->SetMaterialPurpose(NewPurpose);
						}
					})
				]
			]
		];
	}
	// clang-format on

	// Add/remove properties so that they retain their usual order
	if (TSharedPtr<IPropertyHandle> TimeProperty = DetailLayoutBuilder.GetProperty(TEXT("Time")))
	{
		CatBuilder.AddProperty(TimeProperty);
	}
	if (TSharedPtr<IPropertyHandle> LevelSequenceProperty = DetailLayoutBuilder.GetProperty(TEXT("LevelSequence")))
	{
		CatBuilder.AddProperty(LevelSequenceProperty);
	}
}

void FUsdStageActorCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	DetailBuilderWeakPtr = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

void FUsdStageActorCustomization::OnComboBoxSelectionChanged(TSharedPtr<FString> NewContext, ESelectInfo::Type SelectType)
{
	if (CurrentActor == nullptr || !NewContext.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("RenderContextChangedTransaction", "Changed the UsdStageActor {0}'s RenderContext to '{1}'"),
		FText::FromString(CurrentActor->GetActorLabel()),
		FText::FromString(NewContext.IsValid() ? *NewContext : TEXT("None"))
	));

	FName NewContextName = (*NewContext) == UnrealIdentifiers::UniversalRenderContextDisplayString ? UnrealIdentifiers::UniversalRenderContext
																								   : FName(**NewContext);

	CurrentActor->SetRenderContext(NewContextName);
}

FText FUsdStageActorCustomization::GetComboBoxSelectedOptionText() const
{
	TSharedPtr<FString> SelectedItem = RenderContextComboBox->GetSelectedItem();
	if (SelectedItem.IsValid())
	{
		return FText::FromString(*SelectedItem);
	}

	return FText::GetEmpty();
}

void FUsdStageActorCustomization::ForceRefreshDetails()
{
	// Raw because we don't want to keep alive the details builder when calling the force refresh details
	IDetailLayoutBuilder* DetailLayoutBuilder = DetailBuilderWeakPtr.Pin().Get();
	if (DetailLayoutBuilder)
	{
		DetailLayoutBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE

#endif	  // WITH_EDITOR
