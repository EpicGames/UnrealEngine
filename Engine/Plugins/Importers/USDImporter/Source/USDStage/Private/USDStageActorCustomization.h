// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "IDetailCustomization.h"
#include "Widgets/Input/SComboBox.h"

class AUsdStageActor;
class IDetailLayoutBuilder;

class FUsdStageActorCustomization : public IDetailCustomization
{
public:
	FUsdStageActorCustomization();
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;
	// End of IDetailCustomization interface

private:
	void OnComboBoxSelectionChanged(TSharedPtr<FString> NewContext, ESelectInfo::Type SelectType);
	FText GetComboBoxSelectedOptionText() const;
	void ForceRefreshDetails();

	AUsdStageActor* CurrentActor;
	TWeakPtr<IDetailLayoutBuilder> DetailBuilderWeakPtr;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> RenderContextComboBox;
	TArray<TSharedPtr<FString>> RenderContextComboBoxItems;

	TArray<TSharedPtr<FString>> MaterialPurposeComboBoxItems;
};

#endif	  // WITH_EDITOR
