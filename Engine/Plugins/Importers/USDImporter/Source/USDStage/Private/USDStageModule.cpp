// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageModule.h"

#include "USDLocatorFragments.h"
#include "USDMemory.h"
#include "USDStageActor.h"
#include "USDStageActorCustomization.h"

#include "Engine/World.h"
#include "EngineUtils.h"
#include "Interfaces/IPluginManager.h"
#include "IUniversalObjectLocatorModule.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "USDLocatorEditors.h"

#include "ISequencerModule.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "PropertyEditorModule.h"
#endif	  // WITH_EDITOR

class FUsdStageModule : public IUsdStageModule
{
public:
	virtual void StartupModule() override
	{
		using namespace UE::UniversalObjectLocator;

		IUniversalObjectLocatorModule& UolModule = FModuleManager::Get().LoadModuleChecked<IUniversalObjectLocatorModule>("UniversalObjectLocator");
		FFragmentTypeParameters Parameters{"usdprim", NSLOCTEXT("USDStageModule", "UsdPrimLocatorFragment", "UsdPrim")};
		Parameters.PrimaryEditorType = "UsdPrim";
		FUsdPrimLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FUsdPrimLocatorFragment>(Parameters);

#if WITH_EDITOR
		IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::LoadModuleChecked<IUniversalObjectLocatorEditorModule>(
			"UniversalObjectLocatorEditor"
		);
		UolEditorModule.RegisterLocatorEditor("UsdPrim", MakeShared<FUsdPrimLocatorEditor>());

		LLM_SCOPE_BYTAG(Usd);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.RegisterCustomClassLayout(
			TEXT("UsdStageActor"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FUsdStageActorCustomization::MakeInstance)
		);

		Sequencers.Reset();
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		OnSequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(FOnSequencerCreated::FDelegate::CreateLambda(
			[this](TSharedRef<ISequencer> NewSequencer)
			{
				// Cleanup stale pointers
				for (int32 Index = Sequencers.Num() - 1; Index >= 0; --Index)
				{
					if (!Sequencers[Index].IsValid())
					{
						const int32 Count = 1;
						Sequencers.RemoveAt(Index, Count, EAllowShrinking::No);
					}
				}

				// Assuming the new one is valid given it's a TSharedRef
				Sequencers.Add(NewSequencer.ToWeakPtr());
			}
		));
#endif	  // WITH_EDITOR
	}

	virtual void ShutdownModule() override
	{
		using namespace UE::UniversalObjectLocator;

		if (IUniversalObjectLocatorModule* UolModule = FModuleManager::Get().GetModulePtr<IUniversalObjectLocatorModule>("UniversalObjectLocator"))
		{
			UolModule->UnregisterFragmentType(FUsdPrimLocatorFragment::FragmentType);
		}

#if WITH_EDITOR
		if (IUniversalObjectLocatorEditorModule* UolEditorModule = FModuleManager::Get().GetModulePtr<IUniversalObjectLocatorEditorModule>(
				"UniversalObjectLocatorEditor"
			))
		{
			UolEditorModule->UnregisterLocatorEditor("UsdPrim");
		}

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomClassLayout(TEXT("UsdStageActor"));

		if (ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer"))
		{
			SequencerModule->UnregisterOnSequencerCreated(OnSequencerCreatedHandle);
			Sequencers.Reset();
		}
#endif	  // WITH_EDITOR
	}

#if WITH_EDITOR
	// It would have been nice if the Sequencer module could provide this, but we can easily do this here too
	const TArray<TWeakPtr<ISequencer>>& GetExistingSequencers() const override
	{
		return Sequencers;
	}
#endif	  // WITH_EDITOR

	virtual AUsdStageActor& GetUsdStageActor(UWorld* World) override
	{
		if (AUsdStageActor* UsdStageActor = FindUsdStageActor(World))
		{
			return *UsdStageActor;
		}
		else
		{
			return *(World->SpawnActor<AUsdStageActor>());
		}
	}

	virtual AUsdStageActor* FindUsdStageActor(UWorld* World) override
	{
		for (FActorIterator ActorIterator(World); ActorIterator; ++ActorIterator)
		{
			if (AUsdStageActor* UsdStageActor = Cast<AUsdStageActor>(*ActorIterator))
			{
				return UsdStageActor;
			}
		}

		return nullptr;
	}

private:
#if WITH_EDITOR
	TArray<TWeakPtr<ISequencer>> Sequencers;
	FDelegateHandle OnSequencerCreatedHandle;
#endif	  // WITH_EDITOR
};

IMPLEMENT_MODULE_USD(FUsdStageModule, USDStage);
