// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDTransactor.h"

#include "UnrealUSDWrapper.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDListener.h"
#include "USDMemory.h"
#include "USDPrimConversion.h"
#include "USDSetUtils.h"
#include "USDStageActor.h"
#include "USDTypesConversion.h"
#include "USDValueConversion.h"
#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdRelationship.h"
#include "UsdWrappers/UsdStage.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Editor/TransBuffer.h"
#include "Misc/ITransaction.h"
#endif	  // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDTransactor)

#define LOCTEXT_NAMESPACE "USDTransactor"

const FName UE::UsdTransactor::ConcertSyncEnableTag = TEXT("EnableConcertSync");

namespace UsdUtils
{
	// Objects adapted from USDListener, since we use slightly different data here
	struct FTransactorAttributeChange
	{
		FString Field;
		FString AttributeTypeName;	   // Full SdfValueTypeName of the attribute (e.g. normal3f, bool, texCoord3d, float2) so that we can undo/redo
									   // attribute creation
		FConvertedVtValue OldValue;	   // Default old value
		FConvertedVtValue NewValue;	   // Default new value
		TArray<double> TimeSamples;
		TArray<FConvertedVtValue> TimeValues;	 // We can't fetch old/new values when time samples change, so we just have one of these and save the
												 // whole set every time
	};

	struct FTransactorObjectChange
	{
		TArray<FTransactorAttributeChange> FieldChanges;
		FPrimChangeFlags Flags;
		FString PrimTypeName;
		TArray<FString> PrimAppliedSchemas;
		FString OldPath;
	};

	enum class EObjectType
	{
		Prim,
		Attribute,
		Relationship
	};

	struct FTransactorRecordedEdit
	{
		EObjectType ObjectType;	   // Describes what object that ObjectPath is pointing to, and also what the "Field" values inside the ObjectChanges
								   // mean (e.g. if ObjectPath describes an Attribute, we know the ObjectChanges are attribute metadata changes)
		FString ObjectPath;
		TArray<FTransactorObjectChange> ObjectChanges;
	};

	struct FTransactorRecordedEdits
	{
		FString EditTargetIdentifier;
		FString IsolatedLayerIdentifier;
		TArray<FTransactorRecordedEdit> Edits;	  // Kept in the order of recording
	};

	using FTransactorEditStorage = TArray<FTransactorRecordedEdits>;

	enum class EApplicationDirection
	{
		Reverse,
		Forward
	};
}	 // namespace UsdUtils

FArchive& operator<<(FArchive& Ar, UsdUtils::FTransactorAttributeChange& Change)
{
	Ar << Change.Field;
	Ar << Change.AttributeTypeName;
	Ar << Change.OldValue;
	Ar << Change.NewValue;
	Ar << Change.TimeSamples;
	Ar << Change.TimeValues;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UsdUtils::FPrimChangeFlags& Flags)
{
	TArray<uint8> Bytes;
	if (Ar.IsLoading())
	{
		Ar << Bytes;

		check(Bytes.Num() == sizeof(Flags));
		FMemory::Memcpy(reinterpret_cast<uint8*>(&Flags), Bytes.GetData(), sizeof(Flags));
	}
	else if (Ar.IsSaving())
	{
		Bytes.SetNumZeroed(sizeof(Flags));
		FMemory::Memcpy(Bytes.GetData(), reinterpret_cast<uint8*>(&Flags), sizeof(Flags));

		Ar << Bytes;
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, UsdUtils::FTransactorObjectChange& Change)
{
	Ar << Change.FieldChanges;
	Ar << Change.Flags;
	Ar << Change.PrimTypeName;
	Ar << Change.PrimAppliedSchemas;
	Ar << Change.OldPath;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UsdUtils::FTransactorRecordedEdit& Edit)
{
	Ar << Edit.ObjectType;
	Ar << Edit.ObjectPath;
	Ar << Edit.ObjectChanges;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, UsdUtils::FTransactorRecordedEdits& Edits)
{
	Ar << Edits.EditTargetIdentifier;
	Ar << Edits.IsolatedLayerIdentifier;
	Ar << Edits.Edits;
	return Ar;
}

namespace UsdUtils
{
	/**
	 * Converts the received VtValue map to an analogue using converted UE types that can be serialized with the UUsdTransactor.
	 * Needs the stage because we need to manually fetch additional prim/attribute data, in order to support undo/redoing attribute creation
	 */
	bool ConvertFieldValueMap(const FObjectChangesByPath& InChanges, const UE::FUsdStage& InStage, FTransactorRecordedEdits& InOutEdits)
	{
		InOutEdits.Edits.Reserve(InChanges.Num() + InOutEdits.Edits.Num());

		for (const TPair<FString, TArray<FSdfChangeListEntry>>& ChangesByPrimPath : InChanges)
		{
			const FString& ObjectPath = ChangesByPrimPath.Key;
			const TArray<FSdfChangeListEntry>& Changes = ChangesByPrimPath.Value;

			UE::FSdfPath UEObjectPath{*ObjectPath};

			UE::FSdfPath PrimPath = UEObjectPath.GetAbsoluteRootOrPrimPath();
			UE::FUsdPrim Prim = InStage.GetPrimAtPath(PrimPath);

			FString PropertyName = UEObjectPath.IsPropertyPath() ? UEObjectPath.GetName() : FString{};

			FTransactorRecordedEdit& Edit = InOutEdits.Edits.Emplace_GetRef();
			Edit.ObjectPath = ObjectPath;
			Edit.ObjectType = UEObjectPath.IsAbsoluteRootOrPrimPath()		  ? EObjectType::Prim
							  : (Prim && Prim.HasRelationship(*PropertyName)) ? EObjectType::Relationship
																			  : EObjectType::Attribute;

			for (const FSdfChangeListEntry& Change : Changes)
			{
				FTransactorObjectChange& ConvertedChange = Edit.ObjectChanges.Emplace_GetRef();
				ConvertedChange.Flags = Change.Flags;
				ConvertedChange.OldPath = Change.OldPath;

				// Adding/removing prim --> Record prim info
				if (Prim
					&& (ConvertedChange.Flags.bDidAddInertPrim ||		//
						ConvertedChange.Flags.bDidAddNonInertPrim ||	//
						ConvertedChange.Flags.bDidRemoveInertPrim ||	//
						ConvertedChange.Flags.bDidRemoveNonInertPrim))
				{
					ConvertedChange.PrimTypeName = Prim.GetTypeName().ToString();

					TArray<FName> AppliedSchemaNames = Prim.GetAppliedSchemas();
					ConvertedChange.PrimAppliedSchemas.Reset(AppliedSchemaNames.Num());
					for (const FName SchemaName : AppliedSchemaNames)
					{
						ConvertedChange.PrimAppliedSchemas.Add(SchemaName.ToString());
					}

					USD_LOG_INFO(
						TEXT("Recorded the %s of prim '%s' with TypeName '%s' and PrimAppliedSchemas [%s]"),
						(ConvertedChange.Flags.bDidAddInertPrim || ConvertedChange.Flags.bDidAddNonInertPrim) ? TEXT("addition") : TEXT("removal"),
						*ObjectPath,
						*ConvertedChange.PrimTypeName,
						*FString::Join(ConvertedChange.PrimAppliedSchemas, TEXT(", "))
					);
				}

				if (Prim && ConvertedChange.Flags.bDidRename)
				{
					USD_LOG_INFO(
						TEXT("Recorded the rename of prim '%s' into '%s'"),
						*ConvertedChange.OldPath,
						*ObjectPath
					);
				}

				// Don't record any attribute changes if we can't find the prim anyway
				if (!Prim)
				{
					// We expect not to find the prim if the change says it was just removed though
					if (!ConvertedChange.Flags.bDidRemoveInertPrim && !ConvertedChange.Flags.bDidRemoveNonInertPrim)
					{
						USD_LOG_WARNING(TEXT("Failed to find prim at path '%s' when serializing object changes in transactor"), *ObjectPath);
					}
					continue;
				}

				for (const FFieldChange& FieldChange : Change.FieldChanges)
				{
					FTransactorAttributeChange& ConvertedAttributeChange = ConvertedChange.FieldChanges.Emplace_GetRef();
					ConvertedAttributeChange.Field = FieldChange.Field;

					FConvertedVtValue ConvertedOldValue;
					if (UsdToUnreal::ConvertValue(FieldChange.OldValue, ConvertedOldValue))
					{
						ConvertedAttributeChange.OldValue = ConvertedOldValue;
					}

					FConvertedVtValue ConvertedNewValue;
					if (UsdToUnreal::ConvertValue(FieldChange.NewValue, ConvertedNewValue))
					{
						ConvertedAttributeChange.NewValue = ConvertedNewValue;
					}

					// We likely won't be able to fetch the attribute if it was removed, so try deducing the typename from the value just so that we
					// have *something*
					if (!FieldChange.OldValue.IsEmpty()
						&& (ConvertedChange.Flags.bDidRemoveProperty ||	   //
							ConvertedChange.Flags.bDidRemovePropertyWithOnlyRequiredFields))
					{
						ConvertedAttributeChange.AttributeTypeName = UsdUtils::GetImpliedTypeName(FieldChange.OldValue);
						USD_LOG_INFO(
							TEXT(
								"Recording the removal of properties is not fully supported: Using underlying type '%s' for record of attribute '%s' of prim '%s', as we don't have access to the attribute's role"
							),
							*ConvertedAttributeChange.AttributeTypeName,
							*PropertyName,
							*ObjectPath
						);
					}

					// Record attribute typename/timeSamples if we can find it
					if (Edit.ObjectType == EObjectType::Attribute && !Prim.IsPseudoRoot() && !PropertyName.IsEmpty()
						&& !ConvertedChange.Flags.bDidRemoveProperty && !ConvertedChange.Flags.bDidRemovePropertyWithOnlyRequiredFields)
					{
						if (UE::FUsdAttribute Attribute = Prim.GetAttribute(*PropertyName))
						{
							ConvertedAttributeChange.AttributeTypeName = Attribute.GetTypeName().ToString();

							// USD doesn't tell us what changed, what type of change it was, or old/new values... so just save the entire timeSamples
							// of the attribute so we can mirror via multi-user
							if (ConvertedChange.Flags.bDidChangeAttributeTimeSamples)
							{
								Attribute.GetTimeSamples(ConvertedAttributeChange.TimeSamples);
								ConvertedAttributeChange.TimeValues.SetNum(ConvertedAttributeChange.TimeSamples.Num());

								for (int32 Index = 0; Index < ConvertedAttributeChange.TimeSamples.Num(); Index++)
								{
									UE::FVtValue UsdValue;
									if (Attribute.Get(UsdValue, ConvertedAttributeChange.TimeSamples[Index]))
									{
										UsdToUnreal::ConvertValue(UsdValue, ConvertedAttributeChange.TimeValues[Index]);
									}
								}
							}
						}
						else
						{
							USD_LOG_WARNING(
								TEXT("Failed to find attribute '%s' for prim at path '%s' when serializing object changes in transactor"),
								*PropertyName,
								*PrimPath.GetString()
							);
						}
					}
				}
			}
		}

		return true;
	}

	bool ApplyPrimChange(
		const UE::FSdfPath& PrimPath,
		const FTransactorObjectChange& PrimChange,
		UE::FUsdStage& Stage,
		EApplicationDirection Direction
	)
	{
		const bool bAdd = PrimChange.Flags.bDidAddInertPrim || PrimChange.Flags.bDidAddNonInertPrim;
		const bool bRemove = PrimChange.Flags.bDidRemoveInertPrim || PrimChange.Flags.bDidRemoveNonInertPrim;
		const bool bRename = PrimChange.Flags.bDidRename;

		if ((bAdd && Direction == EApplicationDirection::Forward) || (bRemove && Direction == EApplicationDirection::Reverse))
		{
			USD_LOG_INFO(TEXT("Creating prim '%s' with typename '%s'"), *PrimPath.GetString(), *PrimChange.PrimTypeName);

			UE::FUsdPrim Prim = Stage.DefinePrim(PrimPath, *PrimChange.PrimTypeName);
			if (!Prim)
			{
				return false;
			}

			return true;
		}
		else if ((bAdd && Direction == EApplicationDirection::Reverse) || (bRemove && Direction == EApplicationDirection::Forward))
		{
			USD_LOG_INFO(TEXT("Removing prim '%s' with typename '%s'"), *PrimPath.GetString(), *PrimChange.PrimTypeName);

			UsdUtils::RemoveAllLocalPrimSpecs(Stage.GetPrimAtPath(PrimPath), Stage.GetEditTarget());
			return true;
		}
		else if (bRename)
		{
			FString NewName;
			FString CurrentPath;

			if (Direction == EApplicationDirection::Forward)
			{
				// It hasn't been renamed yet, so it's still at the old path
				CurrentPath = PrimChange.OldPath;
				NewName = PrimPath.GetElementString();
			}
			else
			{
				CurrentPath = PrimPath.GetString();
				NewName = UE::FSdfPath(*PrimChange.OldPath).GetElementString();
			}

			// When redoing, we'll be using the old path, and USD sends it with all the variant selections in there
			// UsdUtils::RenamePrim can figure out the variant selections on its own, but we need to strip them here
			// to be able to GetPrimAtPath with this path
			UE::FSdfPath UsdCurrentPath = UE::FSdfPath(*CurrentPath).StripAllVariantSelections();

			if (UE::FUsdPrim Prim = Stage.GetPrimAtPath(UsdCurrentPath))
			{
				USD_LOG_INFO(TEXT("Renaming prim '%s' to '%s'"), *Prim.GetPrimPath().GetString(), *NewName);

				const bool bDidRename = UsdUtils::RenamePrim(Prim, *NewName);
				if (bDidRename)
				{
					return true;
				}
			}
			else if (UE::FUsdPrim TargetPrim = Stage.GetPrimAtPath(UsdCurrentPath.ReplaceName(*NewName)))
			{
				// We couldn't find a prim at the old path but found one at the new path, so just assume its the prim that we
				// wanted to rename anyway, as USD wouldn't have let us rename a prim onto an existing path in the first place.
				// This is useful because sometimes we may get multiple rename edits for the same prim in the same notice, like when we have
				// multiple specs per prim on the same layer.
				return true;
			}

			USD_LOG_WARNING(TEXT("Failed to rename prim at path '%s' to name '%s'"), *CurrentPath, *NewName);
		}

		return false;
	}

	bool ApplyAttributeTimeSamples(const FTransactorAttributeChange& AttributeChange, const UE::FSdfPath& ObjectPath, UE::FUsdPrim& Prim)
	{
		if (!Prim || AttributeChange.TimeSamples.Num() == 0 || AttributeChange.TimeSamples.Num() != AttributeChange.TimeValues.Num())
		{
			return false;
		}

		FString PropertyName = ObjectPath.GetName();

		// Try Getting first because we shouldn't trust our AttributeTypeName to always just CreateAttribute, as it may be just deduced from a value
		// and be different
		UE::FUsdAttribute Attribute = Prim.GetAttribute(*PropertyName);
		if (!Attribute)
		{
			Attribute = Prim.CreateAttribute(*PropertyName, *AttributeChange.AttributeTypeName);
		}
		if (!Attribute)
		{
			return false;
		}

		// Clear all timesamples because we may have more timesamples than we receive, and we want our old ones to be removed
		// This corresponds to the token SdfFieldKeys->TimeSamples, and is extracted from UsdAttribute::Clear
		Attribute.ClearMetadata(TEXT("timeSamples"));

		USD_LOG_INFO(
			TEXT("Applying '%d' timeSamples for attribute '%s' of prim '%s'"),
			AttributeChange.TimeSamples.Num(),
			*PropertyName,
			*Prim.GetPrimPath().GetString()
		);

		bool bSuccess = true;
		for (int32 Index = 0; Index < AttributeChange.TimeSamples.Num(); Index++)
		{
			UE::FVtValue Value;
			if (UnrealToUsd::ConvertValue(AttributeChange.TimeValues[Index], Value))
			{
				if (!Attribute.Set(Value, AttributeChange.TimeSamples[Index]))
				{
					USD_LOG_WARNING(
						TEXT("Failed to apply value '%s' at timesample '%f' for attribute '%s' of prim '%s'"),
						*UsdUtils::Stringify(Value),
						AttributeChange.TimeSamples[Index],
						*PropertyName,
						*Prim.GetPrimPath().GetString()
					);
					bSuccess = false;
				}
			}
			else
			{
				USD_LOG_WARNING(
					TEXT("Failed to convert value for timesample '%f' for attribute '%s' of prim '%s'"),
					AttributeChange.TimeSamples[Index],
					*PropertyName,
					*Prim.GetPrimPath().GetString()
				);
				bSuccess = false;
			}
		}

		return bSuccess;
	}

	bool ApplyAttributeChange(
		const UE::FUsdPrim& Prim,			 // This is always the leafmost, relevant prim. Regardless of ObjectType
		const UE::FSdfPath& ObjectPath,		 // Path to the object containing the field. Could be a property or prim
		const EObjectType ObjectType,		 // Describes what the ObjectPath is pointing to
		const FString& Field,				 // The actual field name. "default" for attr default values, or "kind"/"payload" for metadata, etc.
		const FString& AttributeTypeName,	 // If ObjectType == EObjectType::Attribute, this describes the type name of the attribute's values
		bool bRemoveProperty,				 // Whether to remove the property instead of just applying values
		const FConvertedVtValue& Value,		 // The value to apply to the field
		TOptional<double> Time = {}			 // The time at which to apply/clear the value to the field
	)
	{
		if (!Prim)
		{
			return false;
		}

		bool bCreated = false;

		FString PropertyName = ObjectPath.IsPropertyPath() ? ObjectPath.GetName() : FString{};
		UE::FUsdAttribute Attribute;
		UE::FUsdRelationship Relationship;

		if (bRemoveProperty)
		{
			if (ObjectType == EObjectType::Relationship)
			{
				PropertyName = ObjectPath.GetName();
				Relationship = Prim.GetRelationship(*PropertyName);
				if (!Relationship)
				{
					return true;
				}
			}
			else if (ObjectType == EObjectType::Attribute)
			{
				PropertyName = ObjectPath.GetName();
				Attribute = Prim.GetAttribute(*PropertyName);
				if (!Attribute)
				{
					return true;
				}
			}
		}
		else
		{
			if (ObjectType == EObjectType::Relationship)
			{
				bool bHadRelationship = Prim.HasRelationship(*PropertyName);
				Relationship = Prim.CreateRelationship(*PropertyName);
				if (!Relationship)
				{
					USD_LOG_WARNING(TEXT("Failed to create relationship '%s' with for prim '%s'"), *PropertyName, *Prim.GetPrimPath().GetString());

					return false;
				}

				bCreated = !bHadRelationship;
			}
			else if (ObjectType == EObjectType::Attribute)
			{
				bool bHadAttr = Prim.HasAttribute(*PropertyName);
				Attribute = Prim.CreateAttribute(*PropertyName, *AttributeTypeName);
				if (!Attribute)
				{
					// We expect to fail to create an attribute if we have no typename here (e.g. undo remove property)
					if (AttributeTypeName.IsEmpty())
					{
						USD_LOG_WARNING(
							TEXT("Failed to create attribute '%s' with typename '%s' for prim '%s'"),
							*PropertyName,
							*AttributeTypeName,
							*Prim.GetPrimPath().GetString()
						);
					}

					return false;
				}

				bCreated = !bHadAttr;
			}
		}

		UE::FVtValue WrapperValue;
		if (!UnrealToUsd::ConvertValue(Value, WrapperValue))
		{
			USD_LOG_WARNING(
				TEXT("Failed to convert VtValue back to USD when applying it to object '%s' field '%s'"),
				*ObjectPath.GetString(),
				*Field
			);
			return false;
		}

		USD_LOG_INFO(
			TEXT("%s object '%s' (typename '%s'), field '%s' with value '%s' at time '%s'"),
			bCreated				 ? TEXT("Creating")
			: WrapperValue.IsEmpty() ? bRemoveProperty ? TEXT("Removing") : TEXT("Clearing")
									 : TEXT("Setting"),
			*ObjectPath.GetString(),
			*AttributeTypeName,
			*Field,
			WrapperValue.IsEmpty() ? TEXT("<empty>") : *UsdUtils::Stringify(WrapperValue),
			Time.IsSet() ? *LexToString(Time.GetValue()) : TEXT("<unset>")
		);

		// If we just want to remove the property don't really bother doing anything else.
		// Note: We never get this flag set to true when dealing with metadata, only properties (attributes and relationships)
		if (bRemoveProperty && !PropertyName.IsEmpty())
		{
			Prim.RemoveProperty(*PropertyName);
			return true;
		}

		if (Field == TEXT("default"))
		{
			if (WrapperValue.IsEmpty())
			{
				if (Time.IsSet() && Attribute)
				{
					Attribute.ClearAtTime(Time.GetValue());
				}
				else if (Attribute)
				{
					Attribute.Clear();
				}
			}
			else if (Attribute)
			{
				Attribute.Set(WrapperValue, Time);
			}
		}
		// This seems to be the field name for the actual value in pxr:UsdRelationship
		else if (ObjectType == EObjectType::Relationship && Field == TEXT("targetPaths"))
		{
			if (WrapperValue.IsEmpty())
			{
				if (Attribute)
				{
					Attribute.Clear();
				}
				else if (Relationship)
				{
					bool bRemoveSpec = false;
					Relationship.ClearTargets(bRemoveSpec);
				}
			}
			else
			{
				// We have to manually convert from the TArray<FString> that our ConvertedValue is holding,
				// as unlike for UE::FUsdAttribute, we can't just feed a VtValue into the UE::FUsdRelationship
				if (Value.SourceType == EUsdBasicDataTypes::String && Value.bIsArrayValued)
				{
					TArray<UE::FSdfPath> Targets;
					for (const FConvertedVtValueEntry& Entry : Value.Entries)
					{
						// For the relationship values we always put a single component per entry
						if (Entry.Num() == 1)
						{
							const FConvertedVtValueComponent& Component = Entry[0];
							if (const FString* HeldString = Component.TryGet<FString>())
							{
								Targets.Add(UE::FSdfPath{**HeldString});
							}
						}
					}

					Relationship.SetTargets(Targets);
				}
			}
		}
		else	// Other metadata fields (on prims or properties)
		{
			if (WrapperValue.IsEmpty())
			{
				if (Attribute)
				{
					Attribute.ClearMetadata(*Field);
				}
				else if (Relationship)
				{
					Relationship.ClearMetadata(*Field);
				}
				else
				{
					Prim.ClearMetadata(*Field);
				}
			}
			else
			{
				if (Attribute)
				{
					Attribute.SetMetadata(*Field, WrapperValue);
				}
				else if (Relationship)
				{
					Relationship.SetMetadata(*Field, WrapperValue);
				}
				else
				{
					Prim.SetMetadata(*Field, WrapperValue);
				}
			}
		}

		return true;
	}

	bool ApplyStageMetadataChange(const FString& FieldName, const FConvertedVtValue& Value, UE::FUsdStage& Stage)
	{
		if (!Stage || FieldName.IsEmpty())
		{
			return false;
		}

		UE::FVtValue WrapperValue;
		if (!UnrealToUsd::ConvertValue(Value, WrapperValue))
		{
			USD_LOG_WARNING(TEXT("Failed to convert VtValue back to USD when applying it to stage metadata field '%s'"), *FieldName);
			return false;
		}

		USD_LOG_INFO(TEXT("Setting stage metadata '%s', with value '%s'"), *FieldName, *UsdUtils::Stringify(WrapperValue));

		UE::FSdfLayer OldEditTarget = Stage.GetEditTarget();
		Stage.SetEditTarget(Stage.GetRootLayer());	  // Stage metadata always needs to be set at the root layer
		{
			if (WrapperValue.IsEmpty())
			{
				Stage.ClearMetadata(*FieldName);
			}
			else
			{
				Stage.SetMetadata(*FieldName, WrapperValue);
			}
		}
		Stage.SetEditTarget(OldEditTarget);

		return true;
	}

	/** Applies the field value pairs to all prims on the stage, and returns a list of prim paths for modified prims */
	UsdUtils::FCaseSensitiveStringSet ApplyFieldMapToStage(const FTransactorEditStorage& EditStorage, EApplicationDirection Direction, AUsdStageActor* StageActor)
	{
		UE::FUsdStage& Stage = StageActor->GetOrOpenUsdStage();
		if (!Stage)
		{
			return {};
		}

		UsdUtils::FCaseSensitiveStringSet PrimsChanged;

		int32 Start = 0;
		int32 End = 0;
		int32 Incr = 0;
		if (Direction == EApplicationDirection::Forward)
		{
			Start = 0;
			End = EditStorage.Num();
			Incr = 1;
		}
		else
		{
			Start = EditStorage.Num() - 1;
			End = -1;
			Incr = -1;
		}

		UE::FSdfLayer OldEditTarget = Stage.GetEditTarget();
		FString LastEditTargetIdentifier;

		for (int32 EditIndex = Start; EditIndex != End; EditIndex += Incr)
		{
			const FTransactorRecordedEdits& Edits = EditStorage[EditIndex];

			// Isolate the correct layer (empty string here means to stop isolating / don't isolate anything)
			const FString CurrentIsolatedLayer = StageActor->GetIsolatedRootLayer();
			if (CurrentIsolatedLayer != Edits.IsolatedLayerIdentifier)
			{
				UE::FSdfLayer LayerToIsolate = Edits.IsolatedLayerIdentifier.IsEmpty() ? UE::FSdfLayer{}
																					   : UE::FSdfLayer::FindOrOpen(*Edits.IsolatedLayerIdentifier);

				// Don't load here, as we'll get our assets, actors and components back from the UE transaction being
				// undone/redone already
				const bool bLoadUsdStage = false;
				StageActor->IsolateLayer(LayerToIsolate, bLoadUsdStage);
			}

			if (Edits.EditTargetIdentifier != LastEditTargetIdentifier)
			{
#if USE_USD_SDK
				UE::FSdfLayer EditTarget = UsdUtils::FindLayerForIdentifier(*Edits.EditTargetIdentifier, Stage);
				if (!EditTarget)
				{
					USD_LOG_WARNING(
						TEXT("Ignoring application of recorded USD stage changes as the edit target with identifier '%s' cannot be found or opened"),
						*Edits.EditTargetIdentifier
					)
					continue;
				}

				Stage.SetEditTarget(EditTarget);
				LastEditTargetIdentifier = Edits.EditTargetIdentifier;
#endif	  // USE_USD_SDK
			}

			for (const FTransactorRecordedEdit& Edit : Edits.Edits)
			{
				if (Edit.ObjectChanges.Num() == 0)
				{
					continue;
				}

				PrimsChanged.Add(Edit.ObjectPath);

				if (Edit.ObjectPath == TEXT("/"))
				{
					for (const FTransactorObjectChange& PrimChange : Edit.ObjectChanges)
					{
						for (const FTransactorAttributeChange& AttributeChange : PrimChange.FieldChanges)
						{
							ApplyStageMetadataChange(
								AttributeChange.Field,
								Direction == EApplicationDirection::Forward ? AttributeChange.NewValue : AttributeChange.OldValue,
								Stage
							);
						}
					}
				}
				else
				{
					for (const FTransactorObjectChange& PrimChange : Edit.ObjectChanges)
					{
						UE::FSdfPath UEObjectPath = UE::FSdfPath{*Edit.ObjectPath};
						UE::FSdfPath PrimPath = UEObjectPath.GetAbsoluteRootOrPrimPath();
						if (ApplyPrimChange(PrimPath, PrimChange, Stage, Direction))
						{
							// If we managed to apply a prim change, we know there aren't any other attribute changes in the same ObjectChange
							continue;
						}

						UE::FUsdPrim Prim = Stage.GetPrimAtPath(PrimPath);
						if (!Prim)
						{
							continue;
						}

						// Whether we should remove a property instead of clearing the opinion, when asked to apply an empty value
						bool bShouldRemove = Direction == EApplicationDirection::Forward
												 ? PrimChange.Flags.bDidRemoveProperty || PrimChange.Flags.bDidRemovePropertyWithOnlyRequiredFields
												 : PrimChange.Flags.bDidAddProperty || PrimChange.Flags.bDidAddPropertyWithOnlyRequiredFields;

						// Can't block more than this, as Defining prims (from within ApplyPrimChange) needs to trigger its notices immediately,
						// and our changes/edits may depend on previous changes/edits triggering
						UE::FSdfChangeBlock ChangeBlock;

						if (PrimChange.Flags.bDidChangeAttributeTimeSamples)
						{
							for (const FTransactorAttributeChange& AttributeChange : PrimChange.FieldChanges)
							{
								ApplyAttributeTimeSamples(AttributeChange, UEObjectPath, Prim);
							}
						}
						else
						{
							for (const FTransactorAttributeChange& FieldChange : PrimChange.FieldChanges)
							{
								ApplyAttributeChange(
									Prim,
									UEObjectPath,
									Edit.ObjectType,
									FieldChange.Field,
									FieldChange.AttributeTypeName,
									bShouldRemove,
									Direction == EApplicationDirection::Forward ? FieldChange.NewValue : FieldChange.OldValue
								);
							}
						}
					}
				}
			}
		}

		Stage.SetEditTarget(OldEditTarget);

		return PrimsChanged;
	}

	// Class aware of undo/redo/ConcertSync that handles serializing/applying the received FTransactorRecordedEdits data.
	// We need the awareness because we respond to undo from PreEditUndo, and respond to redo from PostEditUndo. This in turn because:
	// - In PreEditUndo we still have OldValues of the current transaction, and to undo we want to apply those OldValues to the stage;
	// - In PostEditUndo we have the NewValues of the next transaction, and to redo we want to apply those NewValues to the stage;
	// - ConcertSync always applies changes and then calls PostEditUndo, and to sync we want to apply those received NewValues to the stage.
	class FUsdTransactorImpl
	{
	public:
		FUsdTransactorImpl();
		~FUsdTransactorImpl();

		void Update(FTransactorRecordedEdits&& NewEdits);
		void Serialize(FArchive& Ar);
		void PreEditUndo(AUsdStageActor* StageActor);
		void PostEditUndo(AUsdStageActor* StageActor);

		// Returns whether the transaction buffer is currently in the middle of an Undo operation.
		// WARNING: This approach is only accurate if we're checking from within PreEditUndo/PostEditUndo/PostTransacted/Serialize (which we always do
		// in this file)
		bool IsTransactionUndoing();

		// Returns whether the transaction buffer is currently in the middle of a Redo operation. Returns false when we're applying
		// a ConcertSync transaction, even though concert sync sort of works by applying transactions via Redo
		// WARNING: This approach is only accurate if we're checking from within PreEditUndo/PostEditUndo/PostTransacted/Serialize (which we always do
		// in this file)
		bool IsTransactionRedoing();

		// Returns whether ConcertSync (multi-user) is currently applying a transaction received from the network
		bool IsApplyingConcertSyncTransaction()
		{
			return bApplyingConcertSync;
		};

	private:
		// This is called after *any* undo/redo transaction is finalized, so our LastFinalizedUndoCount is kept updated
		void HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState);
		int32 LastFinalizedUndoCount = 0;

		// We use this to detect when a ConcertSync transaction is starting, as it has a particular title
		void HandleBeforeOnRedoUndo(const FTransactionContext& TransactionContext);

		// We use this to detect when a ConcertSync transaction has ended, as it has a particular title
		void HandleOnRedo(const FTransactionContext& TransactionContext, bool bSucceeded);

	private:
		// Main data storage container
		FTransactorEditStorage Values;

		// We use these to stash our Values before they're overwritten by ConcertSync, and to restore them afterwards.
		// This is because when we receive a ConcertSync transaction the UUsdTransactor's values will be overwritten with the received data.
		// That is ok because we want to apply it to the stage, but after that we want to discard those values altogether, so that if *we*
		// undo, we won't undo the received transaction, but instead undo the last transaction that *we* made.
		FTransactorEditStorage StoredValues;

		// When ClientA undoes a change, it handles it's own undo changes from its PreEditUndo, but its final state after the undo transaction
		// is complete will have the *previous* OldValues/NewValues. This final state is what is sent over the network. ClientB that receives this
		// can't use these previous OldValues/NewValues to undo the change that ClientA just undone: It needs something else, which this
		// member provides. When ClientA starts to undo, it will stash it's *current* OldValues in here, and make sure they are visible when
		// serialized by ConcertSync. ClientB will receive these, and when available will apply those to the scene instead, undoing the same changes
		// that ClientA undone.
		TOptional<FTransactorEditStorage> ReceivedValuesBeforeUndo;

		// During the same transaction we continuously append the received change info into the same storage. When the transaction changes, we clear
		// it
		FGuid LastTransactionId;

		bool bApplyingConcertSync = false;
	};

	FUsdTransactorImpl::FUsdTransactorImpl()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
			{
				Transactor->OnTransactionStateChanged().AddRaw(this, &FUsdTransactorImpl::HandleTransactionStateChanged);
				Transactor->OnBeforeRedoUndo().AddRaw(this, &FUsdTransactorImpl::HandleBeforeOnRedoUndo);
				Transactor->OnRedo().AddRaw(this, &FUsdTransactorImpl::HandleOnRedo);
			}
		}
#endif	  // WITH_EDITOR
	}

	FUsdTransactorImpl::~FUsdTransactorImpl()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
			{
				Transactor->OnTransactionStateChanged().RemoveAll(this);
				Transactor->OnBeforeRedoUndo().RemoveAll(this);
				Transactor->OnRedo().RemoveAll(this);
			}
		}
#endif	  // WITH_EDITOR
	}

	void FUsdTransactorImpl::Update(FTransactorRecordedEdits&& NewEdits)
	{
		if (!GUndo)
		{
			return;
		}

		// New transaction -> Start a new storage
		FTransactionContext Context = GUndo->GetContext();
		if (Context.TransactionId != LastTransactionId)
		{
			LastTransactionId = Context.TransactionId;
			Values.Reset();
		}

		Values.Add(NewEdits);
	}

	void FUsdTransactorImpl::Serialize(FArchive& Ar)
	{
		Ar << Values;

		// If we have some ReceivedValuesBeforeUndo and the undo system is trying to overwrite it with its old version to apply the undo, keep our
		// values instead! We need this data to be with us whenever the ConcertSync serializes us to send it over the network during an undo, which
		// happens shortly after this
		if (Ar.IsTransacting() && Ar.IsLoading() && IsTransactionUndoing() && ReceivedValuesBeforeUndo.IsSet())
		{
			TOptional<FTransactorEditStorage> Dummy = {};
			Ar << Dummy;
		}
		else
		{
			Ar << ReceivedValuesBeforeUndo;
		}
	}

	void FUsdTransactorImpl::PreEditUndo(AUsdStageActor* StageActor)
	{
		if (!StageActor)
		{
			return;
		}

		if (IsTransactionUndoing())
		{
			// We can't respond to notices from the attribute that we'll set. Whatever changes setting the attribute causes in UE
			// actors/components/assets will already be accounted for by those actors/components/assets undoing/redoing by themselves via the UE
			// transaction buffer.
			FScopedBlockNotices BlockNotices(StageActor->GetUsdListener());

			UsdUtils::FCaseSensitiveStringSet PrimsChanged = UsdUtils::ApplyFieldMapToStage(	//
				Values,
				UsdUtils::EApplicationDirection::Reverse,
				StageActor
			);

			// Partial rebuild of the info cache after we have undone the USD stage changes for this transaction
			StageActor->RebuildInfoCacheFromStoredChanges();

			if (PrimsChanged.Num() > 0)
			{
				for (const FString& Prim : PrimsChanged)
				{
					StageActor->OnPrimChanged.Broadcast(Prim, false);
				}
			}

			// Make sure our OldValues survive the undo in case we need to send them over ConcertSync once the transaction is complete
			ReceivedValuesBeforeUndo = Values;
		}
		else
		{
			ReceivedValuesBeforeUndo.Reset();

			// ConcertSync calls PreEditUndo, then updates our data with the received data, then calls PostEditUndo
			if (IsApplyingConcertSyncTransaction())
			{
				// Make sure that our own Values survive when overwritten by values that we will receive from ConcertSync.
				// We'll restore this to our values once the ConcertSync action has finished applying
				StoredValues = Values;
			}
		}
	}

	void FUsdTransactorImpl::PostEditUndo(AUsdStageActor* StageActor)
	{
		const bool bIsRedoing = IsTransactionRedoing();
		const bool bIsApplyingConcertSync = IsApplyingConcertSyncTransaction();

		if (StageActor && (bIsRedoing || bIsApplyingConcertSync))
		{
			// If we're just redoing it's a bit of a waste to let the AUsdStageActor respond to notices from the fields that we'll set,
			// because any relevant changes caused to the level/assets would be redone by themselves if the actors/assets are also in the transaction
			// buffer. If we're receiving a ConcertSync transaction, however, we do want to respond to notices because transient actors/assets
			// aren't tracked by ConcertSync
			TOptional<FScopedBlockNotices> BlockNotices;
			if (bIsRedoing)
			{
				BlockNotices.Emplace(StageActor->GetUsdListener());
			}

			UE::FUsdStage& Stage = StageActor->GetOrOpenUsdStage();

			UsdUtils::FCaseSensitiveStringSet PrimsChanged;
			if (bIsApplyingConcertSync && ReceivedValuesBeforeUndo.IsSet())
			{
				// If we're applying a received ConcertSync transaction that actually is an undo on the source client then we want to use it's
				// ReceivedValuesBeforeUndo to replicate the same undo that they did
				PrimsChanged = UsdUtils::ApplyFieldMapToStage(
					ReceivedValuesBeforeUndo.GetValue(),
					UsdUtils::EApplicationDirection::Reverse,
					StageActor
				);
			}
			else
			{
				// Just a common Redo operation or any other type of ConcertSync transaction, so just apply the new values
				PrimsChanged = UsdUtils::ApplyFieldMapToStage(Values, UsdUtils::EApplicationDirection::Forward, StageActor);

				// Partial rebuild of the info cache after we have redone the USD stage changes for this transaction
				StageActor->RebuildInfoCacheFromStoredChanges();
			}

			// If we're redoing or applying ConcertSync we don't want to end up with these values when the transaction finalizes as it could be
			// replicated to other clients
			ReceivedValuesBeforeUndo.Reset();

			if (PrimsChanged.Num() > 0)
			{
				for (const FString& Prim : PrimsChanged)
				{
					StageActor->OnPrimChanged.Broadcast(Prim, false);
				}
			}
		}

		if (bIsApplyingConcertSync)
		{
			// If we're finishing applying a ConcertSync transaction, revert our Values to the state that they were before we received
			// the ConcertSync transaction. This is important so that if we undo now, we undo the last change that *we* made
			Values = StoredValues;
		}
	}

	bool FUsdTransactorImpl::IsTransactionUndoing()
	{
#if WITH_EDITOR
		if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
		{
			// We moved away from the end of the transaction buffer -> We're undoing
			if (GIsTransacting && Transactor->UndoCount > LastFinalizedUndoCount)
			{
				return true;
			}
		}
#endif	  // WITH_EDITOR

		return false;
	}

	bool FUsdTransactorImpl::IsTransactionRedoing()
	{
#if WITH_EDITOR
		if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
		{
			// We moved towards the end of the transaction buffer -> We're redoing
			if (GIsTransacting && Transactor->UndoCount < LastFinalizedUndoCount)
			{
				return true;
			}
		}
#endif	  // WITH_EDITOR

		return false;
	}

	void FUsdTransactorImpl::HandleTransactionStateChanged(
		const FTransactionContext& InTransactionContext,
		const ETransactionStateEventType InTransactionState
	)
	{
#if WITH_EDITOR
		if (InTransactionState == ETransactionStateEventType::UndoRedoFinalized
			|| InTransactionState == ETransactionStateEventType::TransactionFinalized)
		{
			if (UTransBuffer* Transactor = Cast<UTransBuffer>(GEditor->Trans))
			{
				// Recording UndoCount works because UTransBuffer::Undo preemptively updates it *before* calling any object function, like
				// PreEditUndo/PostEditUndo, so from within PreEditUndo/PostEditUndo we will always have a delta from this value to the value that is
				// recorded after any transaction was finalized, which we record right here
				LastFinalizedUndoCount = Transactor->UndoCount;
			}
		}
#endif	  // WITH_EDITOR
	}

	// Must be the same as the title used within ConcertClientTransactionBridgeUtil::ProcessTransactionEvent
	const FText ConcertSyncTransactionTitle = LOCTEXT("ConcertTransactionEvent", "Concert Transaction Event");

	void FUsdTransactorImpl::HandleBeforeOnRedoUndo(const FTransactionContext& TransactionContext)
	{
		if (TransactionContext.Title.EqualTo(ConcertSyncTransactionTitle))
		{
			bApplyingConcertSync = true;
		}
	}

	void FUsdTransactorImpl::HandleOnRedo(const FTransactionContext& TransactionContext, bool bSucceeded)
	{
		if (bApplyingConcertSync && TransactionContext.Title.EqualTo(ConcertSyncTransactionTitle))
		{
			bApplyingConcertSync = false;
		}
	}
}	 // namespace UsdUtils

// Boilerplate to support Pimpl in an UObject
UUsdTransactor::UUsdTransactor(FVTableHelper& Helper)
	: Super(Helper)
{
}
UUsdTransactor::UUsdTransactor()
{
#if USE_USD_SDK
	Impl = MakeUnique<UsdUtils::FUsdTransactorImpl>();
#endif
}
UUsdTransactor::~UUsdTransactor() = default;

void UUsdTransactor::Initialize(AUsdStageActor* InStageActor)
{
	StageActor = InStageActor;
}

void UUsdTransactor::Update(const UsdUtils::FObjectChangesByPath& NewInfoChanges, const UsdUtils::FObjectChangesByPath& NewResyncChanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UUsdTransactor::Update);

	// We always send notices even when we're undoing/redoing changes (so that multi-user can broadcast them).
	// Make sure that we only ever update our OldValues/NewValues when we receive *new* updates though
	if (Impl.IsValid() && (Impl->IsTransactionUndoing() || Impl->IsTransactionRedoing() || Impl->IsApplyingConcertSyncTransaction()))
	{
		return;
	}

	// In case we close a stage in the same transaction where the actor is destroyed - our UE::FUsdStage could turn invalid at any point otherwise
	// Not much else we can do as this will get to us before the StageActor's destructor/Destroyed are called
	AUsdStageActor* StageActorPtr = StageActor.Get();
	if (!StageActorPtr || StageActorPtr->IsActorBeingDestroyed())
	{
		return;
	}

	Modify();

	const UE::FUsdStage& Stage = StageActorPtr->GetOrOpenUsdStage();
	if (!Stage)
	{
		return;
	}

	const UE::FSdfLayer& EditTarget = Stage.GetEditTarget();

	UsdUtils::FTransactorRecordedEdits NewEdits;
	NewEdits.EditTargetIdentifier = EditTarget.GetIdentifier();
	NewEdits.IsolatedLayerIdentifier = StageActorPtr->GetIsolatedRootLayer();

	UsdUtils::ConvertFieldValueMap(NewInfoChanges, Stage, NewEdits);
	UsdUtils::ConvertFieldValueMap(NewResyncChanges, Stage, NewEdits);

	Impl->Update(MoveTemp(NewEdits));
}

void UUsdTransactor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Impl.IsValid())
	{
		Impl->Serialize(Ar);
	}
}

#if WITH_EDITOR

void UUsdTransactor::PreEditUndo()
{
	if (Impl.IsValid())
	{
		Impl->PreEditUndo(StageActor.Get());
	}

	Super::PreEditUndo();
}

void UUsdTransactor::PostEditUndo()
{
	if (Impl.IsValid())
	{
		Impl->PostEditUndo(StageActor.Get());
	}

	Super::PostEditUndo();
}
#endif	  // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
