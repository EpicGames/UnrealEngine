// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"

#include "USDDynamicBindingResolverLibrary.generated.h"

#define UE_API USDSTAGE_API

struct FMovieSceneDynamicBindingResolveParams;

UCLASS(meta = (SequencerBindingResolverLibrary))

class UUsdDynamicBindingResolverLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Resolves a Sequencer Dynamic Binding described on Params, returning the actor or component that the Sequencer
	 * should bind to
	 * @param WorldContextObject - Some UObject that lives in the UWorld that we're talking about
	 * @param Params - The binding to resolve
	 * @param StageActorIDNameFilter - The "ID Name"/FName of a AUsdStageActor to restrict our search for actor and components to. Can be
	 * empty.
	 * @param RootLayerFilter - The root layer file path to restrict our search for actor and components to. This should match what is shown
	 * on the Stage Actor's RootLayer property. Can be empty.
	 * @param PrimPath - The path to the prim that generates the actor or component that this binding should resolve to
	 * (e.g. '/cube')
	 * @return The result struct containing the resolved UObject
	 */
	UFUNCTION(BlueprintPure, Category = "USD|Dynamic Binding", meta = (WorldContext = "WorldContextObject"))
	static UE_API FMovieSceneDynamicBindingResolveResult ResolveWithStageActor(
		UObject* WorldContextObject,
		const FMovieSceneDynamicBindingResolveParams& Params,
		const FString& StageActorIDNameFilter,
		const FString& RootLayerFilter,
		const FString& PrimPath
	);
};

#undef UE_API
