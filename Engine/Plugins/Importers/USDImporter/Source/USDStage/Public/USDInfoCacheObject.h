// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"

#include "USDInfoCacheObject.generated.h"

#define UE_API USDSTAGE_API

class FUsdInfoCache;
namespace UE
{
	class FSdfPath;
}

/**
 * Minimal UObject wrapper around FUsdInfoCache, since we want this data to be
 * owned by an independently serializable UObject, but the implementation must be in
 * an RTTI-enabled module
 */
UCLASS(MinimalAPI)
class UUsdInfoCache : public UObject
{
	GENERATED_BODY()

public:
	UE_API UUsdInfoCache();

	// Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End UObject interface

	UE_API FUsdInfoCache& GetInner();
	UE_API const FUsdInfoCache& GetInner() const;

private:
	TPimplPtr<FUsdInfoCache> Impl;
};

#undef UE_API
