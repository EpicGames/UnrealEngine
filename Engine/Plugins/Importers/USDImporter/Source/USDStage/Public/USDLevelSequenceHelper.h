// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#include "UsdWrappers/ForwardDeclarations.h"

#define UE_API USDSTAGE_API

class AUsdStageActor;
class FScopedBlockMonitoringChangesForTransaction;
class FUsdInfoCache;
class UUsdPrimLinkCache;
class FUsdLevelSequenceHelperImpl;
class ULevelSequence;
class UUsdPrimTwin;
enum class EUsdRootMotionHandling : uint8;

namespace UE
{
	class FUsdGeomBBoxCache;
}

/**
 * Builds and maintains the level sequence and subsequences for a Usd Stage
 */
class FUsdLevelSequenceHelper
{
public:
	UE_API FUsdLevelSequenceHelper();
	UE_API virtual ~FUsdLevelSequenceHelper();

	// Copy semantics are there for convenience only. Copied FUsdLevelSequenceHelper are empty and require a call to Init().
	UE_API FUsdLevelSequenceHelper(const FUsdLevelSequenceHelper& Other);
	UE_API FUsdLevelSequenceHelper& operator=(const FUsdLevelSequenceHelper& Other);

	UE_API FUsdLevelSequenceHelper(FUsdLevelSequenceHelper&& Other);
	UE_API FUsdLevelSequenceHelper& operator=(FUsdLevelSequenceHelper&& Other);

public:
	/** Creates the main level sequence and subsequences from the usd stage layers */
	UE_API ULevelSequence* Init(const UE::FUsdStage& UsdStage);

	/** Allows serialization for transaction support */
	UE_API bool Serialize(FArchive& Ar);

	UE_API void SetPrimLinkCache(UUsdPrimLinkCache* PrimLinkCache);

	/** Sets the BBoxCache to use when importing bound animations for prims. Needed for importing, where we don't have a stage actor to take the
	 * BBoxCache from */
	UE_API void SetBBoxCache(TSharedPtr<UE::FUsdGeomBBoxCache> InBBoxCache);

	/* Returns true if we have at least one possessable or a reference to a subsequence */
	UE_API bool HasData() const;

	/** Call this whenever the stage actor is renamed, to replace the possessable binding with a new one */
	UE_API void OnStageActorRenamed();

	/** Resets the helper, abandoning all managed LevelSequences */
	UE_API void Clear();

	/** Creates the time track for the StageActor. This will also set the root handling mode to the provided actor's, if any */
	UE_API void BindToUsdStageActor(AUsdStageActor* StageActor);
	UE_API void UnbindFromUsdStageActor();

	/**
	 * Gets the current root motion handling mode.
	 * We use this to prevent animation that has already been baked into AnimSequences as root joint motion from also
	 * being parsed as LevelSequence tracks.
	 */
	UE_API EUsdRootMotionHandling GetRootMotionHandling() const;

	/**
	 * Sets the current root motion handling mode.
	 */
	UE_API void SetRootMotionHandling(EUsdRootMotionHandling NewValue);

	/**
	 * Adds the necessary tracks for a given prim to the level sequence.
	 * If bForceVisibilityTracks is true, will add visibility tracks even if this prim
	 * doesn't actually have timeSamples on its visibility attribute (use this when
	 * a parent does have animated visibility, and we need to "bake" that out to a dedicated
	 * visibility track so that the standalone LevelSequence asset behaves as expected).
	 * We'll check the prim for animated bounds and add bounds tracks, unless bHasAnimatedBounds
	 * already provides whether the prim has animated bounds or not.
	 */
	UE_API void AddPrim(UUsdPrimTwin& PrimTwin, bool bForceVisibilityTracks = false, TOptional<bool> HasAnimatedBounds = {});

	/** Removes any track associated with this prim */
	UE_API void RemovePrim(const UUsdPrimTwin& PrimTwin);

	UE_API void UpdateControlRigTracks(UUsdPrimTwin& PrimTwin);

	/** Blocks updating the level sequences & tracks from object changes. */
	UE_API void StartMonitoringChanges();
	UE_API void StopMonitoringChanges();
	UE_API void BlockMonitoringChangesForThisTransaction();

	UE_API ULevelSequence* GetMainLevelSequence() const;
	UE_API TArray<ULevelSequence*> GetSubSequences() const;

	DECLARE_EVENT_OneParam(FUsdLevelSequenceHelper, FOnSkelAnimationBaked, const FString& /*SkeletonPrimPath*/);
	UE_API FOnSkelAnimationBaked& GetOnSkelAnimationBaked();

private:
	friend class FScopedBlockMonitoringChangesForTransaction;
	TUniquePtr<FUsdLevelSequenceHelperImpl> UsdSequencerImpl;
};

class FScopedBlockMonitoringChangesForTransaction final
{
public:
	UE_API explicit FScopedBlockMonitoringChangesForTransaction(FUsdLevelSequenceHelper& InHelper);
	UE_API explicit FScopedBlockMonitoringChangesForTransaction(FUsdLevelSequenceHelperImpl& InHelperImpl);
	UE_API ~FScopedBlockMonitoringChangesForTransaction();

	FScopedBlockMonitoringChangesForTransaction() = delete;
	FScopedBlockMonitoringChangesForTransaction(const FScopedBlockMonitoringChangesForTransaction&) = delete;
	FScopedBlockMonitoringChangesForTransaction(FScopedBlockMonitoringChangesForTransaction&&) = delete;
	FScopedBlockMonitoringChangesForTransaction& operator=(const FScopedBlockMonitoringChangesForTransaction&) = delete;
	FScopedBlockMonitoringChangesForTransaction& operator=(FScopedBlockMonitoringChangesForTransaction&&) = delete;

private:
	FUsdLevelSequenceHelperImpl& HelperImpl;
	bool bStoppedMonitoringChanges = false;
};

#undef UE_API
