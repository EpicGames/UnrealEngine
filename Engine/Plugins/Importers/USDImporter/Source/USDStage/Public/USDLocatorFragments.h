// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"

#include "USDLocatorFragments.generated.h"

class AUsdStageActor;

/**
 * 32 Bytes (40 in-editor).
 */
USTRUCT()
struct FUsdPrimLocatorFragment
{
	GENERATED_BODY()

	UPROPERTY()
	bool bPreferComponent = false;

	UPROPERTY()
	FString PrimPath;

	USDSTAGE_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FUsdPrimLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FUsdPrimLocatorFragment& A)
	{
		return GetTypeHash(A.PrimPath) ^ GetTypeHash(A.bPreferComponent);
	}

	friend bool operator==(const FUsdPrimLocatorFragment& A, const FUsdPrimLocatorFragment& B)
	{
		return A.bPreferComponent == B.bPreferComponent && A.PrimPath == B.PrimPath;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);

	void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);

	static uint32 ComputePriority(const UObject* Object, const UObject* Context);

	static const AUsdStageActor* GetAttachParentStageActor(const UObject* Object);
	static AUsdStageActor* GetAttachParentStageActor(UObject* Object);
};

template<>
struct TStructOpsTypeTraits<FUsdPrimLocatorFragment> : public TStructOpsTypeTraitsBase2<FUsdPrimLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};
