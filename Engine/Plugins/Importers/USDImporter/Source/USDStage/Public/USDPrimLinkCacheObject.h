// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"

#include "USDPrimLinkCacheObject.generated.h"

#define UE_API USDSTAGE_API

class FUsdPrimLinkCache;
namespace UE
{
	class FSdfPath;
}

/**
 * Minimal UObject wrapper around FUsdPrimLinkCache, since we want this to be accessible
 * from USDSchemas which is an RTTI-enabled module, but also to be owned by an independently
 * serializable UObject
 */
UCLASS(MinimalAPI)
class UUsdPrimLinkCache : public UObject
{
	GENERATED_BODY()

public:
	UE_API UUsdPrimLinkCache();

	// Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End UObject interface

	UE_API FUsdPrimLinkCache& GetInner();
	UE_API const FUsdPrimLinkCache& GetInner() const;

private:
	TPimplPtr<FUsdPrimLinkCache> Impl;
};

#undef UE_API
