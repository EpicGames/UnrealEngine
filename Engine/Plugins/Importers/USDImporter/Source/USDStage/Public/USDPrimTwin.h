// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "UObject/WeakObjectPtr.h"

#include "USDPrimTwin.generated.h"

/** The Unreal equivalent (twin) of a USD prim */
UCLASS(Transient, MinimalAPI)
class UUsdPrimTwin final : public UObject
{
	GENERATED_BODY()

public:
	UUsdPrimTwin& AddChild(const FString& InPrimPath);
	void RemoveChild(const TCHAR* InPrimPath);
	const TMap<FString, TObjectPtr<UUsdPrimTwin>>& GetChildren() const
	{
		return Children;
	}

	UUsdPrimTwin* GetParent() const
	{
		return Parent.Get();
	}

	void Clear();

	void Iterate(TFunction<void(UUsdPrimTwin&)> Func, bool bRecursive)
	{
		for (TMap<FString, TObjectPtr<UUsdPrimTwin>>::TIterator It = Children.CreateIterator(); It; ++It)
		{
			// During some busy transitions it is possible that our children were already destroyed by
			// the time we try clearing our own children
			if (UUsdPrimTwin* Child = It->Value)
			{
				Func(*Child);

				if (bRecursive)
				{
					Child->Iterate(Func, bRecursive);
				}
			}
		}
	}

	UUsdPrimTwin* Find(const FString& InPrimPath);
	UUsdPrimTwin* Find(const USceneComponent* InSceneComponent);

	USceneComponent* GetSceneComponent() const;

	// Begin UObject interface
	void Serialize(FArchive& Ar) override;
	// End UObject interface

	DECLARE_EVENT_OneParam(UUsdPrimTwin, FOnUsdPrimTwinDestroyed, const UUsdPrimTwin&);
	FOnUsdPrimTwinDestroyed OnDestroyed;

public:
	const static FName GetChildrenPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UUsdPrimTwin, Children);
	}

public:
	UPROPERTY()
	FString PrimPath;

	UPROPERTY()
	TWeakObjectPtr<class USceneComponent> SceneComponent;

private:
	// Transient as we don't want to save this to disk, but we'll implement Serialize() to
	// duplicate this over when we're being duplicated
	UPROPERTY(Transient)
	TMap<FString, TObjectPtr<UUsdPrimTwin>> Children;

	UPROPERTY(Transient)
	TWeakObjectPtr<UUsdPrimTwin> Parent;
};
