// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Objects/USDInfoCache.h"
#include "USDInfoCacheObject.h"
#include "USDLevelSequenceHelper.h"
#include "USDListener.h"
#include "USDMetadataImportOptions.h"
#include "USDPrimLinkCacheObject.h"
#include "USDSkeletalDataConversion.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdStage.h"

#include "GameFramework/Actor.h"
#include "Misc/ITransaction.h"

#include "USDStageActor.generated.h"

#define UE_API USDSTAGE_API

class FUsdInfoCache;
class ISequencer;
class ULevelSequence;
class UUsdAssetCache2;
class UUsdAssetCache3;
class UUsdPrimTwin;
class UUsdTransactor;
enum class EMovieSceneDataChangeType;
enum class EUsdCollisionType : uint8;
struct FUsdSchemaTranslationContext;
namespace UE
{
	class FUsdGeomBBoxCache;
	class FUsdPrim;
}
namespace UsdUtils
{
	class FUsdTransactorImpl;
}

UENUM()
enum class EUsdStageState : uint8
{
	Closed,
	Opened,
	OpenedAndLoaded
};

UCLASS(MinimalAPI, config = USDImporter)
class AUsdStageActor : public AActor
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", meta = (RelativeToGameDir))
	FFilePath RootLayer;

	/**
	 * What to do with the USD Stage pointed to by RootLayer, if any
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	EUsdStageState StageState;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD")
	TObjectPtr<UUsdAssetCache3> AssetCache;

	/**
	 * 'Transient' as this shouldn't be saved and should be zero'd when loading (i.e. each stage actor should always have
	 * a unique instance).
	 *
	 * 'Instanced' or else even though this is created as a DefaultSubObject the engine will still share the same object
	 * between different stage actors in some scenarios (e.g. spawnables going into PIE), which we absolutely never want.
	 *
	 * In addition, we manually serialize this, PrimLinkCache and RootUsdTwin when going into PIE via ::Serialize
	 * (as opposed to letting the engine leave us with reset property values), as we won't LoadUsdStage() when
	 * duplicating for PIE, given that the engine will already duplicate our actor/component spawns anyway
	 */
	UPROPERTY(Transient, Instanced)
	TObjectPtr<UUsdInfoCache> UsdInfoCache;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UUsdPrimLinkCache> PrimLinkCache;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the 'AssetCache' Property, that uses the new AssetCache3 asset type.")
	UPROPERTY()
	TObjectPtr<UUsdAssetCache2> UsdAssetCache_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// These properties are configs so that spawned actors read them from the CDO when spawned.
	// This allows the defaults for them to be configured on EditorPerProjectUserSettings.ini, and allows us to write
	// to that config from the USD Stage Editor, specifying our options before the editor is attached to any stage actor.

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	EUsdInitialLoadSet InitialLoadSet;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	EUsdInterpolationType InterpolationType;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	EGeometryCacheImport GeometryCacheImport;

	/**
	 * Use KindsToCollapse to determine when to collapse prim subtrees or not (defaults to enabled).
	 * Disable this if you want to prevent collapsing, or to control it manually by right-clicking on individual prims.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	bool bUsePrimKindsForCollapsing;

	/**
	 * Whether to try to combine individual assets and components of the same type on a kind-per-kind basis,
	 * like multiple Mesh prims into a single Static Mesh
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config, meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdDefaultKind"))
	int32 KindsToCollapse;

	/**
	 * If enabled, when multiple mesh prims are collapsed into a single static mesh, identical material slots are merged into one slot.
	 * Otherwise, materials slots are simply appended to the list.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	bool bMergeIdenticalMaterialSlots;

	/**
	 * If true, whenever two prims would have generated identical UAssets (like identical StaticMeshes or materials) then only one instance of
	 * that asset is generated, and the asset is shared by the components generated for both prims.
	 * If false, we will always generate a dedicated asset for each prim.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	bool bShareAssetsForIdenticalPrims;

	UE_DEPRECATED(all, "This property has been renamed to 'Share Assets for Identical Prims'")
	UPROPERTY()
	bool bReuseIdenticalAssets_DEPRECATED;

	UE_DEPRECATED(all, "This option is now controlled via the cvar 'USD.CollapseTopLevelPointInstancers'")
	UPROPERTY()
	bool bCollapseTopLevelPointInstancers_DEPRECATED;

	/* Only load prims with these specific purposes from the USD file */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config, meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdPurpose"))
	int32 PurposesToLoad;

	/** Try enabling Nanite for static meshes that are generated with at least this many triangles */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config, meta = (NoSpinbox = "true", UIMin = "0", ClampMin = "0"))
	int32 NaniteTriangleThreshold;

	/** Specifies which set of shaders to use when parsing USD materials, in addition to the universal render context. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	FName RenderContext;

	/** Specifies which material purbose to use when parsing USD material bindings, in addition to the "allPurpose" fallback. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	FName MaterialPurpose;

	// Describes what to add to the root bone animation within generated AnimSequences, if anything
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	EUsdRootMotionHandling RootMotionHandling;

	// What type of collision to use for static meshes generated from Prims that don't have physics schemas applied
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	EUsdCollisionType FallbackCollisionType;

	/**
	 * Subdivision level to use for all subdivision meshes on the opened stage. 0 means "don't subdivide".
	 * The maximum level of subdivision allowed can be configured via the 'USD.Subdiv.MaxSubdivLevel' cvar.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	int32 SubdivisionLevel;

	/* Describes if/how we should collect metadata from USD prims onto the assets and components we generate when opening stages */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "USD", config)
	FUsdMetadataImportOptions MetadataOptions;

public:
	DECLARE_EVENT_OneParam(AUsdStageActor, FOnActorLoaded, AUsdStageActor*);
	USDSTAGE_API static FOnActorLoaded OnActorLoaded;

	DECLARE_EVENT(AUsdStageActor, FOnStageActorEvent);
	FOnStageActorEvent OnPreStageChanged;
	FOnStageActorEvent OnStageChanged;
	FOnStageActorEvent OnActorDestroyed;
	FOnStageActorEvent OnStageLoaded;
	FOnStageActorEvent OnStageUnloaded;

	DECLARE_EVENT_TwoParams(AUsdStageActor, FOnPrimChanged, const FString&, bool);
	FOnPrimChanged OnPrimChanged;

	DECLARE_MULTICAST_DELEGATE(FOnUsdStageTimeChanged);
	FOnUsdStageTimeChanged OnTimeChanged;

	DECLARE_EVENT_OneParam(AUsdStageActor, FOnOpenStageEditorClicked, AUsdStageActor*);
	USDSTAGE_API static FOnOpenStageEditorClicked OnOpenStageEditorClicked;

public:
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetRootLayer(const FString& RootFilePath);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetStageState(EUsdStageState NewStageState);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "The UUsdAssetCache2 class has been deprecated. Use UUsdAssetCache3 and call SetUsdAssetCache instead")
	UFUNCTION()
	USDSTAGE_API void SetAssetCache(UUsdAssetCache2* NewCache);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetUsdAssetCache(UUsdAssetCache3* NewCache);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetInitialLoadSet(EUsdInitialLoadSet NewLoadSet);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetInterpolationType(EUsdInterpolationType NewType);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetGeometryCacheImport(EGeometryCacheImport ImportOption);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetUsePrimKindsForCollapsing(bool bUse);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetKindsToCollapse(int32 NewKindsToCollapse);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetMergeIdenticalMaterialSlots(bool bMerge);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetShareAssetsForIdenticalPrims(bool bShare);

	UE_DEPRECATED(all, "Use 'SetShareAssetsForIdenticalPrims' instead.")
	UFUNCTION()
	USDSTAGE_API void SetReuseIdenticalAssets(bool bReuse);

	UE_DEPRECATED(all, "This option is now controlled via the cvar 'USD.CollapseTopLevelPointInstancers'")
	UFUNCTION()
	USDSTAGE_API void SetCollapseTopLevelPointInstancers(bool bCollapse);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetPurposesToLoad(int32 NewPurposesToLoad);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetNaniteTriangleThreshold(int32 NewNaniteTriangleThreshold);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetRenderContext(const FName& NewRenderContext);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetMaterialPurpose(const FName& NewMaterialPurpose);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetRootMotionHandling(EUsdRootMotionHandling NewHandlingStrategy);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetFallbackCollisionType(EUsdCollisionType NewCollisionType);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetSubdivisionLevel(int32 NewLevel);

	UFUNCTION(BlueprintCallable, Category = "USD|Metadata", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetCollectMetadata(bool bNewCollectValue);

	UFUNCTION(BlueprintCallable, Category = "USD|Metadata", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetCollectFromEntireSubtrees(bool bNewCollectValue);

	UFUNCTION(BlueprintCallable, Category = "USD|Metadata", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetCollectOnComponents(bool bNewCollectValue);

	UFUNCTION(BlueprintCallable, Category = "USD|Metadata", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetBlockedPrefixFilters(const TArray<FString>& NewFilters);

	UFUNCTION(BlueprintCallable, Category = "USD|Metadata", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetInvertFilters(bool bNewInvertValue);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API float GetTime() const;

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetTime(float InTime);

	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API ULevelSequence* GetLevelSequence() const;

	/**
	 * Gets the transient component that was generated for a prim with a given prim path.
	 * Warning: The lifetime of the component is managed by the AUsdStageActor, and it may be force-destroyed at any time (e.g. when closing the
	 * stage)
	 * @param PrimPath - Full path to the source prim, e.g. "/root_prim/my_prim"
	 * @return The corresponding spawned component. It may correspond to a parent prim, if the prim at PrimPath was collapsed. Nullptr if path is
	 * invalid.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API USceneComponent* GetGeneratedComponent(const FString& PrimPath) const;

	/**
	 * Gets the transient assets that were generated for a prim with a given prim path. Likely one asset (e.g. UStaticMesh), but can be multiple
	 * (USkeletalMesh, USkeleton, etc.)
	 * @param PrimPath - Full path to the source prim, e.g. "/root_prim/my_mesh"
	 * @return The corresponding generated assets. May be empty if path is invalid or if that prim led to no generated assets.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API TArray<UObject*> GetGeneratedAssets(const FString& PrimPath) const;

	/**
	 * Gets the path to the prim that was parsed to generate the given `Object`.
	 * @param Object - UObject to query with. Can be one of the transient components generated when a stage was opened, or something like a
	 * UStaticMesh.
	 * @return The path to the source prim, e.g. "/root_prim/some_prim". May be empty in case we couldn't find the source prim.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API FString GetSourcePrimPath(const UObject* Object) const;

	// Creates a brand new, memory-only USD stage and opens it
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void NewStage();

	/**
	 * If IsolatedStageRootLayer is the identifier of one of the sublayers of the currently opened stage, this will
	 * enter isolated mode by creating a new stage with IsolatedStageRootLayer as its root, and displaying that.
	 * Provide an empty string to leave isolated mode.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API void SetIsolatedRootLayer(const FString& IsolatedStageRootLayer);

	/**
	 * Returns the root layer identifier of the currently isolated stage if we're in isolated mode, and the empty
	 * string otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD", meta = (CallInEditor = "true"))
	USDSTAGE_API FString GetIsolatedRootLayer() const;

public:
	// Opens the stage with RootLayer if its not loaded already, and returns the either the isolated stage (if any) or
	// the base stage
	USDSTAGE_API UE::FUsdStage& GetOrOpenUsdStage();

	// Returns either the isolated stage (if any) or the base stage
	USDSTAGE_API const UE::FUsdStage& GetUsdStage() const;

	// Always returns the base stage, regardless of whether we have an isolated stage or not
	USDSTAGE_API const UE::FUsdStage& GetBaseUsdStage() const;

	// Always returns the isolated stage, being an invalid stage in case we're not isolating anything
	USDSTAGE_API const UE::FUsdStage& GetIsolatedUsdStage() const;

	USDSTAGE_API void SetUsdStage(const UE::FUsdStage& NewStage);

	/**
	 * Enters isolated mode by creating a new USD Stage using the provided layer as its root.
	 * @param bLoadUsdStage - Whether to load assets, actors and components from the isolated stage
	 */
	USDSTAGE_API void IsolateLayer(const UE::FSdfLayer& Layer, bool bLoadUsdStage = true);

	// Creates new ULevelSequence assets for the root layer and sublayers.
	// If we already had a previous LevelSequence opened, we will close its Sequencer and return true
	USDSTAGE_API bool RegenerateLevelSequence();

	/**
	 * Fills in the current LevelSequences with animations for the prims of the opened stage.
	 *
	 * Can be called even if our current LevelSequence has already been populated: It will simply refresh the
	 * current LevelSequence, adding/removing tracks and bindings as needed.
	 */
	USDSTAGE_API void RepopulateLevelSequence();

	USDSTAGE_API void OpenLevelSequence();

	USDSTAGE_API TSharedPtr<UE::FUsdGeomBBoxCache> GetBBoxCache();
	USDSTAGE_API TMap<FString, TMap<FString, int32>> GetMaterialToPrimvarToUVIndex();
	USDSTAGE_API const UsdUtils::FBlendShapeMap& GetBlendShapeMap();
	USDSTAGE_API FUsdListener& GetUsdListener();
	USDSTAGE_API const FUsdListener& GetUsdListener() const;

	/** Control whether we respond to USD notices or not. Mostly used to prevent us from responding to them when we're writing data to the stage */
	USDSTAGE_API void StopListeningToUsdNotices();
	USDSTAGE_API void ResumeListeningToUsdNotices();
	USDSTAGE_API bool IsListeningToUsdNotices() const;

	/** Prevents writing back data to the USD stage whenever our LevelSequences are modified */
	USDSTAGE_API void StopMonitoringLevelSequence();
	USDSTAGE_API void ResumeMonitoringLevelSequence();
	USDSTAGE_API void BlockMonitoringLevelSequenceForThisTransaction();

	// Begin UObject interface
	USDSTAGE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	USDSTAGE_API virtual void Serialize(FArchive& Ar) override;
	USDSTAGE_API virtual void Destroyed() override;
	USDSTAGE_API virtual void PostActorCreated() override;
	USDSTAGE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	USDSTAGE_API virtual void BeginDestroy() override;
	USDSTAGE_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	USDSTAGE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	USDSTAGE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	USDSTAGE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	USDSTAGE_API virtual void PreEditUndo() override;
#endif	  // WITH_EDITOR
	// End UObject interface

	// Begin AActor interface
	USDSTAGE_API virtual void Reset() override;
	USDSTAGE_API virtual void PostRegisterAllComponents() override;
	USDSTAGE_API virtual void UnregisterAllComponents(bool bForReregister = false) override;
	USDSTAGE_API virtual void PostUnregisterAllComponents() override;
	// End AActor interface

	USDSTAGE_API AUsdStageActor();

protected:
	bool UnloadAssets(const UE::FSdfPath& PrimPath, bool bForEntireSubtree);

	/** Loads the asset for a single prim */
	bool LoadAsset(FUsdSchemaTranslationContext& TranslationContext, const UE::FUsdPrim& Prim);

	/** Loads the assets for all prims from StartPrim and its children */
	bool LoadAssets(FUsdSchemaTranslationContext& TranslationContext, const UE::FUsdPrim& StartPrim);

	void Refresh() const;
	void AnimatePrims();

	UUsdPrimTwin* GetRootPrimTwin();
	UUsdPrimTwin* GetRootPrimTwin() const;
	UUsdPrimTwin* GetOrCreatePrimTwin(const UE::FSdfPath& UsdPrimPath);

	UUsdPrimTwin* ExpandPrim(
		const UE::FUsdPrim& Prim,
		bool bResync,
		FUsdSchemaTranslationContext& TranslationContext,
		TOptional<bool> bParentHasAnimatedVisibility = {}
	);
	void UpdatePrim(const UE::FSdfPath& UsdPrimPath, bool bResync, FUsdSchemaTranslationContext& TranslationContext);

	void OpenUsdStage();
	void CloseUsdStage(bool bUnloadIfNeeded = true);

	void LoadUsdStage(bool bOpenIfNeeded = true);
	void UnloadUsdStage();

	void SetupAssetCacheIfNeeded();
	void SetupBBoxCacheIfNeeded();

	void RebuildInfoCacheFromStoredChanges();

	bool HasAuthorityOverStage() const;

	void UpdateSpawnedObjectsTransientFlag(bool bTransient);

	USDSTAGE_API void RequestDelayedTransactorReset();

#if WITH_EDITOR
	void OnBeginPIE(bool bIsSimulating);
	void OnPostPIEStarted(bool bIsSimulating);
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ObjectReplacementMap);
	void OnLevelActorDeleted(AActor* DeletedActor);
	void HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState);
#endif	  // WITH_EDITOR

	void OnPreUsdImport(FString FilePath);
	void OnPostUsdImport(FString FilePath);
	void OnUsdObjectsChanged(const UsdUtils::FObjectChangesByPath& InfoChanges, const UsdUtils::FObjectChangesByPath& ResyncChanges);
	void HandleAccumulatedNotices();
	void OnUsdPrimTwinDestroyed(const UUsdPrimTwin& UsdPrimTwin);
	void OnObjectPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);
	void HandlePropertyChangedEvent(FPropertyChangedEvent& PropertyChangedEvent);
	void OnSkelAnimationBaked(const FString& SkeletonPrimPath);

protected:
	friend class SUsdStage;	   // So that it can call RequestDelayedTransactorReset
	friend struct FUsdStageActorImpl;
	friend class FUsdLevelSequenceHelperImpl;
	friend class UsdUtils::FUsdTransactorImpl;

	UPROPERTY(
		Category = UsdStageActor,
		VisibleAnywhere,
		BlueprintReadOnly,
		meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh", AllowPrivateAccess = "true")
	)
	TObjectPtr<class USceneComponent> SceneComponent;

	/* TimeCode to evaluate the USD stage at */
	UPROPERTY(EditAnywhere, Category = "USD")
	float Time;

	UPROPERTY(VisibleAnywhere, Category = "USD", Transient)
	TObjectPtr<ULevelSequence> LevelSequence;

	/*
	 * c.f. UsdInfoCache UPROPERTY for property flag reasoning.
	 */
	UPROPERTY(Transient, Instanced)
	TObjectPtr<UUsdPrimTwin> RootUsdTwin;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UUsdTransactor> Transactor;

	UPROPERTY(Transient)
	TSet<FString> PrimsToAnimate;

	UPROPERTY(Transient)
	TMap<TObjectPtr<UObject>, FString> ObjectsToWatch;

	/**
	 * USD can emit multiple notices within the same UE transaction. We accumulate these in here, and respond
	 * to them only once, when the transaction is about to finish
	 */
	UsdUtils::FObjectChangesByPath AccumulatedInfoChanges;
	UsdUtils::FObjectChangesByPath AccumulatedResyncChanges;
	bool bLayerReloaded = false;
	TArray<UE::FSdfPath> ResyncedPrimsForThisTransaction;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Caches various information about prims that are expensive to query */
	TSharedPtr<FUsdInfoCache> InfoCache;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** USD bounding box cache for the stage. Constructed on-demand */
	TSharedPtr<UE::FUsdGeomBBoxCache> BBoxCache;

	FUsdListener UsdListener;

	FUsdLevelSequenceHelper LevelSequenceHelper;

	// The main UsdStage that is currently opened
	UE::FUsdStage UsdStage;

	// Another stage that has as root layer one of the non-root local layers of UsdStage. This is the stage we'll
	// be displaying if it is valid, otherwise we'll be displaying UsdStage directly.
	UE::FUsdStage IsolatedStage;

	/** Keep track of blend shapes so that we can map 'inbetween shapes' to their separate morph targets when animating */
	UsdUtils::FBlendShapeMap BlendShapesByPath;
	/**
	 * When parsing materials, we keep track of which primvar we mapped to which UV channel.
	 * When parsing meshes later, we use this data to place the correct primvar values in each UV channel.
	 * We keep this here as these are generated when the materials stored in AssetsCache are parsed, so it should accompany them
	 */
	TMap<FString, TMap<FString, int32>> MaterialToPrimvarToUVIndex;

	/**
	 * We use PostRegisterAllComponents and PostUnregisterAllComponents as main entry points to decide when to load/unload
	 * the USD stage. These are the three exceptions we must avoid though:
	 *  - We don't want to load/unload when duplicating into PIE as we want our duplicated actors/components to go with us;
	 *  - On the editor, the register/unregister functions are called from AActor::PostEditChangeProperty, and we obviously
	 *    don't want to load/unload the stage on every single property edit.
	 *  - We never want to load/unload actors and components on undo/redo: We always want to fetch them from the transaction buffer
	 */
	bool bIsTransitioningIntoPIE;
	bool bIsModifyingAProperty;
	bool bIsUndoRedoing;
	bool bIsAddingBinding = false;
	bool bIsPendingTransactorReset = false;

	FDelegateHandle OnRedoHandle;
	FDelegateHandle OnSequencerCreatedHandle;

	FThreadSafeCounter IsBlockedFromUsdNotices;

	/**
	 * Helps us know whether a transaction changed our RootLayer or not. We need this because we can only tag spawned
	 * transient actors and components after the initial actor/component spawning transaction has completed. Otherwise,
	 * the spawns will be replicated on each client in addition to the actors/components that they will spawn by themselves
	 * for opening the stage
	 */
	FFilePath OldRootLayer;
};

class FScopedBlockNoticeListening final
{
public:
	UE_API explicit FScopedBlockNoticeListening(AUsdStageActor* InStageActor);
	UE_API ~FScopedBlockNoticeListening();

	FScopedBlockNoticeListening() = delete;
	FScopedBlockNoticeListening(const FScopedBlockNoticeListening&) = delete;
	FScopedBlockNoticeListening(FScopedBlockNoticeListening&&) = delete;
	FScopedBlockNoticeListening& operator=(const FScopedBlockNoticeListening&) = delete;
	FScopedBlockNoticeListening& operator=(FScopedBlockNoticeListening&&) = delete;

private:
	TWeakObjectPtr<AUsdStageActor> StageActor;
};

#undef UE_API
