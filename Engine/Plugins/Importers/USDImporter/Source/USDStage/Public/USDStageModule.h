// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

class AUsdStageActor;
class UWorld;
class ISequencer;

class IUsdStageModule : public IModuleInterface
{
public:
	virtual AUsdStageActor& GetUsdStageActor(UWorld* World) = 0;
	virtual AUsdStageActor* FindUsdStageActor(UWorld* World) = 0;

#if WITH_EDITOR
	virtual const TArray<TWeakPtr<ISequencer>>& GetExistingSequencers() const = 0;
#endif	  // WITH_EDITOR
};
