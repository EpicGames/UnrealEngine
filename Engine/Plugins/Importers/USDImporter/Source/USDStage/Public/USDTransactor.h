// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "USDSetUtils.h"

#include "USDTransactor.generated.h"

#define UE_API USDSTAGE_API

class AUsdStageActor;

namespace UsdUtils
{
	class FUsdTransactorImpl;

	struct FSdfChangeListEntry;
	using FObjectChangesByPath = TCaseSensitiveStringMap<TArray<FSdfChangeListEntry>>;
}

namespace UE
{
	namespace UsdTransactor
	{
		/** Actors and components cleared to be transacted by ConcertSync even if they are transient will receive this tag */
		extern USDSTAGE_API const FName ConcertSyncEnableTag;
	}
}

/**
 * Class that allows us to log prim attribute changes into the unreal transaction buffer.
 * The AUsdStageActor owns one of these, and whenever a USD notice is fired this class transacts and serializes
 * the notice data with itself. When undo/redoing it applies its values to the AUsdStageActors' current stage.
 *
 * Additionally this class naturally allows multi-user (ConcertSync) support for USD stage interactions, by letting
 * these notice data to be mirrored on other clients.
 */
UCLASS(MinimalAPI)
class UUsdTransactor : public UObject
{
	GENERATED_BODY()

public:
	// Boilerplate for Pimpl usage with UObject
	UE_API UUsdTransactor();
	UE_API UUsdTransactor(FVTableHelper& Helper);
	UE_API ~UUsdTransactor();

	UE_API void Initialize(AUsdStageActor* InStageActor);
	UE_API void Update(const UsdUtils::FObjectChangesByPath& NewInfoChanges, const UsdUtils::FObjectChangesByPath& NewResyncChanges);

	// Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void PreEditUndo();
	UE_API virtual void PostEditUndo();
#endif	  // WITH_EDITOR
		  //~ End UObject interface

private:
	TWeakObjectPtr<AUsdStageActor> StageActor;

	TUniquePtr<UsdUtils::FUsdTransactorImpl> Impl;
};

#undef UE_API
