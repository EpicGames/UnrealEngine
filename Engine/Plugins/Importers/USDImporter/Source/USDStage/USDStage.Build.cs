// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDStage : ModuleRules
	{
		public USDStage(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"USDSchemas" // Has to be a public dependency because the stage actor has an FUsdInfoCache member
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AnimationCore",
					"CinematicCamera",
					"ControlRig",
					"Core",
					"CoreUObject",
					"Engine",
					"GeometryCache",
					"GeometryCacheTracks",
					"GeometryCacheUSD",
					"HairStrandsCore",
					"LevelSequence",
					"LiveLinkComponents", // For tracking edits to LiveLinkComponentController properties and writing to USD
					"LiveLinkInterface",
					"MeshDescription",
					"MovieScene",
					"MovieSceneTracks",
					"Niagara",	// Needed by GroomComponent.h
					"RigVM",
					"Slate",
					"SlateCore",
					"StaticMeshDescription",
					"UniversalObjectLocator",
					"UnrealUSDWrapper",
					"USDClasses",
					"USDUtilities",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"BlueprintGraph", // For setting up the Sequencer Dynamic Binding blueprint graphs
						"ControlRigDeveloper",
						"DeveloperToolSettings",
						"EditorStyle", // For the font style on the stage actor customization
						"InputCore", // For keyboard control on the widget in the stage actor customization
						"Kismet", // For setting up the Sequencer Dynamic Binding blueprint graphs
						"LevelSequenceEditor",
						"MovieSceneTools",
						"PropertyEditor", // For the stage actor's details customization
						"RigVMDeveloper",
						"SceneOutliner", // For drag and drop operations on the Universal Object Locator editor
						"Sequencer",
						"UniversalObjectLocatorEditor",
						"UnrealEd",
						"USDClassesEditor",
					}
				);
			}

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
