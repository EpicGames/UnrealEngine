// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDLayersTreeView.h"

#include "Framework/Application/SlateApplication.h"
#include "SUSDStageEditorStyle.h"
#include "USDClassesModule.h"
#include "USDDragDropOp.h"
#include "USDErrorUtils.h"
#include "USDLayersViewModel.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDProjectSettings.h"
#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"

#include "DesktopPlatformModule.h"
#include "EngineAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IDesktopPlatform.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SToolTip.h"

#if USE_USD_SDK

#define LOCTEXT_NAMESPACE "SUSDLayersTreeView"

namespace UE::USDLayersTreeViewImpl::Private
{
	void ExportLayerToPath(const UE::FSdfLayer& LayerToExport, const FString& TargetPath)
	{
		if (!LayerToExport)
		{
			return;
		}

		// Clone the layer so that we don't modify the currently opened stage when we do the remapping below
		UE::FSdfLayer OutputLayer = UE::FSdfLayer::CreateNew(*TargetPath);
		if (!OutputLayer)
		{
			USD_LOG_WARNING(TEXT("Failed to export USD layer to path '%s'"), *TargetPath);
			return;
		}

		OutputLayer.TransferContent(LayerToExport);

		// Update references to assets (e.g. textures) so that they're absolute and also work from the new file
		UsdUtils::ConvertAssetRelativePathsToAbsolute(OutputLayer, LayerToExport);

		// Convert layer references to absolute paths so that it still works at its target location
		FString LayerPath = LayerToExport.GetRealPath();
		FPaths::NormalizeFilename(LayerPath);
		const TSet<FString> AssetDependencies = OutputLayer.GetCompositionAssetDependencies();
		for (const FString& Ref : AssetDependencies)
		{
			FString AbsRef = FPaths::ConvertRelativePathToFull(FPaths::GetPath(LayerPath), Ref);	// Relative to the original file
			OutputLayer.UpdateCompositionAssetDependency(*Ref, *AbsRef);
		}

		bool bForce = true;
		OutputLayer.Save(bForce);
	}

	void ToggleMuteOrShowWarning(FUsdLayerViewModelRef LayerItem)
	{
		if (!LayerItem->IsValid() || !LayerItem->CanMuteLayer())
		{
			return;
		}

		// If the layer is not dirty we can just mute it without worry and early out
		if (!LayerItem->IsLayerDirty())
		{
			LayerItem->ToggleMuteLayer();
			return;
		}

		// Show a warning if the layer is dirty, as muting it will discard changes
		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (!LayerItem->IsLayerMuted() && Settings && Settings->bShowConfirmationWhenMutingDirtyLayers)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(LOCTEXT("ConfirmMutingLayer", "Muting dirty layer"));
			Toast.SubText = FText::Format(
				LOCTEXT(
					"ConfirmMutingLayer_Subtext",
					"Layer '{0}' has unsaved changes that will be lost if muted.\n\nDo you wish to proceed with muting the layer?"
				),
				LayerItem->GetDisplayName()
			);
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;

			Toast.ButtonDetails.Emplace(
				LOCTEXT("ConfirmMutingOkAll", "Always proceed"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[LayerItem]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();

							UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>();
							Settings->bShowConfirmationWhenMutingDirtyLayers = false;
							Settings->SaveConfig();

							LayerItem->ToggleMuteLayer();
						}
					}
				)
			);

			Toast.ButtonDetails.Emplace(
				LOCTEXT("ConfirmMutingOk", "Proceed"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[LayerItem]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();

							LayerItem->ToggleMuteLayer();
						}
					}
				)
			);

			Toast.ButtonDetails.Emplace(
				LOCTEXT("ConfirmMutingCancel", "Cancel"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Fail);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
		else
		{
			// Don't show prompt, always just mute
			LayerItem->ToggleMuteLayer();
		}
	}

	void ReloadOrShowWarning(FUsdLayerViewModelRef LayerItem)
	{
		if (!LayerItem->IsValid() || !LayerItem->CanReload())
		{
			return;
		}

		// If the layer is not dirty we can just mute it without worry and early out
		if (!LayerItem->IsLayerDirty())
		{
			LayerItem->Reload();
			return;
		}

		// Show a warning if the layer is dirty, as muting it will discard changes
		const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
		if (!LayerItem->IsLayerMuted() && Settings && Settings->bShowConfirmationWhenReloadingDirtyLayers)
		{
			static TWeakPtr<SNotificationItem> Notification;

			FNotificationInfo Toast(LOCTEXT("ConfirmReloadingDirtyLayer", "Reloading dirty layer"));
			Toast.SubText = FText::Format(
				LOCTEXT(
					"ConfirmReloadingDirtyLayer_Subtext",
					"Layer '{0}' has unsaved changes that will be lost if reloaded.\n\nDo you wish to proceed with reloading the layer?"
				),
				LayerItem->GetDisplayName()
			);
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = false;
			Toast.FadeOutDuration = 0.0f;
			Toast.ExpireDuration = 0.0f;
			Toast.bUseThrobber = false;
			Toast.bUseSuccessFailIcons = false;

			Toast.ButtonDetails.Emplace(
				LOCTEXT("ConfirmMutingOkAll", "Always proceed"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[LayerItem]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();

							UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>();
							Settings->bShowConfirmationWhenReloadingDirtyLayers = false;
							Settings->SaveConfig();

							LayerItem->Reload();
						}
					}
				)
			);

			Toast.ButtonDetails.Emplace(
				LOCTEXT("ConfirmMutingOk", "Proceed"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[LayerItem]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
							PinnedNotification->ExpireAndFadeout();

							LayerItem->Reload();
						}
					}
				)
			);

			Toast.ButtonDetails.Emplace(
				LOCTEXT("ConfirmMutingCancel", "Cancel"),
				FText::GetEmpty(),
				FSimpleDelegate::CreateLambda(
					[]()
					{
						if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
						{
							PinnedNotification->SetCompletionState(SNotificationItem::CS_Fail);
							PinnedNotification->ExpireAndFadeout();
						}
					}
				)
			);

			// Only show one at a time
			if (!Notification.IsValid())
			{
				Notification = FSlateNotificationManager::Get().AddNotification(Toast);
			}

			if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
			{
				PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
			}
		}
		else
		{
			// Don't show prompt, always just reload
			LayerItem->Reload();
		}
	}
}	 // namespace UE::USDLayersTreeViewImpl::Private

class FUsdLayerNameColumn
	: public FUsdTreeViewColumn
	, public TSharedFromThis<FUsdLayerNameColumn>
{
public:
	FSlateColor GetForegroundColor(const FUsdLayerViewModelRef TreeItem) const
	{
		return TreeItem->IsInIsolatedStage() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	}

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		FUsdLayerViewModelRef TreeItem = StaticCastSharedRef<FUsdLayerViewModel>(InTreeItem.ToSharedRef());
		TWeakPtr<FUsdLayerViewModel> TreeItemWeak = TreeItem;

		// clang-format off
		return SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(TreeItem, &FUsdLayerViewModel::GetDisplayName)
				.ColorAndOpacity(this, &FUsdLayerNameColumn::GetForegroundColor, TreeItem)
				.ToolTipText_Lambda([TreeItemWeak]
				{
					if (TSharedPtr<FUsdLayerViewModel> PinnedTreeItem = TreeItemWeak.Pin())
					{
						return FText::FromString(PinnedTreeItem->LayerIdentifier);
					}

					return FText::GetEmpty();
				})
			];
		// clang-format on
	}
};

class FUsdLayerMutedColumn
	: public FUsdTreeViewColumn
	, public TSharedFromThis<FUsdLayerMutedColumn>
{
public:
	FReply OnClicked(const FUsdLayerViewModelRef TreeItem)
	{
		UE::USDLayersTreeViewImpl::Private::ToggleMuteOrShowWarning(TreeItem);

		return FReply::Handled();
	}

	const FSlateBrush* GetBrush(const FUsdLayerViewModelRef TreeItem, const TSharedPtr<SButton> Button) const
	{
		const bool bIsButtonHovered = Button.IsValid() && Button->IsHovered();

		if (!TreeItem->CanMuteLayer())
		{
			return nullptr;
		}
		else if (TreeItem->IsLayerMuted())
		{
			return bIsButtonHovered ? FAppStyle::GetBrush("Level.NotVisibleHighlightIcon16x") : FAppStyle::GetBrush("Level.NotVisibleIcon16x");
		}
		else
		{
			return bIsButtonHovered ? FAppStyle::GetBrush("Level.VisibleHighlightIcon16x") : FAppStyle::GetBrush("Level.VisibleIcon16x");
		}
	}

	FSlateColor GetForegroundColor(const FUsdLayerViewModelRef TreeItem, const TSharedPtr<ITableRow> TableRow, const TSharedPtr<SButton> Button) const
	{
		if (!TableRow.IsValid() || !Button.IsValid())
		{
			return FSlateColor::UseForeground();
		}

		const bool bIsRowHovered = TableRow->AsWidget()->IsHovered();
		const bool bIsButtonHovered = Button->IsHovered();
		const bool bIsRowSelected = TableRow->IsItemSelected();
		const bool bIsLayerMuted = TreeItem->IsLayerMuted();

		if (!bIsLayerMuted && !bIsRowHovered && !bIsRowSelected)
		{
			return FLinearColor::Transparent;
		}
		else if (bIsButtonHovered && !bIsRowSelected)
		{
			return FAppStyle::GetSlateColor(TEXT("Colors.ForegroundHover"));
		}

		return FSlateColor::UseForeground();
	}

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		if (!InTreeItem)
		{
			return SNullWidget::NullWidget;
		}

		FUsdLayerViewModelRef TreeItem = StaticCastSharedRef<FUsdLayerViewModel>(InTreeItem.ToSharedRef());
		FUsdLayerViewModelWeak TreeItemWeak = TreeItem;
		const float ItemSize = FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight");

		// clang-format off
		if (!TreeItem->CanMuteLayer())
		{
			return SNew(SBox)
				.HeightOverride(ItemSize)
				.WidthOverride(ItemSize)
				.Visibility(EVisibility::Visible)
				.ToolTip(SNew(SToolTip).Text(LOCTEXT("CantMuteLayerTooltip", "Cannot mute root layers or edit targets")));
		}

		TSharedPtr<SButton> Button = SNew(SButton)
			.ContentPadding(0)
			.IsEnabled_Lambda([TreeItemWeak]()->bool
			{
				if (TSharedPtr<FUsdLayerViewModel> PinnedTreeItem = TreeItemWeak.Pin())
				{
					return PinnedTreeItem->IsInIsolatedStage();
				}
				return false;
			})
			.ButtonStyle(FUsdStageEditorStyle::Get(), TEXT("NoBorder"))
			.OnClicked(this, &FUsdLayerMutedColumn::OnClicked, TreeItem)
			.ToolTip(SNew(SToolTip).Text(LOCTEXT("MuteLayerTooltip", "Mute or unmute this layer")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center);

		TSharedPtr<SImage> Image = SNew(SImage)
			.Image(this, &FUsdLayerMutedColumn::GetBrush, TreeItem, Button)
			.ColorAndOpacity(this, &FUsdLayerMutedColumn::GetForegroundColor, TreeItem, TableRow, Button);

		Button->SetContent(Image.ToSharedRef());

		return SNew(SBox)
			.HeightOverride(ItemSize)
			.WidthOverride(ItemSize)
			.Visibility(EVisibility::Visible)
			[
				Button.ToSharedRef()
			];
		// clang-format on
	}
};

class FUsdLayerEditColumn
	: public FUsdTreeViewColumn
	, public TSharedFromThis<FUsdLayerEditColumn>
{
public:
	FReply OnClicked(const FUsdLayerViewModelRef TreeItem)
	{
		return TreeItem->EditLayer() ? FReply::Handled() : FReply::Unhandled();
	}

	FSlateColor GetForegroundColor(const FUsdLayerViewModelRef TreeItem, const TSharedPtr<ITableRow> TableRow, const TSharedPtr<SButton> Button) const
	{
		if (!TableRow.IsValid() || !Button.IsValid())
		{
			return FSlateColor::UseForeground();
		}

		const bool bIsButtonHovered = Button->IsHovered();
		const bool bIsLayerEditTarget = TreeItem->IsEditTarget();

		if (bIsLayerEditTarget)
		{
			return FSlateColor::UseForeground();
		}
		else
		{
			if (!bIsButtonHovered)
			{
				return FLinearColor::Transparent;
			}
			else
			{
				return FAppStyle::GetSlateColor(TEXT("Colors.ForegroundHover"));
			}
		}
	}

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		const FUsdLayerViewModelRef TreeItem = StaticCastSharedRef<FUsdLayerViewModel>(InTreeItem.ToSharedRef());
		FUsdLayerViewModelWeak TreeItemWeak = TreeItem;

		float ItemSize = FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight");

		// clang-format off
		TSharedPtr<SButton> Button = SNew(SButton)
			.ContentPadding(0)
			.IsEnabled_Lambda([TreeItemWeak]()->bool
			{
				if (TSharedPtr<FUsdLayerViewModel> PinnedTreeItem = TreeItemWeak.Pin())
				{
					return PinnedTreeItem->IsInIsolatedStage();
				}
				return false;
			})
			.ButtonStyle(FUsdStageEditorStyle::Get(), TEXT("NoBorder"))
			.OnClicked(this, &FUsdLayerEditColumn::OnClicked, TreeItem)
			.ToolTip(SNew(SToolTip).Text(LOCTEXT("EditLayerButtonToolTip", "Edit layer")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center);

		TSharedPtr<SImage> Image = SNew(SImage)
			.Image(FUsdStageEditorStyle::Get()->GetBrush("UsdStageEditor.CheckBoxImage"))
			.ColorAndOpacity(this, &FUsdLayerEditColumn::GetForegroundColor, TreeItem, TableRow, Button);

		Button->SetContent(Image.ToSharedRef());

		return SNew(SBox)
			.HeightOverride(ItemSize)
			.WidthOverride(ItemSize)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				Button.ToSharedRef()
			];
		// clang-format on
	}
};

void SUsdLayersTreeView::Construct(const FArguments& InArgs)
{
	SUsdTreeView::Construct(SUsdTreeView::FArguments());

	LayerIsolatedDelegate = InArgs._OnLayerIsolated;

	OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SUsdLayersTreeView::ConstructLayerContextMenu);

	OnExpansionChanged = FOnExpansionChanged::CreateLambda(
		[this](const FUsdLayerViewModelRef& LayerViewModel, bool bIsExpanded)
		{
			if (!LayerViewModel->IsValid())
			{
				return;
			}

			TreeItemExpansionStates.Add(LayerViewModel->LayerIdentifier, bIsExpanded);
		}
	);
}

void SUsdLayersTreeView::Refresh(const UE::FUsdStageWeak& NewStage, const UE::FUsdStageWeak& InIsolatedStage, bool bResync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SUsdLayersTreeView::Refresh);

	if (bResync)
	{
		bool bShouldResetExpansionStates = false;
		if (RootItems.Num() > 0)
		{
			const FUsdLayerViewModelRef& FirstRootItem = RootItems[0];
			const UE::FUsdStageWeak& OldStage = FirstRootItem->UsdStage;
			bShouldResetExpansionStates = NewStage != OldStage;
		}

		if (bShouldResetExpansionStates)
		{
			TreeItemExpansionStates.Reset();
		}

		UsdStage = NewStage;
		IsolatedStage = InIsolatedStage;

		BuildUsdLayersEntries();

		RestoreExpansionStates();
	}
	else
	{
		for (FUsdLayerViewModelRef TreeItem : RootItems)
		{
			TreeItem->RefreshData();
		}
	}

	RequestTreeRefresh();
}

FReply SUsdLayersTreeView::OnRowDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	TArray<FUsdLayerViewModelRef> Items = GetSelectedItems();

	TSet<TSharedRef<IUsdTreeViewItem>> DraggedItems;
	DraggedItems.Reserve(Items.Num());

	for (const FUsdLayerViewModelRef& Item : Items)
	{
		// Don't let a drag operation begin if we're dragging the root or session layer
		if (Item->ParentItem == nullptr)
		{
			return FReply::Unhandled();
		}

		DraggedItems.Add(Item);
	}

	const FSlateBrush* Icon = FAppStyle::GetBrush(TEXT("Layer.Icon16x"));
	FText DefaultHoverText = DraggedItems.Num() == 1
								 ? FText::Format(
									   LOCTEXT("DefaultTextSingle", "USD layer '{0}'"),
									   FText::FromString(StaticCastSharedRef<FUsdLayerViewModel>(*DraggedItems.CreateConstIterator())->LayerIdentifier
									   )
								   )
								 : FText::Format(LOCTEXT("DefaultTextMultiple", "{0} USD layers"), DraggedItems.Num());

	TSharedRef<FUsdDragDropOp> Op = MakeShared<FUsdDragDropOp>();
	Op->OpType = EUsdDragDropOpType::Layers;
	Op->DraggedItems = DraggedItems;
	Op->SetToolTip(DefaultHoverText, Icon);
	Op->SetupDefaults();
	Op->Construct();	// Required to initialize the little window that shows tooltips

	return FReply::Handled().BeginDragDrop(Op);
}

void SUsdLayersTreeView::OnRowDragLeave(const FDragDropEvent& Event)
{
	if (TSharedPtr<FUsdDragDropOp> Op = Event.GetOperationAs<FUsdDragDropOp>())
	{
		Op->ResetToDefaultToolTip();
	}
}

namespace UE::USDLayersTreeViewImpl::Private
{
	// Checks if we can add all of our dragged layers as sublayers to Parent
	UsdUtils::ECanInsertSublayerResult CanAddAsSubLayers(const UE::FSdfLayer& Parent, const TSet<TSharedRef<IUsdTreeViewItem>>& SubLayerItems)
	{
		for (const TSharedRef<IUsdTreeViewItem>& Item : SubLayerItems)
		{
			const FUsdLayerViewModelRef& SubLayerItem = StaticCastSharedRef<FUsdLayerViewModel>(Item);

			UsdUtils::ECanInsertSublayerResult Result = UsdUtils::CanInsertSubLayer(Parent, *SubLayerItem->LayerIdentifier);

			if (Result != UsdUtils::ECanInsertSublayerResult::Success)
			{
				return Result;
			}
		}

		return UsdUtils::ECanInsertSublayerResult::Success;
	}

	// Checks if this operation would effectively do nothing
	bool IsNoOp(
		const TArray<FUsdLayerViewModelRef>& ExistingItems,
		const TSet<TSharedRef<IUsdTreeViewItem>>& DraggedItems,
		int32 TargetItemIndex,
		EItemDropZone DropZone
	)
	{
		switch (DropZone)
		{
			case EItemDropZone::AboveItem:
			{
				ensure(TargetItemIndex != INDEX_NONE);

				// If dropped, we'll add the layers before the target index within ExistingItems, so here we just have
				// to check if the previous items of ExistingItems (before the target) are the ones we're dragging

				// We have dragged more items than there are items after the target, so there is
				// no way this can be a no-op
				if (DraggedItems.Num() > TargetItemIndex)
				{
					return false;
				}

				// See if all the items immediately before the target are the same as the ones we're
				// dragging
				TSet<FUsdLayerViewModelRef> UniqueItemsBeforeTarget;
				for (int32 Index = TargetItemIndex - 1; Index >= 0; --Index)
				{
					const FUsdLayerViewModelRef& OtherItem = ExistingItems[Index];

					if (DraggedItems.Contains(OtherItem))
					{
						UniqueItemsBeforeTarget.Add(OtherItem);
					}
					else
					{
						// We found something else other than the dragged items
						break;
					}
				}

				// This means there is another item before the target that is *not* something we dragged:
				// That means we're dragging something that is not immediately after the target, so
				// this operation would actually do something
				if (UniqueItemsBeforeTarget.Num() < DraggedItems.Num())
				{
					return false;
				}
				break;
			}
			case EItemDropZone::BelowItem:
			{
				ensure(TargetItemIndex != INDEX_NONE);

				// If dropped, we'll add the layers after the target index within ExistingItems, so here we just have
				// to check if the next items of ExistingItems (after the target) are the ones we're dragging

				// We have dragged more items than there are items after the target, so there is
				// no way this can be a no-op
				if (DraggedItems.Num() > ExistingItems.Num() - TargetItemIndex - 1)
				{
					return false;
				}

				// See if all the items immediately after the target are the same as the ones we're
				// dragging
				TSet<FUsdLayerViewModelRef> UniqueItemsAfterTarget;
				for (int32 Index = TargetItemIndex + 1; Index < ExistingItems.Num(); ++Index)
				{
					const FUsdLayerViewModelRef& OtherItem = ExistingItems[Index];

					if (DraggedItems.Contains(OtherItem))
					{
						UniqueItemsAfterTarget.Add(OtherItem);
					}
					else
					{
						// We found something else other than the dragged items
						break;
					}
				}

				// This means there is another item after the target that is *not* something we dragged:
				// That means we're dragging something that is not immediately after the target, so
				// this operation would actually do something
				if (UniqueItemsAfterTarget.Num() < DraggedItems.Num())
				{
					return false;
				}
				break;
			}
			case EItemDropZone::OntoItem:
			{
				// If dropped, we add the layers at then end of the sublayer list, so here we just
				// have to check if the last items of the sublayer list are the ones we're dragging

				if (DraggedItems.Num() > ExistingItems.Num())
				{
					return false;
				}

				// See if the last ExistingItems are the same as the ones we're dragging
				TSet<FUsdLayerViewModelRef> UniqueExistingItemsTail;
				for (int32 Index = ExistingItems.Num() - 1; Index >= 0; --Index)
				{
					const FUsdLayerViewModelRef& OtherItem = ExistingItems[Index];

					if (DraggedItems.Contains(OtherItem))
					{
						UniqueExistingItemsTail.Add(OtherItem);
					}
					else
					{
						// We found something else other than the dragged items
						break;
					}
				}

				// This means there is another item at the end of ExistingItems that is *not* something we dragged:
				// That means we're dragging something that is not already at the end of ExistingITems, and so that
				// this operation would actually do something
				if (UniqueExistingItemsTail.Num() < DraggedItems.Num())
				{
					return false;
				}

				break;
			}
			default:
			{
				return false;
			}
		}

		return true;
	}
}	 // namespace UE::USDLayersTreeViewImpl::Private

TOptional<EItemDropZone> SUsdLayersTreeView::OnRowCanAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, FUsdLayerViewModelRef Item)
{
	using namespace UE::USDLayersTreeViewImpl::Private;

	TOptional<EItemDropZone> Result;
	if (TSharedPtr<FUsdDragDropOp> Op = Event.GetOperationAs<FUsdDragDropOp>())
	{
		FText NewHoverText;
		const FSlateBrush* NewHoverIcon = nullptr;
		const int32 NumItems = Op->DraggedItems.Num();
		bool bResetToDefaultAndBlockOp = false;

		if (Op->OpType == EUsdDragDropOpType::Layers && NumItems > 0)
		{
			// Don't accept dropping onto one of the currently dragged layers
			for (const TSharedRef<IUsdTreeViewItem>& DraggedItem : Op->DraggedItems)
			{
				if (StaticCastSharedRef<FUsdLayerViewModel>(DraggedItem) == Item)
				{
					bResetToDefaultAndBlockOp = true;
					break;
				}
			}

			if (!bResetToDefaultAndBlockOp)
			{
				// Special case when hovering the root/session layer. We can only add as sublayers here, as we don't
				// support swapping root/session layers or having multiple of them
				// (USD doesn't either... that would be more like reopening a stage?)
				if (Item->ParentItem == nullptr)
				{
					Zone = EItemDropZone::OntoItem;
				}

				FUsdLayerViewModel* ParentItem = nullptr;
				FText MessageEnd;
				switch (Zone)
				{
					case EItemDropZone::AboveItem:
					{
						ParentItem = Item->ParentItem;
						MessageEnd = FText::Format(
							LOCTEXT("DropAboveMessageEnd", ", before '{0}'"),
							FText::FromString(FPaths::GetCleanFilename(Item->LayerIdentifier))
						);
						break;
					}
					case EItemDropZone::BelowItem:
					{
						ParentItem = Item->ParentItem;
						MessageEnd = FText::Format(
							LOCTEXT("DropBelowMessageEnd", ", after '{0}'"),
							FText::FromString(FPaths::GetCleanFilename(Item->LayerIdentifier))
						);
						break;
					}
					default:
					case EItemDropZone::OntoItem:
					{
						ParentItem = &Item.Get();
						MessageEnd = FText::GetEmpty();
						break;
					}
				}

				const UsdUtils::ECanInsertSublayerResult CanAddResult = CanAddAsSubLayers(ParentItem->GetLayer(), Op->DraggedItems);

				// Find the index of the hovered item within its parent, so we know where to insert the dragged layers
				// into. We don't need this for the EItemDropZone::OntoItem case as we always append them at the end
				int32 ItemIndexInParent = INDEX_NONE;
				if (CanAddResult == UsdUtils::ECanInsertSublayerResult::Success && Zone != EItemDropZone::OntoItem)
				{
					for (int32 Index = 0; Index < ParentItem->Children.Num(); ++Index)
					{
						const FUsdLayerViewModelRef& SiblingItem = ParentItem->Children[Index];
						if (SiblingItem == Item)
						{
							ItemIndexInParent = Index;
							break;
						}
					}
				}

				// We can't allow duplicate sublayers on the same list, so here we need to check if
				// ParentItem->Children has any item with an identifier that matches one in dragged items list *without
				// being one of the dragged items itself*.
				// We have to check for this particular error here and not within CanAddAsSubLayers because
				// we need the actual tree view items.
				bool bHasDuplicateLayer = false;
				if (CanAddResult == UsdUtils::ECanInsertSublayerResult::Success)
				{
					TSet<FString> DraggedItemIdentifiers;
					DraggedItemIdentifiers.Reserve(Op->DraggedItems.Num());
					for (const TSharedRef<IUsdTreeViewItem>& DraggedItem : Op->DraggedItems)
					{
						const FUsdLayerViewModelRef& DraggedLayerItem = StaticCastSharedRef<FUsdLayerViewModel>(DraggedItem);

						DraggedItemIdentifiers.Add(DraggedLayerItem->LayerIdentifier);
					}

					for (const FUsdLayerViewModelRef& SiblingItem : ParentItem->Children)
					{
						if (!Op->DraggedItems.Contains(StaticCastSharedRef<IUsdTreeViewItem>(SiblingItem))
							&& DraggedItemIdentifiers.Contains(SiblingItem->LayerIdentifier))
						{
							bHasDuplicateLayer = true;
							break;
						}
					}
				}

				const FText ErrorMessage = bHasDuplicateLayer ? LOCTEXT("CanDropDuplicate_Text", "Cannot add duplicate SubLayer!")
															  : UsdUtils::ToText(CanAddResult);

				const bool bIsNoOp = !bHasDuplicateLayer && CanAddResult == UsdUtils::ECanInsertSublayerResult::Success
									 && IsNoOp(ParentItem->Children, Op->DraggedItems, ItemIndexInParent, Zone);

				// Drag and drop would do nothing
				if (bIsNoOp)
				{
					bResetToDefaultAndBlockOp = true;
				}
				// Drag and drop cannot be performed for some reason
				else if (bHasDuplicateLayer || CanAddResult != UsdUtils::ECanInsertSublayerResult::Success)
				{
					NewHoverText = FText::Format(
						LOCTEXT(
							"CanDropError_Text",
							"Cannot add dragged {0}|plural(one=layer,other=layers) as {0}|plural(one=sublayer,other=sublayers) of '{1}': {2}"
						),
						NumItems,
						FText::FromString(FPaths::GetCleanFilename(ParentItem->LayerIdentifier)),
						ErrorMessage
					);
				}
				// Can drop the one dragged layer
				else if (NumItems == 1)
				{
					FString FirstDraggedLayer = FPaths::GetCleanFilename(
						StaticCastSharedRef<FUsdLayerViewModel>(*Op->DraggedItems.CreateConstIterator())->LayerIdentifier
					);

					NewHoverText = FText::Format(
						LOCTEXT("CanDropSingle_Text", "Add layer '{0}' as a sublayer of '{1}'{2}"),
						FText::FromString(FirstDraggedLayer),
						FText::FromString(FPaths::GetCleanFilename(ParentItem->LayerIdentifier)),
						MessageEnd
					);

					Result = {Zone};
				}
				// Can drop multiple dragged layers
				else
				{
					NewHoverText = FText::Format(
						LOCTEXT("CanDropMultiple_Text", "Add {0} layers as sublayers of '{1}'{2}"),
						NumItems,
						FText::FromString(FPaths::GetCleanFilename(ParentItem->LayerIdentifier)),
						MessageEnd
					);

					Result = {Zone};
				}
			}
		}
		else
		{
			NewHoverText = LOCTEXT("CanDropUnsupported_Text", "Can only drag and drop layers for now");
		}

		if (bResetToDefaultAndBlockOp)
		{
			Op->ResetToDefaultToolTip();
			Result.Reset();
		}
		else
		{
			NewHoverIcon = FAppStyle::GetBrush(Result.IsSet() ? TEXT("Graph.ConnectorFeedback.Ok") : TEXT("Graph.ConnectorFeedback.Error"));

			Op->SetToolTip(NewHoverText, NewHoverIcon);
		}
	}

	return Result;
}

FReply SUsdLayersTreeView::OnRowAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, FUsdLayerViewModelRef Item)
{
	// This doesn't work very well USD-wise yet, and I don't know how we can possibly undo/redo layer reassignments
	// like this. We do need a transaction though, as this may trigger the creation or deletion of UObjects
	FScopedTransaction Transaction(LOCTEXT("OnAcceptDropTransaction", "Drop dragged USD layers"));

	if (TSharedPtr<FUsdDragDropOp> Op = Event.GetOperationAs<FUsdDragDropOp>())
	{
		// Only accept layers for now
		const int32 NumItems = Op->DraggedItems.Num();
		if (Op->OpType == EUsdDragDropOpType::Layers && NumItems > 0)
		{
			UE::FSdfChangeBlock ChangeBlock;

			// Make sure our layers aren't closed by USD between us removing them and readding
			TArray<UE::FSdfLayer> LayerPins;
			LayerPins.Reserve(Op->DraggedItems.Num());

			// Remove the dragged items from their parents before we even fetch where to insert them into as this will
			// affect the target indices
			for (const TSharedRef<IUsdTreeViewItem>& DraggedItem : Op->DraggedItems)
			{
				const FUsdLayerViewModelRef& LayerItem = StaticCastSharedRef<FUsdLayerViewModel>(DraggedItem);
				UE::FSdfLayer LayerPin = LayerItem->GetLayer();
				LayerPins.Add(LayerPin);

				// We are only allowed to drag off things that have parents: There would never be anywhere to add the
				// root/session layer into as that would create loops/strange situations
				FUsdLayerViewModel* OldParent = LayerItem->ParentItem;
				if (ensure(OldParent))
				{
					if (UE::FSdfLayer OldParentLayer = OldParent->GetLayer())
					{
						int32 OldIndex = OldParent->Children.IndexOfByKey(LayerItem->AsShared());
						ensure(OldIndex != INDEX_NONE);

						OldParentLayer.RemoveSubLayerPath(OldIndex);

						// If we have dragged multiple items, the upcoming insertion of this item may affect all the
						// other removals indices too, so we need to make sure our widgets are up-to-date. They won't
						// be updated on the next iteration of this loop even if we remove the outer FSdfChangeBlock,
						// as the resync would end up creating a new batch of widgets anyway
						OldParent->Children.RemoveAt(OldIndex);
					}
				}
			}

			UE::FSdfLayer ParentLayer;
			int32 TargetSubLayerIndex = INDEX_NONE;
			switch (Zone)
			{
				case EItemDropZone::AboveItem:
				{
					// Here we'll add the dragged items as siblings of Item, before it

					FUsdLayerViewModel* ParentItem = Item->ParentItem;
					if (ensure(ParentItem))
					{
						ParentLayer = ParentItem->GetLayer();
						TargetSubLayerIndex = ParentItem->Children.IndexOfByKey(Item);
						ensure(TargetSubLayerIndex != INDEX_NONE);
					}
					break;
				}
				case EItemDropZone::BelowItem:
				{
					// Here we'll add the dragged items as siblings of Item, after it

					FUsdLayerViewModel* ParentItem = Item->ParentItem;
					if (ensure(ParentItem))
					{
						ParentLayer = ParentItem->GetLayer();
						TargetSubLayerIndex = ParentItem->Children.IndexOfByKey(Item);
						ensure(TargetSubLayerIndex != INDEX_NONE);
					}
					break;
				}
				default:
				case EItemDropZone::OntoItem:
				{
					// Just add the dragged items as children of Item, at the end of its list of children

					ParentLayer = Item->GetLayer();
					break;
				}
			}

			for (const TSharedRef<IUsdTreeViewItem>& DraggedItem : Op->DraggedItems)
			{
				const FUsdLayerViewModelRef& LayerItem = StaticCastSharedRef<FUsdLayerViewModel>(DraggedItem);
				UE::FSdfLayer LayerPin = LayerItem->GetLayer();

				const bool bInserted = UsdUtils::InsertSubLayer(
					ParentLayer,
					*LayerPin.GetIdentifier(),
					Zone == EItemDropZone::OntoItem	   ? -1	   // Always at the end
					: Zone == EItemDropZone::AboveItem ? TargetSubLayerIndex++
													   : ++TargetSubLayerIndex
				);

				if (!bInserted)
				{
					USD_LOG_WARNING(
						TEXT("Failed to insert layer '%s' as a sublayer of '%s' at index '%d'"),
						*LayerPin.GetIdentifier(),
						*ParentLayer.GetIdentifier(),
						TargetSubLayerIndex
					);
				}
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

TArray<UE::FSdfLayer> SUsdLayersTreeView::GetSelectedLayers() const
{
	TArray<UE::FSdfLayer> SelectedLayers;
	TArray<FUsdLayerViewModelRef> SelectedViewModels = GetSelectedItems();
	SelectedLayers.Reserve(SelectedViewModels.Num());

	for (const FUsdLayerViewModelRef& SelectedItem : SelectedViewModels)
	{
		SelectedLayers.AddUnique(SelectedItem->GetLayer());
	}

	return SelectedLayers;
}

void SUsdLayersTreeView::SetSelectedLayers(const TArray<UE::FSdfLayer>& NewSelection)
{
	TSet<FString> IdentifiersToSelect;
	IdentifiersToSelect.Reserve(NewSelection.Num());
	for (const UE::FSdfLayer& SelectedLayer : NewSelection)
	{
		IdentifiersToSelect.Add(SelectedLayer.GetIdentifier());
	}

	TArray<FUsdLayerViewModelRef> ItemsToSelect;
	ItemsToSelect.Reserve(NewSelection.Num());

	TFunction<void(const TArray<FUsdLayerViewModelRef>&)> Traverse;
	Traverse = [this, &IdentifiersToSelect, &ItemsToSelect, &Traverse](const TArray<FUsdLayerViewModelRef>& Items)
	{
		for (const FUsdLayerViewModelRef& Item : Items)
		{
			if (IdentifiersToSelect.Contains(Item->LayerIdentifier))
			{
				ItemsToSelect.Add(Item);
			}

			Traverse(Item->Children);
		}
	};
	Traverse(RootItems);

	Private_ClearSelection();

	const bool bSelected = true;
	SetItemSelection(ItemsToSelect, bSelected);
}

TSharedRef<ITableRow> SUsdLayersTreeView::OnGenerateRow(FUsdLayerViewModelRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	// clang-format off
	return SNew(SUsdTreeRow< FUsdLayerViewModelRef >, InDisplayNode, OwnerTable, SharedData)
		.OnDragDetected(this, &SUsdLayersTreeView::OnRowDragDetected)
		.OnDragLeave(this, &SUsdLayersTreeView::OnRowDragLeave)
		.OnCanAcceptDrop(this, &SUsdLayersTreeView::OnRowCanAcceptDrop)
		.OnAcceptDrop(this, &SUsdLayersTreeView::OnRowAcceptDrop);
	// clang-format on
}

void SUsdLayersTreeView::OnGetChildren(FUsdLayerViewModelRef InParent, TArray<FUsdLayerViewModelRef>& OutChildren) const
{
	for (const FUsdLayerViewModelRef& Child : InParent->GetChildren())
	{
		OutChildren.Add(Child);
	}
}

void SUsdLayersTreeView::BuildUsdLayersEntries()
{
	RootItems.Empty();

	if (UsdStage)
	{
		RootItems.Add(MakeSharedUnreal<FUsdLayerViewModel>(nullptr, UsdStage, IsolatedStage, UsdStage.GetRootLayer().GetIdentifier()));
		RootItems.Add(MakeSharedUnreal<FUsdLayerViewModel>(nullptr, UsdStage, IsolatedStage, UsdStage.GetSessionLayer().GetIdentifier()));
	}
}

void SUsdLayersTreeView::SetupColumns()
{
	HeaderRowWidget->ClearColumns();

	SHeaderRow::FColumn::FArguments LayerMutedColumnArguments;
	LayerMutedColumnArguments.FixedWidth(24.f);
	LayerMutedColumnArguments.HAlignHeader(HAlign_Center);
	LayerMutedColumnArguments.HAlignCell(HAlign_Center);
	TSharedRef<FUsdLayerMutedColumn> LayerMutedColumn = MakeShared<FUsdLayerMutedColumn>();
	AddColumn(TEXT("Mute"), FText(), LayerMutedColumn, LayerMutedColumnArguments);

	TSharedRef<FUsdLayerNameColumn> LayerNameColumn = MakeShared<FUsdLayerNameColumn>();
	LayerNameColumn->bIsMainColumn = true;

	AddColumn(TEXT("Layers"), LOCTEXT("Layers", "Layers"), LayerNameColumn);

	TSharedRef<FUsdLayerEditColumn> LayerEditColumn = MakeShared<FUsdLayerEditColumn>();
	AddColumn(TEXT("Edit"), LOCTEXT("Edit", "Edit"), LayerEditColumn);
}

TSharedPtr<SWidget> SUsdLayersTreeView::ConstructLayerContextMenu()
{
	TSharedRef<SWidget> MenuWidget = SNullWidget::NullWidget;

	FMenuBuilder LayerOptions(true, nullptr);
	LayerOptions.BeginSection("Layer", LOCTEXT("Layer", "Layer"));
	{
		LayerOptions.AddMenuEntry(
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					if (IsolatedStage)
					{
						TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
						if (MySelectedItems.Num() == 1)
						{
							const UE::FSdfLayerWeak& Layer = MySelectedItems[0]->GetLayer();
							if (Layer == IsolatedStage.GetRootLayer())
							{
								return LOCTEXT("StopIsolatingStage_Text", "Stop isolating");
							}
						}
					}

					return LOCTEXT("IsolateStage_Text", "Isolate");
				}
			)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					if (IsolatedStage)
					{
						TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
						if (MySelectedItems.Num() == 1)
						{
							const UE::FSdfLayerWeak& Layer = MySelectedItems[0]->GetLayer();
							if (Layer == IsolatedStage.GetRootLayer())
							{
								return LOCTEXT(
									"StopIsolatingStage_ToolTip",
									"Stops isolating this layer and go back to showing the full composed stage"
								);
							}
						}
					}

					return LOCTEXT("IsolateStage_ToolTip", "Isolate a stage with this layer as its root layer");
				}
			)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnIsolateSelectedLayer),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanIsolateSelectedLayer)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("EditLayer", "Edit"),
			LOCTEXT("EditLayer_ToolTip", "Sets the layer as the edit target"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnEditSelectedLayer),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanEditSelectedLayer)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					bool bAnyIsMuted = false;
					TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
					for (const FUsdLayerViewModelRef& SelectedItem : MySelectedItems)
					{
						if (SelectedItem->IsLayerMuted())
						{
							bAnyIsMuted = true;
							break;
						}
					}

					return bAnyIsMuted ? LOCTEXT("UnmuteLayerRightClick_Text", "Unmute") : LOCTEXT("MuteLayerRightClick_Text", "Mute");
				}
			)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					const bool bCanMute = CanMuteSelectedLayer();

					bool bAnyIsMuted = false;
					if (bCanMute)
					{
						TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
						for (const FUsdLayerViewModelRef& SelectedItem : MySelectedItems)
						{
							if (SelectedItem->IsLayerMuted())
							{
								bAnyIsMuted = true;
								break;
							}
						}
					}

					return bCanMute ? (bAnyIsMuted ? LOCTEXT("UnmuteLayerRightClick_ToolTip", "Unmute the selected muted layers")
												   : LOCTEXT("MuteLayerRightClick_ToolTip", "Mute all selected layers"))
									: LOCTEXT("CantMuteLayerRightClick_ToolTip", "Cannot mute root layers or edit targets");
				}
			)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnMuteSelectedLayer),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanMuteSelectedLayer)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("ClearLayer", "Clear"),
			LOCTEXT("ClearLayer_ToolTip", "Clears this layer of all data"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnClearSelectedLayers),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanClearSelectedLayers)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("ReloadLayer", "Reload"),
			LOCTEXT("ReloadLayer_ToolTip", "Reloads just this layer from disk"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnReloadSelectedLayers),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanReloadSelectedLayers)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("SaveLayer", "Save"),
			LOCTEXT("SaveLayer_ToolTip", "Saves the layer modifications to disk"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnSaveSelectedLayers),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanSaveSelectedLayers)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("ExportLayer", "Export"),
			LOCTEXT("Export_ToolTip", "Export the selected layers, having the exported layers reference the original stage's layers"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						ExportSelectedLayers();
					}
				),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	LayerOptions.EndSection();

	LayerOptions.BeginSection("SubLayers", LOCTEXT("SubLayers", "SubLayers"));
	{
		LayerOptions.AddMenuEntry(
			LOCTEXT("AddExistingSubLayer", "Add Existing"),
			LOCTEXT("AddExistingSubLayer_ToolTip", "Adds a sublayer from an existing file to this layer"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnAddSubLayer),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanInsertSubLayer)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("AddNewSubLayer", "Add New"),
			LOCTEXT("AddNewSubLayer_ToolTip", "Adds a sublayer using a new file to this layer"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnNewSubLayer),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanInsertSubLayer)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("RemoveSubLayer", "Remove"),
			LOCTEXT("RemoveSubLayer_ToolTip", "Removes the sublayer from its owner"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdLayersTreeView::OnRemoveSelectedLayers),
				FCanExecuteAction::CreateSP(this, &SUsdLayersTreeView::CanRemoveSelectedLayers)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	LayerOptions.EndSection();

	MenuWidget = LayerOptions.MakeWidget();

	return MenuWidget;
}

bool SUsdLayersTreeView::CanIsolateSelectedLayer() const
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	if (MySelectedItems.Num() != 1)
	{
		return false;
	}

	FUsdLayerViewModelRef SelectedItem = MySelectedItems[0];
	const UE::FSdfLayerWeak& Layer = SelectedItem->GetLayer();
	const bool bLayerIsStageRoot = Layer == UsdStage.GetRootLayer();

	return SelectedItem->IsValid() && !bLayerIsStageRoot;
}

void SUsdLayersTreeView::OnIsolateSelectedLayer()
{
	if (!LayerIsolatedDelegate.IsBound())
	{
		return;
	}

	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	if (MySelectedItems.Num() != 1)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("IsolateTransaction", "Changed layer isolation"));

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		UE::FSdfLayer Layer = SelectedItem->GetLayer();

		if (IsolatedStage && IsolatedStage.GetRootLayer() == Layer)
		{
			// We're already isolating this one -> Toggle isolate mode off
			LayerIsolatedDelegate.Execute(UE::FSdfLayer{});
		}
		else
		{
			// Isolate the provided layer
			LayerIsolatedDelegate.Execute(Layer);
		}
	}
}

bool SUsdLayersTreeView::CanEditSelectedLayer() const
{
	bool bHasEditableLayer = false;

	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->CanEditLayer())
		{
			bHasEditableLayer = true;
			break;
		}
	}

	return bHasEditableLayer;
}

void SUsdLayersTreeView::OnEditSelectedLayer()
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->EditLayer())
		{
			break;
		}
	}
}

bool SUsdLayersTreeView::CanMuteSelectedLayer() const
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	if (MySelectedItems.Num() < 1)
	{
		return false;
	}

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->CanMuteLayer())
		{
			return false;
		}
	}

	return true;
}

void SUsdLayersTreeView::OnMuteSelectedLayer()
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
	if (MySelectedItems.Num() < 1)
	{
		return;
	}

	bool bAnyIsMuted = false;
	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->IsLayerMuted())
		{
			bAnyIsMuted = true;
			break;
		}
	}

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if ((bAnyIsMuted && SelectedItem->IsLayerMuted()) || !bAnyIsMuted)
		{
			UE::USDLayersTreeViewImpl::Private::ToggleMuteOrShowWarning(SelectedItem);
		}
	}

	return;
}

void SUsdLayersTreeView::OnClearSelectedLayers()
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	// We'll show a confirmation toast, which is non-modal. So keep track of the original selected items so that
	// if anything changes by the time we accept the dialog we'll still know which layers to clear
	TArray<UE::FSdfLayerWeak> SelectedLayers;
	SelectedLayers.Reserve(MySelectedItems.Num());

	FString LayerNames;
	const FString Separator = TEXT(", ");
	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (UE::FSdfLayer Layer = SelectedItem->GetLayer())
		{
			SelectedLayers.Add(Layer);
			LayerNames += Layer.GetDisplayName() + Separator;
		}
	}
	LayerNames.RemoveFromEnd(Separator);

	TFunction<void()> ClearSelectedLayersInner = [SelectedLayers]()
	{
		// This doesn't work very well USD-wise yet, and I don't know how we can possibly undo/redo
		// clearing a layer like this. We do need a transaction though, as this may trigger the creation
		// or deletion of UObjects
		FScopedTransaction Transaction(LOCTEXT("ClearTransaction", "Clear selected layers"));

		for (UE::FSdfLayerWeak Layer : SelectedLayers)
		{
			if (Layer)
			{
				Layer.Clear();
			}
		}
	};

	const UUsdProjectSettings* Settings = GetDefault<UUsdProjectSettings>();
	if (Settings && Settings->bShowConfirmationWhenClearingLayers)
	{
		static TWeakPtr<SNotificationItem> Notification;

		FNotificationInfo Toast(LOCTEXT("ConfirmClearingLayer", "Clearing cannot be undone"));
		Toast.SubText = FText::Format(
			LOCTEXT(
				"ConfirmClearingLayer_Subtext",
				"Clearing USD layers cannot be undone.\n\nDo you wish to proceed clearing {0}|plural(one=layer,other=layers) '{1}' ?"
			),
			SelectedLayers.Num(),
			FText::FromString(LayerNames)
		);
		Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		Toast.bUseLargeFont = false;
		Toast.bFireAndForget = false;
		Toast.FadeOutDuration = 0.0f;
		Toast.ExpireDuration = 0.0f;
		Toast.bUseThrobber = false;
		Toast.bUseSuccessFailIcons = false;

		Toast.ButtonDetails.Emplace(
			LOCTEXT("ConfirmClearingLayerOkAll", "Always proceed"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda(
				[ClearSelectedLayersInner]()
				{
					if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
					{
						PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
						PinnedNotification->ExpireAndFadeout();

						UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>();
						Settings->bShowConfirmationWhenClearingLayers = false;
						Settings->SaveConfig();

						ClearSelectedLayersInner();
					}
				}
			)
		);

		Toast.ButtonDetails.Emplace(
			LOCTEXT("ConfirmClearingLayerOk", "Proceed"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda(
				[ClearSelectedLayersInner]()
				{
					if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
					{
						PinnedNotification->SetCompletionState(SNotificationItem::CS_Success);
						PinnedNotification->ExpireAndFadeout();

						ClearSelectedLayersInner();
					}
				}
			)
		);

		Toast.ButtonDetails.Emplace(
			LOCTEXT("ConfirmClearingLayerCancel", "Cancel"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda(
				[]()
				{
					if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
					{
						PinnedNotification->SetCompletionState(SNotificationItem::CS_Fail);
						PinnedNotification->ExpireAndFadeout();
					}
				}
			)
		);

		// Only show one at a time
		if (!Notification.IsValid())
		{
			Notification = FSlateNotificationManager::Get().AddNotification(Toast);
		}

		if (TSharedPtr<SNotificationItem> PinnedNotification = Notification.Pin())
		{
			PinnedNotification->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
	else
	{
		// Don't show prompt, just clear
		ClearSelectedLayersInner();
	}
}

bool SUsdLayersTreeView::CanClearSelectedLayers() const
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->GetLayer().IsEmpty())
		{
			return true;
		}
	}

	return false;
}

void SUsdLayersTreeView::OnReloadSelectedLayers()
{
	FScopedTransaction Transaction(LOCTEXT("ReloadLayersTransaction", "Reload selected layers"));

	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		UE::USDLayersTreeViewImpl::Private::ReloadOrShowWarning(SelectedItem);
	}
}

bool SUsdLayersTreeView::CanReloadSelectedLayers() const
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->CanReload())
		{
			return false;
		}
	}

	return true;
}

void SUsdLayersTreeView::OnSaveSelectedLayers()
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->IsLayerDirty())
		{
			const bool bForce = true;
			SelectedItem->GetLayer().Save(bForce);
		}
	}
}

bool SUsdLayersTreeView::CanSaveSelectedLayers() const
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->IsLayerDirty())
		{
			return true;
		}
	}

	return false;
}

void SUsdLayersTreeView::ExportSelectedLayers(const FString& OutputLayerOrDirectory) const
{
	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();

	TArray<UE::FSdfLayer> LayersToExport;
	LayersToExport.Reserve(MySelectedItems.Num());

	for (FUsdLayerViewModelRef SelectedItem : MySelectedItems)
	{
		UE::FSdfLayer SelectedLayer = SelectedItem->GetLayer();
		if (!SelectedLayer)
		{
			continue;
		}

		LayersToExport.Add(SelectedLayer);
	}

	double StartTime = FPlatformTime::Cycles64();
	FString Extension;
	FString OutputLayerOrDirectoryCopy = OutputLayerOrDirectory;

	// Single layer -> Allow picking the target layer filename
	if (LayersToExport.Num() == 1)
	{
		if (OutputLayerOrDirectoryCopy.IsEmpty())
		{
			TOptional<FString> UsdFilePath = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Save);
			if (!UsdFilePath.IsSet())
			{
				return;
			}

			OutputLayerOrDirectoryCopy = UsdFilePath.GetValue();
		}

		StartTime = FPlatformTime::Cycles64();
		Extension = FPaths::GetExtension(OutputLayerOrDirectoryCopy);

		UE::USDLayersTreeViewImpl::Private::ExportLayerToPath(LayersToExport[0], OutputLayerOrDirectoryCopy);
	}
	// Multiple layers -> Pick folder and export them with the same name
	else if (LayersToExport.Num() > 1)
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (!DesktopPlatform)
		{
			return;
		}

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
									   ? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
									   : nullptr;

		if (OutputLayerOrDirectoryCopy.IsEmpty())
		{
			if (!DesktopPlatform->OpenDirectoryDialog(
					ParentWindowHandle,
					LOCTEXT("ChooseFolder", "Choose output folder").ToString(),
					TEXT(""),
					OutputLayerOrDirectoryCopy
				))
			{
				return;
			}
			OutputLayerOrDirectoryCopy = FPaths::ConvertRelativePathToFull(OutputLayerOrDirectoryCopy);
		}

		StartTime = FPlatformTime::Cycles64();
		Extension = FPaths::GetExtension(LayersToExport[0].GetDisplayName());

		for (const UE::FSdfLayer& LayerToExport : LayersToExport)
		{
			FString TargetFileName = FPaths::GetCleanFilename(LayerToExport.GetRealPath());
			FString FullPath = FPaths::Combine(OutputLayerOrDirectoryCopy, TargetFileName);
			FString FinalFullPath = FullPath;

			uint32 Suffix = 0;
			while (FPaths::FileExists(FinalFullPath))
			{
				FinalFullPath = FString::Printf(TEXT("%s_%u"), *FullPath, Suffix++);
			}

			UE::USDLayersTreeViewImpl::Private::ExportLayerToPath(LayerToExport, FinalFullPath);
		}
	}

	// Send analytics
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace(TEXT("NumLayersExported"), LayersToExport.Num());

		bool bAutomated = false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		IUsdClassesModule::SendAnalytics(
			MoveTemp(EventAttributes),
			TEXT("ExportSelectedLayers"),
			bAutomated,
			ElapsedSeconds,
			LayersToExport.Num() > 0 ? UsdUtils::GetSdfLayerNumFrames(LayersToExport[0]) : 0,
			Extension
		);
	}
}

bool SUsdLayersTreeView::CanInsertSubLayer() const
{
	return GetSelectedItems().Num() > 0;
}

void SUsdLayersTreeView::OnAddSubLayer()
{
	TOptional<FString> SubLayerFile = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Composition);

	if (!SubLayerFile)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddExistingSublayerTransaction", "Add existing sublayer"));

	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
	if (MySelectedItems.Num() > 0)
	{
		FUsdLayerViewModelRef SelectedItem = MySelectedItems[0];
		SelectedItem->AddSubLayer(*SubLayerFile.GetValue());
	}

	RequestTreeRefresh();
}

void SUsdLayersTreeView::OnNewSubLayer()
{
	TOptional<FString> SubLayerFile = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Save);

	if (!SubLayerFile)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddNewSublayerTransaction", "Add new sublayer"));

	TArray<FUsdLayerViewModelRef> MySelectedItems = GetSelectedItems();
	if (MySelectedItems.Num() > 0)
	{
		FUsdLayerViewModelRef SelectedItem = MySelectedItems[0];
		SelectedItem->NewSubLayer(*SubLayerFile.GetValue());
	}

	RequestTreeRefresh();
}

bool SUsdLayersTreeView::CanRemoveLayer(FUsdLayerViewModelRef LayerItem) const
{
	// We can't remove root layers
	return (LayerItem->IsValid() && LayerItem->ParentItem && LayerItem->ParentItem->IsValid());
}

bool SUsdLayersTreeView::CanRemoveSelectedLayers() const
{
	bool bHasRemovableLayer = false;

	TArray<FUsdLayerViewModelRef> SelectedLayers = GetSelectedItems();

	for (FUsdLayerViewModelRef SelectedLayer : SelectedLayers)
	{
		// We can't remove root layers
		if (CanRemoveLayer(SelectedLayer))
		{
			bHasRemovableLayer = true;
			break;
		}
	}

	return bHasRemovableLayer;
}

void SUsdLayersTreeView::OnRemoveSelectedLayers()
{
	bool bLayerRemoved = false;

	TArray<FUsdLayerViewModelRef> SelectedLayers = GetSelectedItems();

	FScopedTransaction Transaction(LOCTEXT("RemoveSublayerTransaction", "Remove sublayers"));

	for (FUsdLayerViewModelRef SelectedLayer : SelectedLayers)
	{
		if (!CanRemoveLayer(SelectedLayer))
		{
			continue;
		}

		int32 SubLayerIndex = 0;
		for (FUsdLayerViewModelRef Child : SelectedLayer->ParentItem->Children)
		{
			if (Child->LayerIdentifier == SelectedLayer->LayerIdentifier)
			{
				if (SelectedLayer->ParentItem)
				{
					bLayerRemoved = SelectedLayer->ParentItem->RemoveSubLayer(SubLayerIndex);
				}
				break;
			}

			++SubLayerIndex;
		}
	}

	if (bLayerRemoved)
	{
		RequestTreeRefresh();
	}
}

void SUsdLayersTreeView::RestoreExpansionStates()
{
	TFunction<void(const FUsdLayerViewModelRef&)> SetExpansionRecursive = [&](const FUsdLayerViewModelRef& Item)
	{
		if (Item->IsValid())
		{
			if (bool* bFoundExpansionState = TreeItemExpansionStates.Find(Item->LayerIdentifier))
			{
				SetItemExpansion(Item, *bFoundExpansionState);
			}
			// Default to showing everything expanded
			else
			{
				const bool bShouldExpand = true;
				SetItemExpansion(Item, bShouldExpand);
			}
		}

		for (const FUsdLayerViewModelRef& Child : Item->Children)
		{
			SetExpansionRecursive(Child);
		}
	};

	for (const FUsdLayerViewModelRef& RootItem : RootItems)
	{
		SetExpansionRecursive(RootItem);
	}
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
