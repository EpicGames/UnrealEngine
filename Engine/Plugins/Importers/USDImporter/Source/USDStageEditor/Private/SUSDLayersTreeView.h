// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/UsdStage.h"
#include "Widgets/SUSDTreeView.h"

#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"

class AUsdStageActor;

using FUsdLayerViewModelRef = TSharedRef<class FUsdLayerViewModel>;
using FUsdLayerViewModelWeak = TWeakPtr<class FUsdLayerViewModel>;

DECLARE_DELEGATE_OneParam(FOnLayerIsolated, const UE::FSdfLayer&);

class SUsdLayersTreeView : public SUsdTreeView<FUsdLayerViewModelRef>
{
public:
	SLATE_BEGIN_ARGS(SUsdLayersTreeView)
	{
	}
	SLATE_EVENT(FOnLayerIsolated, OnLayerIsolated)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void Refresh(const UE::FUsdStageWeak& NewStage, const UE::FUsdStageWeak& IsolatedStage = {}, bool bResync = false);

	// Drag and drop interface for our rows
	FReply OnRowDragDetected(const FGeometry& Geometry, const FPointerEvent& PointerEvent);
	void OnRowDragLeave(const FDragDropEvent& Event);
	TOptional<EItemDropZone> OnRowCanAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, FUsdLayerViewModelRef Item);
	FReply OnRowAcceptDrop(const FDragDropEvent& Event, EItemDropZone Zone, FUsdLayerViewModelRef Item);
	// End drag and drop interface

	TArray<UE::FSdfLayer> GetSelectedLayers() const;
	void SetSelectedLayers(const TArray<UE::FSdfLayer>& NewSelection);

	void ExportSelectedLayers(const FString& OutputDirectory = {}) const;

	const UE::FUsdStageWeak& GetStage() const
	{
		return UsdStage;
	}

private:
	virtual TSharedRef<ITableRow> OnGenerateRow(FUsdLayerViewModelRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable) override;
	virtual void OnGetChildren(FUsdLayerViewModelRef InParent, TArray<FUsdLayerViewModelRef>& OutChildren) const override;

	virtual void SetupColumns() override;

	void BuildUsdLayersEntries();

	TSharedPtr<SWidget> ConstructLayerContextMenu();

	bool CanIsolateSelectedLayer() const;
	void OnIsolateSelectedLayer();

	bool CanEditSelectedLayer() const;
	void OnEditSelectedLayer();

	bool CanMuteSelectedLayer() const;
	void OnMuteSelectedLayer();

	void OnClearSelectedLayers();
	bool CanClearSelectedLayers() const;

	void OnReloadSelectedLayers();
	bool CanReloadSelectedLayers() const;

	void OnSaveSelectedLayers();
	bool CanSaveSelectedLayers() const;

	bool CanInsertSubLayer() const;
	void OnAddSubLayer();
	void OnNewSubLayer();

	bool CanRemoveLayer(FUsdLayerViewModelRef LayerItem) const;
	bool CanRemoveSelectedLayers() const;
	void OnRemoveSelectedLayers();

	void RestoreExpansionStates();

public:
	// We update only on slate tick, but can receive a USD notice at any point, from any thread.
	// These variables are used to control what needs to be refreshed on the next tick

	enum class ELayersTreeViewState : uint8
	{
		NoRefreshNeeded = 0,
		RefreshNeeded = 1,
		ResyncNeeded = 2
	};
	ELayersTreeViewState bUpdateState = ELayersTreeViewState::NoRefreshNeeded;

	// Use a lock because some of the notices that will update these refresh variables can be
	// sent from TBB threads
	mutable FRWLock RefreshStateLock;

private:
	// Should always be valid, we keep the one we're given on Refresh()
	UE::FUsdStageWeak UsdStage;

	// A stage we create based on one of the sublayers of UsdStage
	UE::FUsdStageWeak IsolatedStage;

	// So that we can store these across refreshes
	TMap<FString, bool> TreeItemExpansionStates;

	// Used so that we can isolate the new layer without coupling to the SUSDStage widget too much
	FOnLayerIsolated LayerIsolatedDelegate;
};
