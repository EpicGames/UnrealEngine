// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDPrimInfo.h"

#include "Widgets/SUSDIntegrationsPanel.h"
#include "Widgets/SUSDObjectFieldList.h"
#include "Widgets/SUSDReferencesList.h"
#include "Widgets/SUSDVariantSetsList.h"

#include "UsdWrappers/UsdStage.h"

#include "Widgets/SBoxPanel.h"

#if USE_USD_SDK

#define LOCTEXT_NAMESPACE "SUSDPrimInfo"

void SUsdPrimInfo::Construct(const FArguments& InArgs)
{
	// auto as this has to fit the different types of list view
	auto MakeListViewGetter = [](auto ListViewSharedPtrPtr) -> TAttribute<float>
	{
		return TAttribute<float>::Create(TAttribute<float>::FGetter::CreateLambda(
			[ListViewSharedPtrPtr]()
			{
				if (ListViewSharedPtrPtr && (*ListViewSharedPtrPtr) && (*ListViewSharedPtrPtr)->GetItems().Num() > 0)
				{
					return 0.5f;
				}

				return 0.0f;
			}
		));
	};

	// clang-format off
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(PropertiesList, SUsdObjectFieldList)
			.NameColumnText(LOCTEXT("NameColumnText", "Name"))
			.OnSelectionChanged_Lambda([this](const TSharedPtr<FUsdObjectFieldViewModel>& NewSelection, ESelectInfo::Type SelectionType)
			{
				// Display property metadata if we have exactly one selected
				TArray<FString> SelectedFields = PropertiesList->GetSelectedFieldNames();
				if (PropertiesList && SelectedFields.Num() == 1 && NewSelection &&
					(NewSelection->Type == EObjectFieldType::Attribute || NewSelection->Type == EObjectFieldType::Relationship)
				)
				{
					PropertyMetadataPanel->SetObjectPath(
						PropertiesList->GetUsdStage(),
						*(FString{PropertiesList->GetObjectPath()} + "." + SelectedFields[0])
					);
					PropertyMetadataPanel->SetVisibility(EVisibility::Visible);
				}
				else
				{
					PropertyMetadataPanel->SetObjectPath({}, TEXT(""));
					PropertyMetadataPanel->SetVisibility(EVisibility::Collapsed);
				}
			})
		]

		+SVerticalBox::Slot()
		.FillHeight(MakeListViewGetter(&PropertyMetadataPanel))
		[
			SAssignNew(PropertyMetadataPanel, SUsdObjectFieldList)
			.NameColumnText_Lambda([this]() -> FText
			{
				FString PropertyName = FPaths::GetExtension(PropertyMetadataPanel->GetObjectPath());
				return FText::FromString(FString::Printf(TEXT("%s metadata"), *PropertyName));
			})
			.Visibility(EVisibility::Collapsed)
		]

		+SVerticalBox::Slot()
		.FillHeight(MakeListViewGetter(&IntegrationsPanel))
		[
			SAssignNew(IntegrationsPanel, SUsdIntegrationsPanel)
		]

		+SVerticalBox::Slot()
		.FillHeight(MakeListViewGetter(&VariantsList))
		[
			SAssignNew(VariantsList, SVariantsList)
		]

		+SVerticalBox::Slot()
		.FillHeight(MakeListViewGetter(&ReferencesList))
		[
			SAssignNew(ReferencesList, SUsdReferencesList)
		]
	];
	// clang-format on
}

void SUsdPrimInfo::SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath)
{
	if (PropertiesList)
	{
		TArray<FString> OldSelectedProperties = PropertiesList->GetSelectedFieldNames();
		TArray<FString> OldSelectedPropertyMetadata = PropertyMetadataPanel->GetSelectedFieldNames();

		// This is the main way with which we resync the properties list in case the prim received some change while
		// we were displaying it
		PropertiesList->SetObjectPath(UsdStage, PrimPath);

		// Try restoring the property selection while we change selected prims
		// This will also restore the object path on the PropertyMetadataPanel because PropertiesList::OnSelectionChanged
		// will fire
		PropertiesList->SetSelectedFieldNames(OldSelectedProperties);
		TArray<FString> NewSelectedProperties = PropertiesList->GetSelectedFieldNames();

		// If we haven't managed to select everything we had before, reset (and hide) the metadata panel.
		// We need this because apparently the list view doesn't generate a selection changed event when it's items are
		// fully rebuilt, which will happen on SetObjectPath. If SetSelectedFieldNames can't select anything, nothing will
		// update the metadata panel
		if (OldSelectedProperties.Num() != NewSelectedProperties.Num() || NewSelectedProperties.Num() != 1)
		{
			PropertyMetadataPanel->SetObjectPath({}, TEXT(""));
			PropertyMetadataPanel->SetVisibility(EVisibility::Collapsed);
		}
		else
		{
			// If we had a metadata panel open, try restoring the selection within that too
			PropertyMetadataPanel->SetSelectedFieldNames(OldSelectedPropertyMetadata);
		}
	}

	if (IntegrationsPanel)
	{
		IntegrationsPanel->SetPrimPath(UsdStage, PrimPath);
	}

	if (VariantsList)
	{
		VariantsList->SetPrimPath(UsdStage, PrimPath);
	}

	if (ReferencesList)
	{
		ReferencesList->SetPrimPath(UsdStage, PrimPath);
	}
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
