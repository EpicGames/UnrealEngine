// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "UsdWrappers/ForwardDeclarations.h"

#if USE_USD_SDK

class SUsdPrimInfo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUsdPrimInfo)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath);

private:
	friend class SUsdStage;
	TSharedPtr<class SUsdObjectFieldList> PropertiesList;
	TSharedPtr<class SUsdObjectFieldList> PropertyMetadataPanel;
	TSharedPtr<class SUsdIntegrationsPanel> IntegrationsPanel;
	TSharedPtr<class SVariantsList> VariantsList;
	TSharedPtr<class SUsdReferencesList> ReferencesList;
};

#endif	  // #if USE_USD_SDK
