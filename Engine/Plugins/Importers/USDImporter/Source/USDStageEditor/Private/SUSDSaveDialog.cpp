// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDSaveDialog.h"

#include "USDStageActor.h"

#include "Editor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SUsdSaveDialog"

namespace UE::UsdSaveDialog::Private
{
	FName CheckboxColumn = TEXT("Checkbox");
	FName IdentifierColumn = TEXT("Identifier");
	FName UsedByStagesColumn = TEXT("Used by stages");
	FName UsedByActorsColumn = TEXT("Used by actors");
}

TArray<FUsdSaveDialogRowData> SUsdSaveDialog::ShowDialog(
	const TArray<FUsdSaveDialogRowData>& InRows,
	const FText& WindowTitle,
	const FText& DescriptionText,
	bool* OutShouldSave,
	bool* OutShouldPromptAgain
)
{
	// clang-format off
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D{900, 500})
		.SizingRule(ESizingRule::UserSized);

	TSharedPtr<SUsdSaveDialog> SaveDialogWidget;
	Window->SetContent(
		SAssignNew(SaveDialogWidget, SUsdSaveDialog)
		.Rows(InRows)
		.WidgetWindow(Window)
		.DescriptionText(DescriptionText)
	);
	// clang-format on
	Window->SetWidgetToFocusOnActivate(SaveDialogWidget);

	GEditor->EditorAddModalWindow(Window);

	TArray<FUsdSaveDialogRowData> Result;

	if (SaveDialogWidget->ShouldSave())
	{
		const TArray<TSharedPtr<FUsdSaveDialogRowData>>& SharedRows = SaveDialogWidget->Rows;
		Result.Reserve(SharedRows.Num());
		for (const TSharedPtr<FUsdSaveDialogRowData>& SharedRow : SharedRows)
		{
			Result.Add(*SharedRow);
		}
	}

	if (OutShouldSave)
	{
		*OutShouldSave = SaveDialogWidget->ShouldSave();
	}

	if (OutShouldPromptAgain)
	{
		*OutShouldPromptAgain = SaveDialogWidget->ShouldPromptAgain();
	}

	return Result;
}

void SUsdSaveDialog::Construct(const FArguments& InArgs)
{
	// Copy into shared pointers because of the list view
	TArray<TSharedPtr<FUsdSaveDialogRowData>> SharedRows;
	SharedRows.Reserve(InArgs._Rows.Num());
	for (const FUsdSaveDialogRowData& RawRow : InArgs._Rows)
	{
		SharedRows.Add(MakeShared<FUsdSaveDialogRowData>(RawRow));
	}

	Rows = SharedRows;
	Window = InArgs._WidgetWindow;

	TSharedRef<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow);
	// clang-format off
	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(UE::UsdSaveDialog::Private::CheckboxColumn)
		[
			SNew(SBox)
			.Padding(FMargin(6,3,6,3))
			.HAlign(HAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					ECheckBoxState Result = ECheckBoxState::Checked;

					for (TSharedPtr<FUsdSaveDialogRowData>& Row : Rows)
					{
						if (!Row->bSaveLayer)
						{
							// Follow logic within SPackagesDialog::GetToggleSelectedState:
							// If any of them is unchecked, treat them all as unchecked so that the first
							// click on the toggle all checkbox checks them
							Result = ECheckBoxState::Unchecked;
						}
					}

					return Result;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					for (TSharedPtr<FUsdSaveDialogRowData>& Row : Rows)
					{
						Row->bSaveLayer = NewState == ECheckBoxState::Checked;
					}
				})
			]
		]
		.FixedWidth(38.0f)
	);
	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(UE::UsdSaveDialog::Private::IdentifierColumn)
		.DefaultLabel(LOCTEXT("IdentifierColumnLabel", "Identifier"))
		.FillWidth(4.0f)
	);
	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(UE::UsdSaveDialog::Private::UsedByStagesColumn)
		.DefaultLabel(LOCTEXT("UsedByStagesColumnLabel", "Used by stages"))
		.FillWidth(1.0f)
	);
	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(UE::UsdSaveDialog::Private::UsedByActorsColumn)
		.DefaultLabel(LOCTEXT("UsedByActorsColumnLabel", "Used by actors"))
		.FillWidth(1.0f)
	);
	const bool bAccept = true;
	const bool bCancel = false;

	TSharedPtr< SHorizontalBox > ButtonsBox = SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5, 0)
		[
			SNew(SCheckBox)
			.IsChecked(false) // If we're showing the prompt we know we have this unchecked
			.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
			{
				bPromptAgain = (NewState == ECheckBoxState::Unchecked);
			})
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DontPromptAgainText", "Don't prompt again"))
				.AutoWrapText(true)
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5, 0)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get(), "PrimaryButton")
			.TextStyle(&FAppStyle::Get(), "PrimaryButtonText")
			.Text(LOCTEXT("SaveSelectedText", "Save selected"))
			.ToolTipText(LOCTEXT("SaveSelectedToolTip", "Saves the selected layers to disk"))
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SUsdSaveDialog::Close, bAccept)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5, 0)
		[
			SNew(SButton)
			.ButtonStyle(&FAppStyle::Get(), "Button")
			.TextStyle(&FAppStyle::Get(), "ButtonText")
			.Text(LOCTEXT("SaveNoneText", "Don't save layers"))
			.ToolTipText(LOCTEXT("SaveNoneToolTip", "Proceed with the Unreal save, but don't save any USD layer"))
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SUsdSaveDialog::Close, bCancel)
		];

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(16))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f,0.0f,0.0f,8.0f)
			[
				SNew(STextBlock)
				.Text(InArgs._DescriptionText)
				.AutoWrapText(true)
			]
			+SVerticalBox::Slot()
			.FillHeight(0.8)
			[
				SNew(SListView< TSharedPtr<FUsdSaveDialogRowData> >)
				.ListItemsSource(&Rows)
				.OnGenerateRow(this, &SUsdSaveDialog::OnGenerateListRow)
				.HeaderRow(HeaderRowWidget)
				.SelectionMode(ESelectionMode::None)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 16.0f, 0.0f, 0.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				ButtonsBox.ToSharedRef()
			]
		]
	];
	// clang-format on
}

bool SUsdSaveDialog::SupportsKeyboardFocus() const
{
	return true;
}

FReply SUsdSaveDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		// If we're exiting via Esc, don't assume we want to commit our choice of bPromptAgain
		bPromptAgain = true;

		const bool bShouldProceed = false;
		return Close(bShouldProceed);
	}

	return FReply::Unhandled();
}

class SUsdSaveDialogRow : public SMultiColumnTableRow<TSharedPtr<FUsdSaveDialogRowData>>
{
public:
	SLATE_BEGIN_ARGS(SUsdSaveDialogRow)
	{
	}
	SLATE_ARGUMENT(TSharedPtr<FUsdSaveDialogRowData>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;

		SMultiColumnTableRow<TSharedPtr<FUsdSaveDialogRowData>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		TSharedRef<SWidget> ItemContentWidget = SNullWidget::NullWidget;

		// clang-format off
		if (ColumnName == UE::UsdSaveDialog::Private::CheckboxColumn)
		{
			ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(FMargin(10, 3, 6, 3))
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() -> ECheckBoxState
					{
						if (Item)
						{
							return Item->bSaveLayer
								? ECheckBoxState::Checked
								: ECheckBoxState::Unchecked;
						}

						return ECheckBoxState::Undetermined;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						if (Item)
						{
							Item->bSaveLayer = NewState == ECheckBoxState::Checked;
						}
					})
				];
		}
		else if (ColumnName == UE::UsdSaveDialog::Private::IdentifierColumn)
		{
			if (Item && Item->Layer)
			{
				ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(3, 3, 3, 3)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Item->Layer.GetIdentifier()))
				];
			}
		}
		else if (ColumnName == UE::UsdSaveDialog::Private::UsedByStagesColumn)
		{
			if (Item)
			{
				FString Text;
				for (const UE::FUsdStageWeak& Stage : Item->ConsumerStages)
				{
					if (Stage)
					{
						Text += Stage.GetRootLayer().GetDisplayName() + TEXT(", ");
					}
				}
				Text.RemoveFromEnd(TEXT(", "));

				ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(3, 3, 3, 3)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Text))
				];
			}
		}
		else if (ColumnName == UE::UsdSaveDialog::Private::UsedByActorsColumn)
		{
			if (Item)
			{
				FString Text;
				FString Delimiter = TEXT(", ");

				for (const AUsdStageActor* Actor : Item->ConsumerActors)
				{
					if (Actor)
					{
						Text += Actor->GetActorLabel() + Delimiter;
					}
				}
				Text.RemoveFromEnd(Delimiter);

				ItemContentWidget = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(3, 3, 3, 3)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Text))
				];
			}
		}
		// clang-format on

		return ItemContentWidget;
	}

private:
	TSharedPtr<FUsdSaveDialogRowData> Item;
};

TSharedRef<ITableRow> SUsdSaveDialog::OnGenerateListRow(TSharedPtr<FUsdSaveDialogRowData> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUsdSaveDialogRow, OwnerTable).Item(Item);
}

FReply SUsdSaveDialog::Close(bool bInProceed)
{
	bProceed = bInProceed;

	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
