// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class AUsdStageActor;
class ITableRow;
class STableViewBase;

struct FUsdSaveDialogRowData
{
	bool bSaveLayer = true;
	UE::FSdfLayerWeak Layer;
	TArray<UE::FUsdStageWeak> ConsumerStages;
	TSet<AUsdStageActor*> ConsumerActors;
};

/** Dialog shown to ask if the user wishes to save any dirty USD layer to disk */
class SUsdSaveDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUsdSaveDialog)
	{
	}
	SLATE_ARGUMENT(TArray<FUsdSaveDialogRowData>, Rows)
	SLATE_ARGUMENT(FText, DescriptionText)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_END_ARGS()

	// Returns the data describing which layers should be saved
	static TArray<FUsdSaveDialogRowData> ShowDialog(
		const TArray<FUsdSaveDialogRowData>& InRows,
		const FText& WindowTitle,
		const FText& DescriptionText,
		bool* OutShouldSave = nullptr,
		bool* OutShouldPromptAgain = nullptr
	);

	void Construct(const FArguments& InArgs);
	bool ShouldSave() const
	{
		return bProceed;
	}
	bool ShouldPromptAgain() const
	{
		return bPromptAgain;
	}

private:
	virtual bool SupportsKeyboardFocus() const override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	TSharedRef<ITableRow> OnGenerateListRow(TSharedPtr<FUsdSaveDialogRowData> Item, const TSharedRef<STableViewBase>& OwnerTable);

	FReply Close(bool bProceed);

private:
	TWeakPtr<SWindow> Window;
	TArray<TSharedPtr<FUsdSaveDialogRowData>> Rows;

	bool bPromptAgain = true;
	bool bProceed = false;
};
