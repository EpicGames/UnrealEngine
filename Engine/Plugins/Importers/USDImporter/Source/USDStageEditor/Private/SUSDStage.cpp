// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDStage.h"

#include "Objects/USDSchemaTranslator.h"
#include "SUSDLayersTreeView.h"
#include "SUSDPrimInfo.h"
#include "SUSDStageEditorStyle.h"
#include "SUSDStageTreeView.h"
#include "UnrealUSDWrapper.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMaterialUtils.h"
#include "USDProjectSettings.h"
#include "USDSetUtils.h"
#include "USDStageActor.h"
#include "USDStageEditorSettings.h"
#include "USDStageImportOptions.h"
#include "USDStageModule.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"
#include "Widgets/SUSDObjectFieldList.h"

#include "ActorTreeItem.h"
#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"
#include "LevelSequence.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SceneOutlinerModule.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSplitter.h"

#define LOCTEXT_NAMESPACE "SUsdStage"

#if USE_USD_SDK

namespace SUSDStageConstants
{
	static const FMargin SectionPadding(1.f, 1.f, 1.f, 1.f);
}

namespace SUSDStageImpl
{
	/**
	 * Makes sure that AllLayers includes all the external references of any of its members.
	 * Example: Receives AllLayers [a.usda], that references b.usda and c.usda, while c.usda also references d.usda -> Modifies AllLayers to be
	 * [a.usda, b.usda, c.usda, d.usda]
	 */
	void AppendAllExternalReferences(TArray<UE::FSdfLayer>& AllLayers)
	{
		// Consider paths in a case-sensitive way for linux
#if PLATFORM_LINUX
		UsdUtils::FCaseSensitiveStringSet LoadedLayers;
#else
		TSet<FString> LoadedLayers;
#endif

		LoadedLayers.Reserve(AllLayers.Num());
		for (const UE::FSdfLayer& Layer : AllLayers)
		{
			FString LayerPath = Layer.GetRealPath();
			FPaths::NormalizeFilename(LayerPath);
			LoadedLayers.Add(LayerPath);
		}

		for (int32 Index = 0; Index < AllLayers.Num(); ++Index)
		{
			UE::FSdfLayer& Layer = AllLayers[Index];

			TArray<UE::FSdfLayer> NewLayers;
			for (const FString& AssetDependency : Layer.GetCompositionAssetDependencies())
			{
				FString AbsoluteReference = Layer.ComputeAbsolutePath(AssetDependency);
				FPaths::NormalizeFilename(AbsoluteReference);

				if (!LoadedLayers.Contains(AbsoluteReference))
				{
					NewLayers.Add(UE::FSdfLayer::FindOrOpen(*AbsoluteReference));
					LoadedLayers.Add(AbsoluteReference);
				}
			}
			AllLayers.Append(NewLayers);
		}
	}

	void SelectGeneratedComponentsAndActors(AUsdStageActor* StageActor, const TArray<FString>& PrimPaths)
	{
		if (!StageActor)
		{
			return;
		}

		TSet<USceneComponent*> ComponentsToSelect;
		for (const FString& PrimPath : PrimPaths)
		{
			if (USceneComponent* GeneratedComponent = StageActor->GetGeneratedComponent(PrimPath))
			{
				ComponentsToSelect.Add(GeneratedComponent);
			}
		}

		TSet<AActor*> ActorsToSelect;
		for (TSet<USceneComponent*>::TIterator It(ComponentsToSelect); It; ++It)
		{
			if (AActor* Owner = (*It)->GetOwner())
			{
				// We always need the parent actor selected to select a component
				ActorsToSelect.Add(Owner);

				// If we're going to select a root component, select the actor instead. This is useful
				// because if we later press "F" to focus on the prim, having the actor selected will use the entire
				// actor's bounding box and focus on something. If all we have selected is an empty scene component
				// however, the camera won't focus on anything
				if (*It == Owner->GetRootComponent())
				{
					It.RemoveCurrent();
				}
			}
		}

		// Don't deselect anything if we're not going to select anything
		if (ActorsToSelect.Num() == 0 && ComponentsToSelect.Num() == 0)
		{
			return;
		}

		// Don't use GEditor->SelectNone() as that will affect *every type of selection in the editor*,
		// including even some UI menus, brushes, etc.
		if (USelection* SelectedActors = GEditor->GetSelectedActors())
		{
			SelectedActors->DeselectAll(AActor::StaticClass());
		}
		if (USelection* SelectedComponents = GEditor->GetSelectedComponents())
		{
			SelectedComponents->DeselectAll(UActorComponent::StaticClass());
		}

		const bool bSelected = true;
		const bool bNotifySelectionChanged = true;
		for (AActor* Actor : ActorsToSelect)
		{
			GEditor->SelectActor(Actor, bSelected, bNotifySelectionChanged);
		}

		for (USceneComponent* Component : ComponentsToSelect)
		{
			GEditor->SelectComponent(Component, bSelected, bNotifySelectionChanged);
		}
	}

	class FStageNameOutlinerColumn : public ISceneOutlinerColumn
	{
	public:
		FStageNameOutlinerColumn(ISceneOutliner& SceneOutliner)
			: WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared()))
		{
		}

		virtual ~FStageNameOutlinerColumn()
		{
		}
		static FName GetID()
		{
			static FName ID("Stage");
			return ID;
		}
		static FString GetStageForItem(const ISceneOutlinerTreeItem* Item)
		{
			if (Item)
			{
				if (const FActorTreeItem* ActorTreeItem = Item->CastTo<FActorTreeItem>())
				{
					if (const AUsdStageActor* StageActor = Cast<const AUsdStageActor>(ActorTreeItem->Actor.Get()))
					{
						if (const UE::FUsdStage& Stage = StageActor->GetUsdStage())
						{
							return Stage.GetRootLayer().GetDisplayName();
						}
					}
				}
			}

			return {};
		}

		// Begin ISceneOutlinerColumn Implementation
		virtual FName GetColumnID() override
		{
			return GetID();
		}
		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override
		{
			return SHeaderRow::Column(FName(TEXT("Stage")));
		}
		virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
			override
		{
			return SNew(STextBlock).Text(FText::FromString(GetStageForItem(&TreeItem.Get())));
		}
		virtual bool SupportsSorting() const override
		{
			return true;
		}
		virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override
		{
			if (SortMode == EColumnSortMode::Ascending)
			{
				OutItems.Sort(
					[](const FSceneOutlinerTreeItemPtr& First, const FSceneOutlinerTreeItemPtr& Second)
					{
						const ISceneOutlinerTreeItem* FirstPtr = First.Get();
						const ISceneOutlinerTreeItem* SecondPtr = Second.Get();
						return GetStageForItem(FirstPtr) < GetStageForItem(SecondPtr);
					}
				);
			}

			if (SortMode == EColumnSortMode::Descending)
			{
				OutItems.Sort(
					[](const FSceneOutlinerTreeItemPtr& First, const FSceneOutlinerTreeItemPtr& Second)
					{
						const ISceneOutlinerTreeItem* FirstPtr = First.Get();
						const ISceneOutlinerTreeItem* SecondPtr = Second.Get();
						return GetStageForItem(FirstPtr) > GetStageForItem(SecondPtr);
					}
				);
			}
		}
		// End ISceneOutlinerColumn Implementation

	private:
		TWeakPtr<ISceneOutliner> WeakSceneOutliner;
	};
}	 // namespace SUSDStageImpl

void SUsdStage::Construct(const FArguments& InArgs)
{
	const bool bFlashButton = true;
	OnActorLoadedHandle = AUsdStageActor::OnActorLoaded.AddSP(SharedThis(this), &SUsdStage::AttachToStageActor, bFlashButton);

	OnViewportSelectionChangedHandle = USelection::SelectionChangedEvent.AddSP(SharedThis(this), &SUsdStage::OnViewportSelectionChanged);

	PostPIEStartedHandle = FEditorDelegates::PostPIEStarted.AddSP(SharedThis(this), &SUsdStage::OnPostPIEStarted);
	EndPIEHandle = FEditorDelegates::EndPIE.AddSP(SharedThis(this), &SUsdStage::OnEndPIE);

	IUsdStageModule& UsdStageModule = FModuleManager::Get().LoadModuleChecked<IUsdStageModule>("UsdStage");
	ViewModel.UsdStageActor = UsdStageModule.FindUsdStageActor(IUsdClassesModule::GetCurrentWorld());

	bUpdatingViewportSelection = false;
	bUpdatingPrimSelection = false;

	UE::FUsdStage UsdStage;

	OnEditorCloseHandle = GEditor->OnEditorClose().AddLambda(
		[this]()
		{
			bEditorIsShuttingDown = true;
		}
	);

	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (StageActor)
	{
		UsdStage = static_cast<const AUsdStageActor*>(StageActor)->GetUsdStage();
	}

	// clang-format off
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
		[
			SNew(SVerticalBox)

			// Menu
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					MakeMainMenu()
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					.Padding(FMargin(0.0f, 0.0f, 10.0f, 0.0f))
					[
						MakeIsolateWarningButton()
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBox)
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						MakeActorPickerMenu()
					]
				]
			]

			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(SUSDStageConstants::SectionPadding)
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)

				+SSplitter::Slot()
				.Value(0.7f)
				[
					SNew(SSplitter)
					.Orientation(Orient_Horizontal)

					// Stage tree
					+SSplitter::Slot()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
						[
							SAssignNew(UsdStageTreeView, SUsdStageTreeView)
							.OnPrimSelectionChanged(this, &SUsdStage::OnPrimSelectionChanged)
						]
					]

					// Prim details
					+SSplitter::Slot()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SAssignNew(UsdPrimInfoWidget, SUsdPrimInfo)
						]
					]
				]

				// Layers tree
				+SSplitter::Slot()
				.Value(0.3f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
					[
						SAssignNew(UsdLayersTreeView, SUsdLayersTreeView)
						.OnLayerIsolated(this, &SUsdStage::OnLayerIsolated)
					]
				]
			]
		]
	];
	// clang-format on

	SetupStageActorDelegates();

	// Pre tick so that we setup/refresh our widgets before Slate renders on this frame
	FSlateApplication::Get().OnPreTick().AddSP(this, &SUsdStage::OnSlateTick);

	// We're opening the USD Stage editor for the first time and we already have a stage: Display it immediately
	if (UsdStage)
	{
		RequestFullRefresh();
	}
}

AUsdStageActor* SUsdStage::GetAttachedStageActor() const
{
	return ViewModel.UsdStageActor.Get();
}

TArray<UE::FSdfLayer> SUsdStage::GetSelectedLayers() const
{
	if (UsdLayersTreeView)
	{
		return UsdLayersTreeView->GetSelectedLayers();
	}

	return {};
}

void SUsdStage::SetSelectedLayers(const TArray<UE::FSdfLayer>& NewSelection) const
{
	if (UsdLayersTreeView)
	{
		UsdLayersTreeView->SetSelectedLayers(NewSelection);
	}
}

TArray<UE::FUsdPrim> SUsdStage::GetSelectedPrims() const
{
	if (UsdStageTreeView)
	{
		return UsdStageTreeView->GetSelectedPrims();
	}

	return {};
}

void SUsdStage::SetSelectedPrims(const TArray<UE::FUsdPrim>& NewSelection) const
{
	if (UsdStageTreeView)
	{
		UsdStageTreeView->SetSelectedPrims(NewSelection);
	}
}

TArray<FString> SUsdStage::GetSelectedPropertyNames() const
{
	if (UsdPrimInfoWidget && UsdPrimInfoWidget->PropertiesList)
	{
		return UsdPrimInfoWidget->PropertiesList->GetSelectedFieldNames();
	}

	return {};
}

void SUsdStage::SetSelectedPropertyNames(const TArray<FString>& NewSelection)
{
	if (UsdPrimInfoWidget && UsdPrimInfoWidget->PropertiesList)
	{
		UsdPrimInfoWidget->PropertiesList->SetSelectedFieldNames(NewSelection);
	}
}

TArray<FString> SUsdStage::GetSelectedPropertyMetadataNames() const
{
	if (UsdPrimInfoWidget && UsdPrimInfoWidget->PropertyMetadataPanel)
	{
		return UsdPrimInfoWidget->PropertyMetadataPanel->GetSelectedFieldNames();
	}

	return {};
}

void SUsdStage::SetSelectedPropertyMetadataNames(const TArray<FString>& NewSelection)
{
	if (UsdPrimInfoWidget && UsdPrimInfoWidget->PropertyMetadataPanel)
	{
		UsdPrimInfoWidget->PropertyMetadataPanel->SetSelectedFieldNames(NewSelection);
	}
}

void SUsdStage::AttachToStageActor(AUsdStageActor* InUsdStageActor, bool bFlashButton)
{
	if (ViewModel.UsdStageActor == InUsdStageActor)
	{
		return;
	}

	if (bFlashButton)
	{
		const float StartTimeSeconds = 0.0f;
		const float FlashDuration = 0.5f;
		const ECurveEaseFunction EaseFunction = ECurveEaseFunction::Linear;
		FlashActorPickerCurve = FCurveSequence{StartTimeSeconds, FlashDuration, EaseFunction};
		FlashActorPickerCurve.Play(this->AsShared());
	}

	ClearStageActorDelegates();
	ViewModel.UsdStageActor = InUsdStageActor;
	SetupStageActorDelegates();

	if (InUsdStageActor)
	{
		UE::FUsdStage UsdStage = static_cast<const AUsdStageActor*>(InUsdStageActor)->GetUsdStage();

		if (this->UsdPrimInfoWidget && InUsdStageActor)
		{
			// Just reset to the pseudoroot for now
			this->UsdPrimInfoWidget->SetPrimPath(UsdStage, TEXT("/"));
		}
	}

	// Refresh here because we may be receiving an actor that has a stage already loaded,
	// like during undo/redo
	RequestFullRefresh();
}

void SUsdStage::SetupStageActorDelegates()
{
	ClearStageActorDelegates();

	if (ViewModel.UsdStageActor.IsValid())
	{
		OnPrimChangedHandle = ViewModel.UsdStageActor->OnPrimChanged.AddLambda(
			[this](const FString& PrimPath, bool bResync)
			{
				if (UsdStageTreeView)
				{
					FWriteScopeLock StageTreeViewLock{UsdStageTreeView->RefreshStateLock};

					UsdStageTreeView->PrimsToRefresh.Emplace(PrimPath, bResync);
				}
			}
		);

		// Fired when we switch which is the currently opened stage
		OnStageChangedHandle = ViewModel.UsdStageActor->OnStageChanged.AddLambda(
			[this]()
			{
				if (UsdStageTreeView)
				{
					{
						FWriteScopeLock StageTreeViewLock{UsdStageTreeView->RefreshStateLock};
						UsdStageTreeView->PrimsToRefresh.Reset();
					}

					RequestFullRefresh();
				}
			}
		);

		OnActorDestroyedHandle = ViewModel.UsdStageActor->OnActorDestroyed.AddLambda(
			[this]()
			{
				// Refresh widgets on game thread, but close the stage right away. In some contexts this is important, for example when
				// running a Python script: If our USD Stage Editor is open and our script deletes an actor, it will trigger OnActorDestroyed.
				// If we had this CloseStage() call inside the ticker, it would only take place when the script has finished running
				// (and control flow returned to the game thread) which can lead to some weird results (and break automated tests).
				// We could get around this on the Python script's side by just yielding, but there may be other scenarios
				ClearStageActorDelegates();
				ViewModel.CloseStage();
				RequestFullRefresh();
			}
		);

		OnStageEditTargetChangedHandle = ViewModel.UsdStageActor->GetUsdListener().GetOnStageEditTargetChanged().AddSP(
			this,
			&SUsdStage::RequestLayersTreeViewRefresh
		);

		OnSdfLayersChangedHandle = ViewModel.UsdStageActor->GetUsdListener().GetOnSdfLayersChanged().AddLambda(
			[this](const UsdUtils::FLayerToSdfChangeList& LayersToChangeList)
			{
				RequestLayersTreeViewRefresh();
			}
		);

		OnSdfLayerDirtinessChangedHandle = ViewModel.UsdStageActor->GetUsdListener().GetOnSdfLayerDirtinessChanged().AddSP(
			this,
			&SUsdStage::RequestLayersTreeViewRefresh
		);
	}
}

void SUsdStage::ClearStageActorDelegates()
{
	const bool bEvenIfPendingKill = true;
	if (AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get(bEvenIfPendingKill))
	{
		StageActor->OnStageChanged.Remove(OnStageChangedHandle);
		StageActor->OnPrimChanged.Remove(OnPrimChangedHandle);
		StageActor->OnActorDestroyed.Remove(OnActorDestroyedHandle);

		StageActor->GetUsdListener().GetOnStageEditTargetChanged().Remove(OnStageEditTargetChangedHandle);
		StageActor->GetUsdListener().GetOnSdfLayersChanged().Remove(OnSdfLayersChangedHandle);
		StageActor->GetUsdListener().GetOnSdfLayerDirtinessChanged().Remove(OnSdfLayerDirtinessChangedHandle);
	}
}

SUsdStage::~SUsdStage()
{
	FSlateApplication::Get().OnPreTick().RemoveAll(this);

	FEditorDelegates::PostPIEStarted.Remove(PostPIEStartedHandle);
	FEditorDelegates::EndPIE.Remove(EndPIEHandle);

	GEditor->OnEditorClose().Remove(OnEditorCloseHandle);

	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnStageActorPropertyChangedHandle);
	AUsdStageActor::OnActorLoaded.Remove(OnActorLoadedHandle);
	USelection::SelectionChangedEvent.Remove(OnViewportSelectionChangedHandle);

	ClearStageActorDelegates();

	ActorPickerMenu.Reset();
}

TSharedRef<SWidget> SUsdStage::MakeMainMenu()
{
	FMenuBarBuilder MenuBuilder(nullptr);
	{
		// File
		MenuBuilder.AddPullDownMenu(
			LOCTEXT("FileMenu", "File"),
			LOCTEXT("FileMenu_ToolTip", "Opens the file menu"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillFileMenu)
		);

		// Actions
		MenuBuilder.AddPullDownMenu(
			LOCTEXT("ActionsMenu", "Actions"),
			LOCTEXT("ActionsMenu_ToolTip", "Opens the actions menu"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillActionsMenu)
		);

		// Options
		MenuBuilder.AddPullDownMenu(
			LOCTEXT("OptionsMenu", "Options"),
			LOCTEXT("OptionsMenu_ToolTip", "Opens the options menu"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillOptionsMenu)
		);
	}

	// Create the menu bar
	TSharedRef<SWidget> MenuBarWidget = MenuBuilder.MakeWidget();
	MenuBarWidget->SetVisibility(EVisibility::Visible);	   // Work around for menu bar not showing on Mac

	return MenuBarWidget;
}

TSharedRef<SWidget> SUsdStage::MakeActorPickerMenu()
{
	// clang-format off
	return
		SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.OnGetMenuContent(this, &SUsdStage::MakeActorPickerMenuContent)
			.MenuPlacement(EMenuPlacement::MenuPlacement_BelowRightAnchor)
			.ToolTipText(LOCTEXT("ActorPicker_ToolTip", "Switch the active stage actor"))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					if (AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
					{
						return FText::FromString(StageActor->GetActorLabel());
					}

					return FText::FromString("None");
				})
			]
		]

		// Overlay for flashing for attention
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.Visibility(EVisibility::HitTestInvisible)
			.BorderImage(FUsdStageEditorStyle::Get()->GetBrush("UsdStageEditor.ActorPickerFlashBrush"))
			.BorderBackgroundColor_Lambda([this]() -> FSlateColor
			{
				FLinearColor Color = FLinearColor::White;

				if (FlashActorPickerCurve.IsPlaying())
				{
					Color.A = 0.6f * (1.0f - FlashActorPickerCurve.GetLerp());
				}
				else
				{
					Color.A = 0.0f;
				}

				return FSlateColor(Color);
			})
		];
	// clang-format on
}

TSharedRef<SWidget> SUsdStage::MakeActorPickerMenuContent()
{
	if (!ActorPickerMenu.IsValid())
	{
		const int32 NamePriority = 0;
		const int32 StagePriority = 1;
		const bool bCanBeHidden = true;
		const float FillSize = 1.0f;

		FSceneOutlinerColumnInfo ActorColumnInfo{ESceneOutlinerColumnVisibility::Visible, NamePriority};
		ActorColumnInfo.ColumnLabel = LOCTEXT("ActorColumnText", "Actor");
		ActorColumnInfo.FillSize = FillSize;

		FSceneOutlinerColumnInfo StageColumnInfo{
			ESceneOutlinerColumnVisibility::Visible,
			StagePriority,
			FCreateSceneOutlinerColumn::CreateLambda(
				[](ISceneOutliner& InSceneOutliner)
				{
					return MakeShareable(new SUSDStageImpl::FStageNameOutlinerColumn(InSceneOutliner));
				}
			),
			bCanBeHidden,
			FillSize
		};

		FSceneOutlinerInitializationOptions InitOptions;
		InitOptions.bShowHeaderRow = true;	  // The header lets us resize/hide the columns if we want
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), ActorColumnInfo);
		InitOptions.ColumnMap.Add(SUSDStageImpl::FStageNameOutlinerColumn::GetID(), StageColumnInfo);
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(
			[](const AActor* Actor)
			{
				return Actor && Actor->IsA<AUsdStageActor>();
			}
		));

		FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
		ActorPickerMenu = SceneOutlinerModule.CreateActorPicker(
			InitOptions,
			FOnActorPicked::CreateLambda(
				[this](AActor* Actor)
				{
					if (Actor && Actor->IsA<AUsdStageActor>())
					{
						// Don't need to flash here as it was an intentional act to attach to a new actor
						const bool bFlashButton = false;
						AttachToStageActor(Cast<AUsdStageActor>(Actor), bFlashButton);

						FSlateApplication::Get().DismissAllMenus();
					}
				}
			)
		);
	}

	if (ActorPickerMenu.IsValid())
	{
		ActorPickerMenu->FullRefresh();

		// clang-format off
		return SNew(SBox)
			.Padding(FMargin(1))    // Add a small margin or else we'll get dark gray on dark gray which can look a bit confusing
			.MinDesiredWidth(600.0f)  // Force a min width or else the tree view item text will run up right to the very edge pixel of the menu
			.HAlign(HAlign_Fill)
			[
				ActorPickerMenu.ToSharedRef()
			];
		// clang-format on
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SUsdStage::MakeIsolateWarningButton()
{
	// clang-format off
	return SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.WarningButton"))
		.ButtonColorAndOpacity(FLinearColor{1.0f, 0.4f, 0.0f, 1.0f})
		.OnClicked_Lambda([this]() -> FReply
		{
			// Isolating nothing is how we un-isolate
			OnLayerIsolated(UE::FSdfLayerWeak{});
			return FReply::Handled();
		})
		.Visibility_Lambda([this]() -> EVisibility
		{
			if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
			{
				if (StageActor->GetIsolatedUsdStage())
				{
					return EVisibility::Visible;
				}
			}
			return EVisibility::Hidden;
		})
		.ToolTipText_Lambda([this]() -> FText
		{
			if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
			{
				if (const UE::FUsdStage& CurrentStage = StageActor->GetUsdStage())
				{
					return FText::Format(
						LOCTEXT("IsolateWarningButtonToolTip", "The USD Stage editor and the Unreal level are displaying an isolated stage with '{0}' as its root layer.\nPress this button to revert to displaying the entire composed stage."),
						FText::FromString(CurrentStage.GetRootLayer().GetDisplayName())
					);
				}
			}

			return FText::GetEmpty();
		})
		.Content()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("IsolateWarningButtonText", "Isolated Mode"))
		];
	// clang-format on
}

void SUsdStage::FillFileMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("File", LOCTEXT("File", "File"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("New", "New"),
			LOCTEXT("New_ToolTip", "Creates a new layer and opens the stage with it at its root"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SUsdStage::FileNew), FCanExecuteAction()),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Open", "Open..."),
			LOCTEXT("Open_ToolTip", "Pick an existing USD file"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						FileOpen();
					}
				),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Save", "Save"),
			LOCTEXT("Save_ToolTip", "Saves the stage"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						FileSave();
					}
				),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							if (UE::FUsdStage Stage = StageActor->GetUsdStage())
							{
								return true;
							}
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddSubMenu(LOCTEXT("Export", "Export"), FText::GetEmpty(), FNewMenuDelegate::CreateSP(this, &SUsdStage::FillExportSubMenu));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Reload", LOCTEXT("Reload", "Reload"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ReloadEntry", "Reload stage"),
			LOCTEXT("ReloadEntry_ToolTip", "Reloads the stage and animations from disk, keeping aspects of the session intact"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::FileReload),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							if (UE::FUsdStage Stage = StageActor->GetUsdStage())
							{
								if (!Stage.GetRootLayer().IsAnonymous())
								{
									return true;
								}
							}
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ResetState", "Reset state"),
			LOCTEXT("ResetState_ToolTip", "Resets the session layer and other options like edit target and muted layers"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::FileReset),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							if (UE::FUsdStage Stage = StageActor->GetUsdStage())
							{
								return true;
							}
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Close", LOCTEXT("Close", "Close"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Close", "Close"),
			LOCTEXT("Close_ToolTip", "Clears the stage actor's root layer, unloading and closing the stage"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::FileClose),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							if (!StageActor->RootLayer.FilePath.IsEmpty())
							{
								return true;
							}
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

void SUsdStage::FillActionsMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("Actions", LOCTEXT("Actions", "Actions"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("Import", "Import..."),
			LOCTEXT("Import_ToolTip", "Imports the stage as Unreal assets"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::ActionsImportWithDialog),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							if (UE::FUsdStage Stage = StageActor->GetUsdStage())
							{
								return true;
							}
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Regenerate", "Regenerate USD level sequence"),
			LOCTEXT("Regenerate_ToolTip", "Regenerates the transient LevelSequences based on the opened stage"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::ActionsRegenerate),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							if (UE::FUsdStage Stage = StageActor->GetUsdStage())
							{
								return true;
							}
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Stage Actor", LOCTEXT("Stage Actor", "Stage Actor"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("SelectStageActor", "Select stage actor"),
			LOCTEXT("SelectStageActor_ToolTip", "Selects the currently attached USD Stage Actor"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::ActionsSelectStageActor),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						return ViewModel.UsdStageActor.Get() != nullptr;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenLevelSequence", "Open level sequence"),
			LOCTEXT("OpenLevelSequence_ToolTip", "Opens the generated LevelSequence asset for this stage"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStage::ActionsOpenLevelSequence),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
						{
							return static_cast<bool>(StageActor->GetUsdStage()) && (StageActor->GetLevelSequence() != nullptr);
						}

						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();
}

void SUsdStage::FillOptionsMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(TEXT("StageOptions"), LOCTEXT("StageOptions", "Stage options"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("State", "Stage state"),
			LOCTEXT("State_ToolTip", "Whether we should open the USD Stage or spawn assets, actors and components"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillStageStateSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("Payloads", "Payloads"),
			LOCTEXT("Payloads_ToolTip", "What to do with payloads when initially opening the stage"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillPayloadsSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("PurposesToLoad", "Purposes to load"),
			LOCTEXT("PurposesToLoad_ToolTip", "Only load prims with these specific purposes from the USD stage"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillPurposesToLoadSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("RenderContext", "Render context"),
			LOCTEXT("RenderContext_ToolTip", "Choose which render context to use when parsing materials"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillRenderContextSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("MaterialPurpose", "Material purpose"),
			LOCTEXT("MaterialPurpose_ToolTip", "Material purpose to use when parsing material bindings in addition to the \"allPurpose\" fallback"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillMaterialPurposeSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("RootMotionHandling", "Root motion handling"),
			LOCTEXT("RootMotionHandling_ToolTip", "Choose how to handle root motion for generated AnimSequences and LevelSequences"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillRootMotionSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("FallbackCollisionType", "Fallback collision type"),
			LOCTEXT(
				"FallbackCollisionType_ToolTip",
				"What type of collision to use for static meshes generated from Prims that don't have physics schemas applied"
			),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillFallbackCollisionTypeMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("SubdivLevel", "Subdivision level"),
			LOCTEXT("SubdivLevel_ToolTip", "Subdivision level to use for all subdivision meshes on the opened stage. 0 means 'don't subdivide'"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillSubdivisionLevelSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("Metadata", "Metadata"),
			LOCTEXT("Metadata_ToolTip", "How to collect and handle metadata from USD prims"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillMetadataSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("Collapsing", "Collapsing"),
			LOCTEXT(
				"Collapsing_ToolTip",
				"Whether to try to combine individual assets and components of the same type on a Kind-per-Kind basis, like multiple Mesh prims into "
				"a single Static Mesh"
			),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillCollapsingSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("AssetReuse", "Asset sharing"),
			LOCTEXT("AssetReuse_ToolTip", "How to behave when generating assets from identical prims"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillShareAssetsSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("InterpolationType", "Interpolation type"),
			LOCTEXT("InterpolationType_ToolTip", "Whether to interpolate between time samples linearly or with 'held' (i.e. constant) interpolation"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillInterpolationTypeSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("GeometryCacheImport", "Geometry Cache Import"),
			LOCTEXT("GeometryCacheImport_ToolTip", "When to convert geometry caches to persistent assets (in stage workflow)"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillGeometryCacheImportSubMenu)
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("EditorOptions"), LOCTEXT("EditorOptions", "Editor options"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("SelectionText", "Selection"),
			LOCTEXT("SelectionText_ToolTip", "How the selection of prims, actors and components should behave"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillSelectionSubMenu)
		);

		MenuBuilder.AddSubMenu(
			LOCTEXT("NaniteSettings", "Nanite"),
			LOCTEXT("NaniteSettings_ToolTip", "Configure how to use Nanite for generated assets"),
			FNewMenuDelegate::CreateSP(this, &SUsdStage::FillNaniteThresholdSubMenu),
			false,
			FSlateIcon(),
			false
		);
	}
	MenuBuilder.EndSection();
}

void SUsdStage::FillExportSubMenu(FMenuBuilder& MenuBuilder)
{
	const FString FilePath;

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ExportAll", "All layers..."),
		LOCTEXT("ExportAll_ToolTip", "Exports copies of all file-based layers in the stage's layer stack to a new folder"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					FileExportAllLayers();
				}
			),
			FCanExecuteAction::CreateLambda(
				[this]()
				{
					if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
					{
						if (UE::FUsdStage Stage = StageActor->GetUsdStage())
						{
							return true;
						}
					}

					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ExportFlattened", "Flattened stage..."),
		LOCTEXT("ExportFlattened_ToolTip", "Flattens the current stage to a single USD layer and exports it as a new USD file"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					FileExportFlattenedStage();
				}
			),
			FCanExecuteAction::CreateLambda(
				[this]()
				{
					if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
					{
						if (UE::FUsdStage Stage = StageActor->GetUsdStage())
						{
							return true;
						}
					}

					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ExportStack", "Flattened layer stack..."),
		LOCTEXT("ExportStack_ToolTip", "Flattens the current stage's local layer stack to a single layer, preserving most composition arcs"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					FileExportFlattenedLayerStack();
				}
			),
			FCanExecuteAction::CreateLambda(
				[this]()
				{
					if (const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
					{
						if (UE::FUsdStage Stage = StageActor->GetUsdStage())
						{
							return true;
						}
					}

					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);
}

void SUsdStage::FillStageStateSubMenu(FMenuBuilder& MenuBuilder)
{
	TFunction<void(EUsdStageState, const FText&, const FText&)> AddStageStateEntry =
		[this, &MenuBuilder](EUsdStageState NewState, const FText& EntryText, const FText& EntryToolTipText)
	{
		MenuBuilder.AddMenuEntry(
			EntryText,
			EntryToolTipText,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, NewState, EntryText]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT("ChangedStageStateTransaction", "Set USD stage actor '{0}' actor to stage state '{1}'"),
								FText::FromString(StageActor->GetActorLabel()),
								EntryText
							));

							// c.f. comment in SUsdStage::FillCollapsingSubMenu
							TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

							StageActor->SetStageState(NewState);
							if (StageActor->IsTemplate())
							{
								StageActor->SaveConfig();
							}
						}
					}
				),
				FCanExecuteAction{},
				FIsActionChecked::CreateLambda(
					[this, NewState]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							return StageActor->StageState == NewState;
						}
						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	};

	AddStageStateEntry(
		EUsdStageState::Closed,
		LOCTEXT("Closed", "Closed"),
		LOCTEXT("Closed_ToolTip", "Don't open or load the USD Stage pointed to by RootLayer")
	);
	AddStageStateEntry(
		EUsdStageState::Opened,
		LOCTEXT("Opened", "Opened"),
		LOCTEXT("Opened_ToolTip", "Open the USD Stage pointed to by RootLayer, but don't spawn assets, actors or components")
	);
	AddStageStateEntry(
		EUsdStageState::OpenedAndLoaded,
		LOCTEXT("OpenedAndLoaded", "Opened and loaded"),
		LOCTEXT("OpenedAndLoaded_ToolTip", "Open the USD Stage pointed to by RootLayer and spawn assets, actors or components")
	);
}

void SUsdStage::FillPayloadsSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("LoadAll", "Load all"),
		LOCTEXT("LoadAll_ToolTip", "Loads all payloads, and make new payloads start out loaded"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetLoadAllTransaction", "Set USD stage actor '{0}' actor to load all payloads"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						ViewModel.SetLoadAllRule();

						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						if (!StageActor->GetUsdStage() || ViewModel.HasDefaultLoadRule())
						{
							return StageActor->InitialLoadSet == EUsdInitialLoadSet::LoadAll;
						}
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("LoadNone", "Load none"),
		LOCTEXT("LoadNone_ToolTip", "Unload all payloads, and make new payloads start out unloaded"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetLoadNoneTransaction", "Set USD stage actor '{0}' actor to load no payloads"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						ViewModel.SetLoadNoneRule();

						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						if (!StageActor->GetUsdStage() || ViewModel.HasDefaultLoadRule())
						{
							return StageActor->InitialLoadSet == EUsdInitialLoadSet::LoadNone;
						}
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SUsdStage::FillPurposesToLoadSubMenu(FMenuBuilder& MenuBuilder)
{
	auto AddPurposeEntry = [&](const EUsdPurpose& Purpose, const FText& Text)
	{
		MenuBuilder.AddMenuEntry(
			Text,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, Purpose]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT("PurposesToLoadTransaction", "Change purposes to load for USD stage actor '{0}'"),
								FText::FromString(StageActor->GetActorLabel())
							));

							// c.f. comment in SUsdStage::FillCollapsingSubMenu
							TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

							StageActor->SetPurposesToLoad((int32)((EUsdPurpose)StageActor->PurposesToLoad ^ Purpose));
							if (StageActor->IsTemplate())
							{
								StageActor->SaveConfig();
							}
						}
					}
				),
				FCanExecuteAction{},
				FIsActionChecked::CreateLambda(
					[this, Purpose]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							return EnumHasAllFlags((EUsdPurpose)StageActor->PurposesToLoad, Purpose);
						}
						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	};

	AddPurposeEntry(EUsdPurpose::Proxy, LOCTEXT("ProxyPurpose", "Proxy"));
	AddPurposeEntry(EUsdPurpose::Render, LOCTEXT("RenderPurpose", "Render"));
	AddPurposeEntry(EUsdPurpose::Guide, LOCTEXT("GuidePurpose", "Guide"));
}

void SUsdStage::FillRenderContextSubMenu(FMenuBuilder& MenuBuilder)
{
	auto AddRenderContextEntry = [&](const FName& RenderContextName)
	{
		FText RenderContextText = FText::FromName(RenderContextName);
		if (RenderContextName == UnrealIdentifiers::UniversalRenderContext)
		{
			RenderContextText = FText::FromString(UnrealIdentifiers::UniversalRenderContextDisplayString);
		}

		MenuBuilder.AddMenuEntry(
			RenderContextText,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, RenderContextName]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT("RenderContextToLoadTransaction", "Change render context to load for USD stage actor '{0}'"),
								FText::FromString(StageActor->GetActorLabel())
							));

							// c.f. comment in SUsdStage::FillCollapsingSubMenu
							TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

							StageActor->SetRenderContext(RenderContextName);
							if (StageActor->IsTemplate())
							{
								StageActor->SaveConfig();
							}
						}
					}
				),
				FCanExecuteAction{},
				FIsActionChecked::CreateLambda(
					[this, RenderContextName]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							return StageActor->RenderContext == RenderContextName;
						}
						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	};

	for (const FName& RenderContext : UsdUnreal::MaterialUtils::GetRegisteredRenderContexts())
	{
		AddRenderContextEntry(RenderContext);
	}
}

void SUsdStage::FillMaterialPurposeSubMenu(FMenuBuilder& MenuBuilder)
{
	MaterialPurposes = {
		MakeShared<FString>(UnrealIdentifiers::MaterialAllPurpose),
		MakeShared<FString>(UnrealIdentifiers::MaterialPreviewPurpose),
		MakeShared<FString>(UnrealIdentifiers::MaterialFullPurpose)
	};

	// Add additional purposes from project settings
	if (const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>())
	{
		MaterialPurposes.Reserve(MaterialPurposes.Num() + ProjectSettings->AdditionalMaterialPurposes.Num());

		TSet<FString> ExistingEntries = {
			UnrealIdentifiers::MaterialAllPurpose,
			UnrealIdentifiers::MaterialPreviewPurpose,
			UnrealIdentifiers::MaterialFullPurpose
		};

		for (const FName& AdditionalPurpose : ProjectSettings->AdditionalMaterialPurposes)
		{
			FString AdditionalPurposeStr = AdditionalPurpose.ToString();

			if (!ExistingEntries.Contains(AdditionalPurposeStr))
			{
				ExistingEntries.Add(AdditionalPurposeStr);
				MaterialPurposes.AddUnique(MakeShared<FString>(AdditionalPurposeStr));
			}
		}
	}

	// clang-format off
	TSharedRef<SBox> Box = SNew(SBox)
	.Padding(FMargin(8.0f, 0.0f))
	.VAlign(VAlign_Center)
	[
		// We have to use TSharedPtr<FString> here as the combobox actually contains a list view.
		// Also, a regular SComboBox<FName> doesn't even call our OnGenerateWidget function in case the item is NAME_None,
		// and we do need that case for allPurpose.
		SNew(SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&MaterialPurposes)
		.OnGenerateWidget_Lambda([&](TSharedPtr<FString> Option)
		{
			TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
			if (Option)
			{
				Widget = SNew(STextBlock)
					.Text(FText::FromString((*Option) == UnrealIdentifiers::MaterialAllPurpose
						? UnrealIdentifiers::MaterialAllPurposeText
						: *Option
					))
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
			}

			return Widget.ToSharedRef();
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo)
		{
			if (ChosenOption)
			{
				if (AUsdStageActor* StageActor = GetStageActorOrCDO())
				{
					FScopedTransaction Transaction(FText::Format(
						LOCTEXT("ChangeMaterialPurpose", "Change selected material purpose for USD stage actor '{0}' to '{1}'"),
						FText::FromString(StageActor->GetActorLabel()),
						FText::FromString(ChosenOption.IsValid() ? *ChosenOption : TEXT(""))
					));

					// c.f. comment in SUsdStage::FillCollapsingSubMenu
					TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

					StageActor->SetMaterialPurpose(**ChosenOption);
					if (StageActor->IsTemplate())
					{
						StageActor->SaveConfig();
					}
				}
			}
		})
		[
			SNew(SEditableTextBox)
			.Text_Lambda([this]() -> FText
			{
				if (AUsdStageActor* StageActor = GetStageActorOrCDO())
				{
					return FText::FromString(StageActor->MaterialPurpose == *UnrealIdentifiers::MaterialAllPurpose
						? UnrealIdentifiers::MaterialAllPurposeText
						: StageActor->MaterialPurpose.ToString()
					);
				}

				return FText::GetEmpty();
			})
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
			{
				if (CommitType != ETextCommit::OnEnter)
				{
					return;
				}

				FString NewPurposeString = NewText.ToString();
				FName NewPurpose = *NewPurposeString;

				bool bIsNew = true;
				for (const TSharedPtr<FString>& Purpose : MaterialPurposes)
				{
					if (Purpose && *Purpose == NewPurposeString)
					{
						bIsNew = false;
						break;
					}
				}

				if (bIsNew)
				{
					if (UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>())
					{
						ProjectSettings->AdditionalMaterialPurposes.AddUnique(NewPurpose);
						ProjectSettings->SaveConfig();
					}
				}

				if (AUsdStageActor* StageActor = GetStageActorOrCDO())
				{
					FScopedTransaction Transaction(FText::Format(
						LOCTEXT("TypeMaterialPurpose", "Change selected material purpose for USD stage actor '{0}' to '{1}'"),
						FText::FromString(StageActor->GetActorLabel()),
						NewText
					));

					StageActor->SetMaterialPurpose(NewPurpose);
					if (StageActor->IsTemplate())
					{
						StageActor->SaveConfig();
					}
				}
			})
		]
	];
	// clang-format on

	const bool bNoIndent = true;
	MenuBuilder.AddWidget(Box, FText::GetEmpty(), bNoIndent);
}

void SUsdStage::FillRootMotionSubMenu(FMenuBuilder& MenuBuilder)
{
	auto AddRootMotionEntry = [&](const EUsdRootMotionHandling& HandlingStrategy, const FText& Text)
	{
		UEnum* Enum = StaticEnum<EUsdRootMotionHandling>();
		FText ToolTip = Enum->GetToolTipTextByIndex(Enum->GetIndexByValue(static_cast<uint8>(HandlingStrategy)));

		MenuBuilder.AddMenuEntry(
			Text,
			ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, HandlingStrategy]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT("RootMotionHandlingTransaction", "Change root motion handling strategy for USD stage actor '{0}'"),
								FText::FromString(StageActor->GetActorLabel())
							));

							// c.f. comment in SUsdStage::FillCollapsingSubMenu
							TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

							StageActor->SetRootMotionHandling(HandlingStrategy);
							if (StageActor->IsTemplate())
							{
								StageActor->SaveConfig();
							}
						}
					}
				),
				FCanExecuteAction{},
				FIsActionChecked::CreateLambda(
					[this, HandlingStrategy]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							return StageActor->RootMotionHandling == HandlingStrategy;
						}
						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	};

	AddRootMotionEntry(EUsdRootMotionHandling::NoAdditionalRootMotion, LOCTEXT("NoAdditionalRootMotionText", "No additional root motion"));
	AddRootMotionEntry(EUsdRootMotionHandling::UseMotionFromSkelRoot, LOCTEXT("UseMotionFromSkelRootText", "Use motion from SkelRoot"));
	AddRootMotionEntry(EUsdRootMotionHandling::UseMotionFromSkeleton, LOCTEXT("UseMotionFromSkeletonText", "Use motion from Skeleton"));
}

void SUsdStage::FillFallbackCollisionTypeMenu(FMenuBuilder& MenuBuilder)
{
	auto AddFallbackCollisionTypeEntry = [&](const EUsdCollisionType& NewFallbackType, const FText& Text)
	{
		UEnum* Enum = StaticEnum<EUsdCollisionType>();
		FText ToolTip = Enum->GetToolTipTextByIndex(Enum->GetIndexByValue(static_cast<uint8>(NewFallbackType)));

		MenuBuilder.AddMenuEntry(
			Text,
			ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, NewFallbackType]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT("FallbackCollisionTypeTransaction", "Change fallback collision type for USD stage actor '{0}'"),
								FText::FromString(StageActor->GetActorLabel())
							));

							// c.f. comment in SUsdStage::FillCollapsingSubMenu
							TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

							StageActor->SetFallbackCollisionType(NewFallbackType);
							if (StageActor->IsTemplate())
							{
								StageActor->SaveConfig();
							}
						}
					}
				),
				FCanExecuteAction{},
				FIsActionChecked::CreateLambda(
					[this, NewFallbackType]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							return StageActor->FallbackCollisionType == NewFallbackType;
						}
						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	};

	AddFallbackCollisionTypeEntry(EUsdCollisionType::None, LOCTEXT("CollisionNone", "None"));
	AddFallbackCollisionTypeEntry(EUsdCollisionType::ConvexDecomposition, LOCTEXT("CollisionConvexDecomposition", "Convex decomposition"));
	AddFallbackCollisionTypeEntry(EUsdCollisionType::ConvexHull, LOCTEXT("CollisionConvexHull", "Convex hull"));
	AddFallbackCollisionTypeEntry(EUsdCollisionType::Sphere, LOCTEXT("CollisionSphere", "Sphere"));
	AddFallbackCollisionTypeEntry(EUsdCollisionType::Cube, LOCTEXT("CollisionCube", "Cube"));
	AddFallbackCollisionTypeEntry(EUsdCollisionType::MeshSimplification, LOCTEXT("CollisionMeshSimplification", "Mesh simplification"));
	AddFallbackCollisionTypeEntry(EUsdCollisionType::Capsule, LOCTEXT("CollisionCapsule", "Capsule"));
}

void SUsdStage::FillSubdivisionLevelSubMenu(FMenuBuilder& MenuBuilder)
{
	if (AUsdStageActor* StageActor = GetStageActorOrCDO())
	{
		CurrentSubdivisionLevel = StageActor->SubdivisionLevel;
	}

	static IConsoleVariable* MaxSubdivCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.Subdiv.MaxSubdivLevel"));
	const int32 MaxSubdivLevel = MaxSubdivCvar ? MaxSubdivCvar->GetInt() : 0;

	// clang-format off
	TSharedRef<SSpinBox<int32>> Slider = SNew(SSpinBox<int32>)
		.MinValue(0)
		.MaxValue(MaxSubdivLevel)
		.ToolTipText(LOCTEXT(
			"SubdivLevelSliderTooltip",
			"Subdivision level to use for all subdivision meshes on the opened stage. 0 means 'don't subdivide'. The maximum level of subdivision allowed can be configured via the 'USD.Subdiv.MaxSubdivLevel' cvar."
		))
		.Value(this, &SUsdStage::GetSubdivisionLevelValue)
		.OnValueChanged(this, &SUsdStage::OnSubdivisionLevelValueChanged)
		.SupportDynamicSliderMaxValue(true)
		.OnValueCommitted(this, &SUsdStage::OnSubdivisionLevelValueCommitted);
	// clang-format on

	const bool bNoIndent = true;
	MenuBuilder.AddWidget(Slider, FText::FromString(TEXT("Subdivision level: ")), bNoIndent);
}

void SUsdStage::FillMetadataSubMenu(FMenuBuilder& MenuBuilder)
{
	AUsdStageActor* StageActor = GetStageActorOrCDO();
	if (!StageActor)
	{
		return;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ColumnWidth = 0.5;
	TSharedRef<IDetailsView> DetailView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateLambda(
		[](const FPropertyAndParent& PropertyAndParent) -> bool
		{
			static const FName MetadataOptionsName = GET_MEMBER_NAME_CHECKED(AUsdStageActor, MetadataOptions);

			const FName PropertyName = PropertyAndParent.Property.GetFName();
			if (PropertyName == MetadataOptionsName)
			{
				return true;
			}

			for (const FProperty* ParentProp : PropertyAndParent.ParentProperties)
			{
				const FName ParentPropertyName = ParentProp->GetFName();
				if (ParentPropertyName == MetadataOptionsName)
				{
					return true;
				}
			}

			return false;
		}
	));
	DetailView->SetIsCustomRowVisibleDelegate(FIsCustomRowVisible::CreateLambda(
		[](FName RowName, FName ParentName) -> bool
		{
			static const FName FilterPropertyName = GET_MEMBER_NAME_CHECKED(FUsdMetadataImportOptions, BlockedPrefixFilters);
			return RowName == FilterPropertyName;
		}
	));
	DetailView->SetObject(StageActor);

	DetailView->OnFinishedChangingProperties().AddLambda(
		[this](const FPropertyChangedEvent& PropertyChangedEvent)
		{
			if (AUsdStageActor* StageActor = GetStageActorOrCDO())
			{
				if (StageActor->IsTemplate())
				{
					StageActor->SaveConfig();
				}
			}
		}
	);

	// Override the widget with some extra width because the prefix filters need some space to be used
	// comfortably
	// clang-format off
	TSharedRef<SBox> Box = SNew(SBox)
	.Padding(FMargin(0.0f, 0.0f))
	.VAlign(VAlign_Fill)
	.HAlign(HAlign_Fill)
	.WidthOverride(500)
	[
		DetailView
	];
	// clang-format on

	const bool bNoIndent = true;
	MenuBuilder.AddWidget(Box, FText::GetEmpty(), bNoIndent);
}

void SUsdStage::FillCollapsingSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("MergeIdenticalMaterialSlots", "Merge identical material slots"),
		LOCTEXT(
			"MergeIdenticalMaterialSlots_ToolTip",
			"If enabled, when multiple mesh prims are collapsed into a single static mesh, identical material slots are merged into one "
			"slot.\nOtherwise, material slots are simply appended to the list."
		),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("MergeIdenticalMaterialSlotsTransaction", "Toggle bMergeIdenticalMaterialSlots on USD stage actor '{0}'"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment within AddKindToCollapseEntry just below
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetMergeIdenticalMaterialSlots(!StageActor->bMergeIdenticalMaterialSlots);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->bMergeIdenticalMaterialSlots;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("UsePrimKindsForCollapsing", "Use prim kinds for collapsing"),
		LOCTEXT(
			"UsePrimKindsForCollapsing_ToolTip",
			"Use KindsToCollapse to determine when to collapse prim subtrees or not (defaults to enabled).\nDisable this if you want to prevent collapsing, or to control it manually by right-clicking on individual prims."
		),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("UsePrimKindsForCollapsingTransaction", "Toggle bUsePrimKindsForCollapsing on USD stage actor '{0}'"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment within AddKindToCollapseEntry just below
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetUsePrimKindsForCollapsing(!StageActor->bUsePrimKindsForCollapsing);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->bUsePrimKindsForCollapsing;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	auto AddKindToCollapseEntry = [&](const EUsdDefaultKind Kind, const FText& Text, FCanExecuteAction CanExecuteAction)
	{
		MenuBuilder.AddMenuEntry(
			Text,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this, Text, Kind]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							FScopedTransaction Transaction(FText::Format(
								LOCTEXT(
									"ToggleCollapsingTransaction",
									"Toggle asset and component collapsing for kind '{0}' on USD stage actor '{1}'"
								),
								Text,
								FText::FromString(StageActor->GetActorLabel())
							));

							// When we change our kinds to collapse, we'll end up loading the stage again and refreshing our selection.
							// We'll take care to try to re-select the same prims within SUsdStage::Refresh after the refresh is complete,
							// but if selection sync is on we'll also later on try updating our prim selection to match our component selection.
							// Not only is this "selection spam" is very visible on the USD Stage Editor, if our originally selected component
							// was just collapsed away, we'd end up updating our prim selection to point to the parent prim instead
							// (the collapsing root), which is not ideal.
							TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

							StageActor->SetKindsToCollapse((int32)((EUsdDefaultKind)StageActor->KindsToCollapse ^ Kind));
							if (StageActor->IsTemplate())
							{
								StageActor->SaveConfig();
							}
						}
					}
				),
				CanExecuteAction,
				FIsActionChecked::CreateLambda(
					[this, Kind]()
					{
						if (AUsdStageActor* StageActor = GetStageActorOrCDO())
						{
							return EnumHasAllFlags((EUsdDefaultKind)StageActor->KindsToCollapse, Kind);
						}
						return false;
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	};

	MenuBuilder.BeginSection("Kinds to collapse", LOCTEXT("KindsToCollapse", "Kinds to collapse"));

	AddKindToCollapseEntry(EUsdDefaultKind::Model, LOCTEXT("ModelKind", "Model"), FCanExecuteAction{});

	AddKindToCollapseEntry(
		EUsdDefaultKind::Component,
		LOCTEXT("ModelComponent", "   Component"),
		FCanExecuteAction::CreateLambda(
			[this]()
			{
				if (AUsdStageActor* StageActor = GetStageActorOrCDO())
				{
					// If we're collapsing all "model" kinds, the all "component"s should be collapsed anyway
					if (!EnumHasAnyFlags((EUsdDefaultKind)StageActor->KindsToCollapse, EUsdDefaultKind::Model))
					{
						return true;
					}
				}

				return false;
			}
		)
	);

	AddKindToCollapseEntry(
		EUsdDefaultKind::Group,
		LOCTEXT("ModelGroup", "   Group"),
		FCanExecuteAction::CreateLambda(
			[this]()
			{
				if (AUsdStageActor* StageActor = GetStageActorOrCDO())
				{
					if (!EnumHasAnyFlags((EUsdDefaultKind)StageActor->KindsToCollapse, EUsdDefaultKind::Model))
					{
						return true;
					}
				}

				return false;
			}
		)
	);

	AddKindToCollapseEntry(
		EUsdDefaultKind::Assembly,
		LOCTEXT("ModelAssembly", "      Assembly"),
		FCanExecuteAction::CreateLambda(
			[this]()
			{
				if (AUsdStageActor* StageActor = GetStageActorOrCDO())
				{
					if (!EnumHasAnyFlags((EUsdDefaultKind)StageActor->KindsToCollapse, EUsdDefaultKind::Model)
						&& !EnumHasAnyFlags((EUsdDefaultKind)StageActor->KindsToCollapse, EUsdDefaultKind::Group))
					{
						return true;
					}
				}

				return false;
			}
		)
	);

	AddKindToCollapseEntry(EUsdDefaultKind::Subcomponent, LOCTEXT("ModelSubcomponent", "Subcomponent"), FCanExecuteAction{});

	MenuBuilder.EndSection();
}

void SUsdStage::FillShareAssetsSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShareAssetsForIdenticalPrims_Text", "Share Assets for Identical Prims"),
		LOCTEXT(
			"bShareAssetsForIdenticalPrims_ToolTip",
			"If true, whenever two prims would have generated identical UAssets (like identical StaticMeshes or materials) then only one instance of "
			"that asset is generated, and the asset is shared by the components generated for both prims. If false, we will generate a dedicated "
			"asset for each prim."
		),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("ShareAssetsForIdenticalPrims_Transaction", "Toggle bShareAssetsForIdenticalPrims on USD stage actor '{0}'"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment within AddKindToCollapseEntry just below
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetShareAssetsForIdenticalPrims(!StageActor->bShareAssetsForIdenticalPrims);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->bShareAssetsForIdenticalPrims;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
}

void SUsdStage::FillInterpolationTypeSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("LinearType", "Linear"),
		LOCTEXT("LinearType_ToolTip", "Attribute values are linearly interpolated between authored values"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetLinearInterpolationType", "Set USD stage actor '{0}' to linear interpolation"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetInterpolationType(EUsdInterpolationType::Linear);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->InterpolationType == EUsdInterpolationType::Linear;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HeldType", "Held"),
		LOCTEXT(
			"HeldType_ToolTip",
			"Attribute values are held constant between authored values. An attribute's value will be equal to the nearest preceding authored value. "
			"If there is no preceding authored value, the value will be equal to the nearest subsequent value."
		),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetHeldInterpolationType", "Set USD stage actor '{0}' to held interpolation"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetInterpolationType(EUsdInterpolationType::Held);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->InterpolationType == EUsdInterpolationType::Held;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SUsdStage::FillGeometryCacheImportSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ImportNever", "Never"),
		LOCTEXT("ImportNever_ToolTip", "Geometry Caches are not imported as persistent assets and are always streamed from the stage instead."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetGeometryCacheImportNever", "Set USD stage actor '{0}' to not import geometry caches"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetGeometryCacheImport(EGeometryCacheImport::Never);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->GeometryCacheImport == EGeometryCacheImport::Never;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ImportOnLoad", "On Load"),
		LOCTEXT("ImportOnLoad_ToolTip", "Geometry Caches are imported as persistents assets on stage load and played back from them."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetGeometryCacheImportOnLoad", "Set USD stage actor '{0}' to import geometry caches on stage load"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetGeometryCacheImport(EGeometryCacheImport::OnLoad);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->GeometryCacheImport == EGeometryCacheImport::OnLoad;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ImportOnSave", "On Save"),
		LOCTEXT(
			"ImportOnSave_ToolTip",
			"Geometry Caches are streamed from the stage until they are saved at which time they will be imported as persistent assets "
			"and played back from them instead."
		),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						FScopedTransaction Transaction(FText::Format(
							LOCTEXT("SetGeometryCacheImportOnSave", "Set USD stage actor '{0}' to import geometry caches on save"),
							FText::FromString(StageActor->GetActorLabel())
						));

						// c.f. comment in SUsdStage::FillCollapsingSubMenu
						TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

						StageActor->SetGeometryCacheImport(EGeometryCacheImport::OnSave);
						if (StageActor->IsTemplate())
						{
							StageActor->SaveConfig();
						}
					}
				}
			),
			FCanExecuteAction{},
			FIsActionChecked::CreateLambda(
				[this]()
				{
					if (AUsdStageActor* StageActor = GetStageActorOrCDO())
					{
						return StageActor->GeometryCacheImport == EGeometryCacheImport::OnSave;
					}
					return false;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SUsdStage::FillSelectionSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SynchronizeText", "Synchronize with Editor"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda(
				[this]()
				{
					if (UUsdStageEditorSettings* Settings = GetMutableDefault<UUsdStageEditorSettings>())
					{
						Settings->bSelectionSynced = !Settings->bSelectionSynced;
						Settings->SaveConfig();
					}

					// Immediately sync the selection, but only if the results are obvious (i.e. either our prim selection or viewport selection are
					// empty)
					if (GEditor)
					{
						int32 NumPrimsSelected = UsdStageTreeView->GetNumItemsSelected();
						int32 NumViewportSelected = FMath::Max(GEditor->GetSelectedComponentCount(), GEditor->GetSelectedActorCount());

						if (NumPrimsSelected == 0)
						{
							USelection* Selection = GEditor->GetSelectedComponentCount() > 0 ? GEditor->GetSelectedComponents()
																							 : GEditor->GetSelectedActors();

							OnViewportSelectionChanged(Selection);
						}
						else if (NumViewportSelected == 0)
						{
							TGuardValue<bool> SelectionLoopGuard(bUpdatingViewportSelection, true);

							SUSDStageImpl::SelectGeneratedComponentsAndActors(
								ViewModel.UsdStageActor.Get(),
								UsdStageTreeView->GetSelectedPrimPaths()
							);
						}
					}
				}
			),
			FCanExecuteAction::CreateLambda(
				[]()
				{
					return true;
				}
			),
			FIsActionChecked::CreateLambda(
				[]()
				{
					const UUsdStageEditorSettings* Settings = GetDefault<UUsdStageEditorSettings>();
					return Settings && Settings->bSelectionSynced;
				}
			)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);
}

void SUsdStage::FillNaniteThresholdSubMenu(FMenuBuilder& MenuBuilder)
{
	if (AUsdStageActor* StageActor = GetStageActorOrCDO())
	{
		CurrentNaniteThreshold = StageActor->NaniteTriangleThreshold;
	}

	// clang-format off
	TSharedRef<SSpinBox<int32>> Slider = SNew(SSpinBox<int32>)
		.MinValue(0)
		.ToolTipText(LOCTEXT("TriangleThresholdTooltip", "Try enabling Nanite for static meshes that are generated with at least this many triangles"))
		.Value(this, &SUsdStage::GetNaniteTriangleThresholdValue)
		.OnValueChanged(this, &SUsdStage::OnNaniteTriangleThresholdValueChanged)
		.SupportDynamicSliderMaxValue(true)
		.OnValueCommitted(this, &SUsdStage::OnNaniteTriangleThresholdValueCommitted);
	// clang-format on

	const bool bNoIndent = true;
	MenuBuilder.AddWidget(Slider, FText::FromString(TEXT("Triangle threshold: ")), bNoIndent);
}

void SUsdStage::FileNew()
{
	ViewModel.NewStage();
}

void SUsdStage::FileOpen(const FString& FilePath)
{
	FString FilePathCopy = FilePath;
	if (FilePathCopy.IsEmpty())
	{
		TOptional<FString> UsdFilePath = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Open);
		if (!UsdFilePath.IsSet())
		{
			return;
		}

		FilePathCopy = UsdFilePath.GetValue();
	}

	OpenStage(*FilePathCopy);
}

void SUsdStage::FileSave(const FString& FilePath)
{
	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	UE::FUsdStage UsdStage = static_cast<const AUsdStageActor*>(StageActor)->GetUsdStage();
	if (!UsdStage)
	{
		return;
	}

	if (UE::FSdfLayer RootLayer = UsdStage.GetRootLayer())
	{
		FString RealPath = RootLayer.GetRealPath();
		if (FPaths::FileExists(RealPath))
		{
			ViewModel.SaveStage();
		}
		else
		{
			FString FilePathCopy = FilePath;
			if (FilePathCopy.IsEmpty())
			{
				TOptional<FString> UsdFilePath = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Save);
				if (!UsdFilePath.IsSet())
				{
					return;
				}

				FilePathCopy = UsdFilePath.GetValue();
			}

			ViewModel.SaveStageAs(*FilePathCopy);
		}
	}
}

void SUsdStage::FileExportAllLayers(const FString& OutputDirectory)
{
	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	UE::FUsdStage UsdStage = StageActor->GetUsdStage();
	if (!UsdStage)
	{
		return;
	}

	const bool bIncludeClipLayers = false;
	TArray<UE::FSdfLayer> LayerStack = UsdStage.GetUsedLayers(bIncludeClipLayers);
	if (LayerStack.Num() < 1)
	{
		return;
	}

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (!DesktopPlatform)
	{
		return;
	}

	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
								   ? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
								   : nullptr;

	FString OutputDirectoryCopy = OutputDirectory;
	if (OutputDirectoryCopy.IsEmpty())
	{
		if (!DesktopPlatform
				 ->OpenDirectoryDialog(ParentWindowHandle, LOCTEXT("ChooseFolder", "Choose output folder").ToString(), TEXT(""), OutputDirectoryCopy))
		{
			return;
		}

		OutputDirectoryCopy = FPaths::ConvertRelativePathToFull(OutputDirectoryCopy);
	}

	double StartTime = FPlatformTime::Cycles64();

	// Manually load any layer that is referenced by any of the stage's layers, but not currently loaded.
	// This can happen if e.g. an unselected variant appends a reference or has a payload.
	// We will need to actually load these as opposed to just copy-paste the files as we may need to update references/paths
	SUSDStageImpl::AppendAllExternalReferences(LayerStack);

	// Discard session layers, but keep the root layer if it's anonymous (temp stage)
	for (int32 Index = LayerStack.Num() - 1; Index >= 0; --Index)
	{
		UE::FSdfLayer& Layer = LayerStack[Index];
		if (Layer.IsAnonymous() && UsdStage.GetRootLayer() != Layer)
		{
			const int32 Count = 1;
			LayerStack.RemoveAt(Index, Count, EAllowShrinking::No);
		}
	}

#if PLATFORM_LINUX
	TMap<FString, FString, FDefaultSetAllocator, UsdUtils::FCaseSensitiveStringMapFuncs<FString>> OldPathToSavedPath;
#else
	TMap<FString, FString> OldPathToSavedPath;
#endif

	// If the stage has layers referencing each other, we will need to remap those to point exclusively at the newly saved files
	TSet<FString> SavedPaths;
	for (const UE::FSdfLayer& Layer : LayerStack)
	{
		FString LayerPath = Layer.GetRealPath();
		FPaths::NormalizeFilename(LayerPath);

		FString TargetPath = FPaths::Combine(OutputDirectoryCopy, Layer.GetDisplayName());
		FPaths::NormalizeFilename(TargetPath);

		// Filename collision (should be rare, but possible given that we're discarding the folder structure)
		if (SavedPaths.Contains(TargetPath))
		{
			FString TargetPathNoExt = FPaths::SetExtension(TargetPath, TEXT(""));
			FString Ext = FPaths::GetExtension(TargetPath);
			int32 Suffix = 0;

			do
			{
				TargetPath = FString::Printf(TEXT("%s_%d.%s"), *TargetPathNoExt, Suffix++, *Ext);
			} while (SavedPaths.Contains(TargetPath));
		}

		OldPathToSavedPath.Add(LayerPath, TargetPath);
		SavedPaths.Add(TargetPath);
	}

	// Actually save the output layers
	for (UE::FSdfLayer& Layer : LayerStack)
	{
		FString LayerPath = Layer.GetRealPath();
		FPaths::NormalizeFilename(LayerPath);

		if (FString* TargetLayerPath = OldPathToSavedPath.Find(LayerPath))
		{
			// Clone the layer so that we don't modify the currently opened stage
			UE::FSdfLayer OutputLayer = UE::FSdfLayer::CreateNew(**TargetLayerPath);
			OutputLayer.TransferContent(Layer);

			// Update references to assets (e.g. textures) so that they're absolute and also work from the new file
			UsdUtils::ConvertAssetRelativePathsToAbsolute(OutputLayer, Layer);

			// Remap references to layers so that they point at the other newly saved files. Note that SUSDStageImpl::AppendAllExternalReferences
			// will have collected all external references already, so OldPathToSavedPath should have entries for all references we'll find
			for (const FString& AssetDependency : OutputLayer.GetCompositionAssetDependencies())
			{
				FString AbsRef = FPaths::ConvertRelativePathToFull(FPaths::GetPath(LayerPath), AssetDependency);	// Relative to the original file
				FPaths::NormalizeFilename(AbsRef);

				if (FString* SavedReference = OldPathToSavedPath.Find(AbsRef))
				{
					OutputLayer.UpdateCompositionAssetDependency(*AssetDependency, **SavedReference);
				}
			}

			bool bForce = true;
			OutputLayer.Save(bForce);
		}
	}

	// Send analytics
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;

		EventAttributes.Emplace(TEXT("NumLayersExported"), LayerStack.Num());

		bool bAutomated = false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(UsdStage.GetRootLayer().GetDisplayName());
		IUsdClassesModule::SendAnalytics(
			MoveTemp(EventAttributes),
			TEXT("ExportStageLayers"),
			bAutomated,
			ElapsedSeconds,
			UsdUtils::GetUsdStageNumFrames(UsdStage),
			Extension
		);
	}
}

void SUsdStage::FileExportFlattenedStage(const FString& OutputLayer)
{
	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	UE::FUsdStage UsdStage = StageActor->GetUsdStage();
	if (!UsdStage)
	{
		return;
	}

	FString OutputLayerCopy = OutputLayer;
	if (OutputLayerCopy.IsEmpty())
	{
		TOptional<FString> UsdFilePath = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Save);
		if (!UsdFilePath.IsSet())
		{
			return;
		}

		OutputLayerCopy = UsdFilePath.GetValue();
	}

	double StartTime = FPlatformTime::Cycles64();

	const bool bResult = UsdStage.Export(*OutputLayerCopy);
	if (!bResult)
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("FailedToExportFlattened", "Failed to export flattened USD Stage to path '{0}'!"),
			FText::FromString(OutputLayerCopy)
		));
	}

	// Send analytics
	if (FEngineAnalytics::IsAvailable())
	{
		bool bAutomated = false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(OutputLayerCopy);
		IUsdClassesModule::SendAnalytics(
			{},
			TEXT("ExportStageFlattened"),
			bAutomated,
			ElapsedSeconds,
			UsdUtils::GetUsdStageNumFrames(UsdStage),
			Extension
		);
	}
}

void SUsdStage::FileExportFlattenedLayerStack(const FString& OutputLayer)
{
	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	UE::FUsdStage UsdStage = StageActor->GetUsdStage();
	if (!UsdStage)
	{
		return;
	}

	FString OutputLayerCopy = OutputLayer;
	if (OutputLayerCopy.IsEmpty())
	{
		TOptional<FString> UsdFilePath = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Save);
		if (!UsdFilePath.IsSet())
		{
			return;
		}

		OutputLayerCopy = UsdFilePath.GetValue();
	}

	double StartTime = FPlatformTime::Cycles64();

	FScopedUsdMessageLog ScopedLog;

	UE::FSdfLayer FlattenedLayer = UsdUtils::FlattenLayerStack(UsdStage);
	if (FUsdLogManager::HasAccumulatedErrors())
	{
		return;
	}

	const bool bResult = FlattenedLayer.Export(*OutputLayerCopy);
	if (!bResult)
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("FailedToExportFlattenedLayerStack", "Failed to export flattened layer stack to path '{0}'!"),
			FText::FromString(OutputLayerCopy)
		));
	}

	// Send analytics
	if (FEngineAnalytics::IsAvailable())
	{
		bool bAutomated = false;
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		FString Extension = FPaths::GetExtension(OutputLayerCopy);
		IUsdClassesModule::SendAnalytics(
			{},
			TEXT("ExportStageFlattenedLayerStack"),
			bAutomated,
			ElapsedSeconds,
			UsdUtils::GetUsdStageNumFrames(UsdStage),
			Extension
		);
	}
}

void SUsdStage::FileReload()
{
	AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	static IConsoleVariable* Cvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.DiscardUndoBufferOnStageOpenClose"));
	const bool bDiscardUndoBufferOnStageOpenClose = Cvar && Cvar->GetBool();

	FScopedTransaction Transaction(LOCTEXT("ReloadTransaction", "Reload USD stage"));

	TOptional<TGuardValue<ITransaction*>> SuppressTransaction;
	if (bDiscardUndoBufferOnStageOpenClose)
	{
		SuppressTransaction.Emplace(GUndo, nullptr);
		StageActor->RequestDelayedTransactorReset();
	}

	ViewModel.ReloadStage();

	if (UsdLayersTreeView)
	{
		const bool bResync = true;
		UsdLayersTreeView->Refresh(StageActor->GetBaseUsdStage(), StageActor->GetIsolatedUsdStage(), bResync);
	}
}

void SUsdStage::FileReset()
{
	ViewModel.ResetStage();

	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();

	if (UsdLayersTreeView && StageActor)
	{
		const bool bResync = true;
		UsdLayersTreeView->Refresh(StageActor->GetBaseUsdStage(), StageActor->GetIsolatedUsdStage(), bResync);
	}
}

void SUsdStage::FileClose()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SUsdStage::FileClose);

	FScopedTransaction Transaction{LOCTEXT("CloseTransaction", "Close USD stage")};

	ViewModel.CloseStage();
	RequestFullRefresh();
}

void SUsdStage::OnLayerIsolated(const UE::FSdfLayer& IsolatedLayer)
{
	FScopedTransaction Transaction(LOCTEXT("StopIsolatingButtonTransaction", "Changed layer isolation"));

	if (AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
	{
		StageActor->IsolateLayer(IsolatedLayer);

		RequestFullRefresh();
	}
}

void SUsdStage::ActionsImportWithDialog()
{
	ViewModel.ImportStage();
}

void SUsdStage::ActionsImport(const FString& OutputContentFolder, UUsdStageImportOptions* Options)
{
	ViewModel.ImportStage(*OutputContentFolder, Options);
}

void SUsdStage::ActionsRegenerate()
{
	FScopedTransaction Transaction(LOCTEXT("RegenerateTransaction", "Regenerate USD level sequence"));

	if (AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get())
	{
		StageActor->RepopulateLevelSequence();
	}
}

void SUsdStage::ActionsSelectStageActor()
{
	AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	// Don't use GEditor->SelectNone() as that will affect *every type of selection in the editor*,
	// including even some UI menus, brushes, etc.
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		SelectedActors->DeselectAll(AActor::StaticClass());
	}
	if (USelection* SelectedComponents = GEditor->GetSelectedComponents())
	{
		SelectedComponents->DeselectAll(UActorComponent::StaticClass());
	}

	const bool bSelected = true;
	const bool bNotifySelectionChanged = true;
	GEditor->SelectActor(StageActor, bSelected, bNotifySelectionChanged);
}

void SUsdStage::ActionsOpenLevelSequence()
{
	AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	ULevelSequence* LevelSequence = StageActor->GetLevelSequence();
	if (!LevelSequence)
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
	AssetEditorSubsystem->OpenEditorForAsset(LevelSequence);
}

void SUsdStage::ExportSelectedLayers(const FString& OutputLayerOrDirectory)
{
	if (UsdLayersTreeView)
	{
		UsdLayersTreeView->ExportSelectedLayers(OutputLayerOrDirectory);
	}
}

void SUsdStage::OnPrimSelectionChanged(const TArray<FString>& PrimPaths)
{
	if (bUpdatingPrimSelection)
	{
		return;
	}

	AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	UE::FUsdStage UsdStage = static_cast<const AUsdStageActor*>(StageActor)->GetUsdStage();
	SelectedPrimPath = PrimPaths.Num() == 1 ? PrimPaths[0] : TEXT("");

	if (UsdPrimInfoWidget)
	{
		UsdPrimInfoWidget->SetPrimPath(UsdStage, *SelectedPrimPath);
	}

	// We don't need to check for actors and components in case we're just selecting the stage root
	// Note: This is also a temp fix that prevents an ensure: When we're switching to a new stage,
	// AUsdStageActor::LoadUsdStage calls AUsdStageActor::OpenUsdStage, which triggers OnStageChanged
	// and gets us here. Ideally this wouldn't happen at the same time as AUsdStageActor::LoadUsdStage
	// because we need the InfoCache to be complete at this point. Strangely though, the SlowTask.EnterProgressFrame
	// within AUsdStageActor::LoadUsdStage triggers a full slate tick right there, that calls us before
	// LoadUsdStage is complete... in the future I think we can fix this by changing our delegates from
	// AsyncTask(ENamedThreads::GameThread) to finishing up on the next tick instead, but for now
	// this will do
	const bool bIsRootSelection = PrimPaths.Num() == 1 && PrimPaths[0] == TEXT("/");

	const UUsdStageEditorSettings* Settings = GetDefault<UUsdStageEditorSettings>();
	if (Settings && Settings->bSelectionSynced && GEditor && !bIsRootSelection)
	{
		TGuardValue<bool> SelectionLoopGuard(bUpdatingViewportSelection, true);

		SUSDStageImpl::SelectGeneratedComponentsAndActors(StageActor, PrimPaths);
	}
}

void SUsdStage::OpenStage(const TCHAR* FilePath)
{
	// Create the transaction before calling UsdStageModule.GetUsdStageActor as that may create the actor, and we want
	// the actor spawning to be part of the transaction
	FScopedTransaction Transaction{FText::Format(LOCTEXT("OpenStageTransaction", "Open USD stage '{0}'"), FText::FromString(FilePath))};

	if (!ViewModel.UsdStageActor.IsValid())
	{
		IUsdStageModule& UsdStageModule = FModuleManager::Get().LoadModuleChecked<IUsdStageModule>("UsdStage");
		AttachToStageActor(&UsdStageModule.GetUsdStageActor(GWorld));
	}

	ViewModel.OpenStage(FilePath);
}

void SUsdStage::RequestLayersTreeViewRefresh()
{
	if (UsdLayersTreeView)
	{
		FWriteScopeLock LayersTreeViewLock{UsdLayersTreeView->RefreshStateLock};

		UsdLayersTreeView->bUpdateState = (SUsdLayersTreeView::ELayersTreeViewState
		)FMath::Max((uint8)SUsdLayersTreeView::ELayersTreeViewState::RefreshNeeded, (uint8)UsdLayersTreeView->bUpdateState);
	}
}

void SUsdStage::RequestFullRefresh()
{
	if (UsdLayersTreeView)
	{
		FWriteScopeLock LayersTreeViewLock{UsdLayersTreeView->RefreshStateLock};

		UsdLayersTreeView->bUpdateState = SUsdLayersTreeView::ELayersTreeViewState::ResyncNeeded;
	}

	if (UsdStageTreeView)
	{
		FWriteScopeLock StageTreeViewLock{UsdStageTreeView->RefreshStateLock};

		UsdStageTreeView->bNeedsFullUpdate = true;

		if (UsdStageTreeView->PrimPathToSelect.IsEmpty())
		{
			UsdStageTreeView->PrimPathToSelect = SelectedPrimPath;
		}
	}
}

void SUsdStage::OnSlateTick(float Time)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SUsdStage::OnSlateTick);

	if (bEditorIsShuttingDown)
	{
		return;
	}

	SUsdLayersTreeView::ELayersTreeViewState LayersTreeViewState = SUsdLayersTreeView::ELayersTreeViewState::NoRefreshNeeded;
	bool bStageTreeViewNeedsFullUpdate = false;
	FString PrimPathToSelect;
	TArray<TPair<FString, bool>> PrimsToRefresh;

	if (UsdLayersTreeView)
	{
		FWriteScopeLock LayersTreeViewLock{UsdLayersTreeView->RefreshStateLock};
		Swap(LayersTreeViewState, UsdLayersTreeView->bUpdateState);
	}

	if (UsdStageTreeView)
	{
		FWriteScopeLock StageTreeViewLock{UsdStageTreeView->RefreshStateLock};
		Swap(bStageTreeViewNeedsFullUpdate, UsdStageTreeView->bNeedsFullUpdate);
		Swap(PrimPathToSelect, UsdStageTreeView->PrimPathToSelect);
		Swap(PrimsToRefresh, UsdStageTreeView->PrimsToRefresh);
	}

	if (LayersTreeViewState != SUsdLayersTreeView::ELayersTreeViewState::NoRefreshNeeded)
	{
		// May be nullptr, but that is ok. Its how the widgets are reset
		AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();

		UE::FUsdStageWeak IsolatedStage = StageActor ? StageActor->GetIsolatedUsdStage() : UE::FUsdStage{};
		UE::FUsdStageWeak BaseStage = StageActor ? StageActor->GetBaseUsdStage() : UE::FUsdStage{};

		// The layers tree view always needs to receive both the full stage as well as the isolated
		const bool bResync = LayersTreeViewState == SUsdLayersTreeView::ELayersTreeViewState::ResyncNeeded;
		UsdLayersTreeView->Refresh(BaseStage, IsolatedStage, bResync);
	}

	if (bStageTreeViewNeedsFullUpdate || !PrimPathToSelect.IsEmpty() || PrimsToRefresh.Num() > 0)
	{
		// May be nullptr, but that is ok. Its how the widgets are reset
		AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();

		// Don't bother refreshing individual prims if we'll do a full refresh anyway
		if (!bStageTreeViewNeedsFullUpdate)
		{
			UsdUtils::FCaseSensitiveStringSet RefreshedPrims;
			UsdUtils::FCaseSensitiveStringSet ResyncedPrims;

			// Note how we go from oldest to newest update here, so after the loop we'll end up with LastResyncedPrim
			// pointing at what it should, and PrimPathToSelect pointing at the last updated prim path to select
			for (const TPair<FString, bool>& PrimAndResync : PrimsToRefresh)
			{
				const FString& PrimPath = PrimAndResync.Key;
				const bool bResync = PrimAndResync.Value;

				// Prevent from refreshing the same prim too many times on a single tick
				if (ResyncedPrims.Contains(PrimPath))
				{
					continue;
				}
				if (!bResync && RefreshedPrims.Contains(PrimPath))
				{
					continue;
				}
				if (bResync)
				{
					ResyncedPrims.Add(PrimPath);
				}
				else
				{
					RefreshedPrims.Add(PrimPath);
				}

				UsdStageTreeView->RefreshPrim(PrimPath, bResync);
				{
					// RefreshPrim may internally set UsdStageTreeView->bNeedsFullUpdate to true
					FWriteScopeLock StageTreeViewLock{UsdStageTreeView->RefreshStateLock};
					bStageTreeViewNeedsFullUpdate |= UsdStageTreeView->bNeedsFullUpdate;
					UsdStageTreeView->bNeedsFullUpdate = false;
				}

				// If we already have a PrimPathToSelect, we're going to already refresh the info panel already or end up
				// looking at another prim anyway, so don't bother refreshing the info widget due to a prim update
				if (PrimPathToSelect.IsEmpty())
				{
					const bool bViewingTheUpdatedPrim = SelectedPrimPath.Equals(PrimPath, ESearchCase::IgnoreCase);
					const bool bViewingStageProperties = SelectedPrimPath.IsEmpty() || SelectedPrimPath == TEXT("/");
					const bool bStageUpdated = PrimPath == TEXT("/");

					if (UsdPrimInfoWidget && StageActor && (bViewingTheUpdatedPrim || (bViewingStageProperties && bStageUpdated)))
					{
						PrimPathToSelect = PrimPath;
					}
				}
			}
		}

		if (bStageTreeViewNeedsFullUpdate)
		{
			UsdStageTreeView->Refresh(GetCurrentStage());
		}

		if (bStageTreeViewNeedsFullUpdate || !PrimPathToSelect.IsEmpty())
		{
			// Refresh will generate brand new RootItems, so let's immediately select the new item
			// that corresponds to the prim we're supposed to be selecting
			UsdStageTreeView->SetSelectedPrimPaths({PrimPathToSelect.IsEmpty() ? SelectedPrimPath : PrimPathToSelect});

			// If we changed/redid the selection, let's make sure that our info panel will update too (right below this)
			PrimPathToSelect = SelectedPrimPath;
		}

		// Refresh the actual slate widget if we changed any underlying data
		if (bStageTreeViewNeedsFullUpdate || PrimsToRefresh.Num() > 0)
		{
			UsdStageTreeView->RequestTreeRefresh();
		}
	}

	if (UsdPrimInfoWidget && (bStageTreeViewNeedsFullUpdate || !PrimPathToSelect.IsEmpty()))
	{
		UsdPrimInfoWidget->SetPrimPath(GetCurrentStage(), *PrimPathToSelect);
		PrimPathToSelect.Empty();
	}
}

void SUsdStage::OnViewportSelectionChanged(UObject* NewSelection)
{
	// This may be called when first opening a project, before the our widgets are fully initialized
	if (!UsdStageTreeView)
	{
		return;
	}

	const UUsdStageEditorSettings* Settings = GetDefault<UUsdStageEditorSettings>();
	if (!Settings || !Settings->bSelectionSynced || bUpdatingViewportSelection)
	{
		return;
	}

	AUsdStageActor* AttachedStageActor = ViewModel.UsdStageActor.Get();
	USelection* Selection = Cast<USelection>(NewSelection);
	if (!Selection || !AttachedStageActor)
	{
		return;
	}

	TGuardValue<bool> SelectionLoopGuard(bUpdatingPrimSelection, true);

	TArray<USceneComponent*> AllSelectedComponents;
	{
		TArray<AActor*> SelectedActors;
		Selection->GetSelectedObjects<AActor>(SelectedActors);

		for (AActor* SelectedActor : SelectedActors)
		{
			// Some actors like AWorldDataLayer don't have a root component
			if (SelectedActor && SelectedActor->GetRootComponent())
			{
				AllSelectedComponents.Add(SelectedActor->GetRootComponent());
			}
		}

		// Push the components last as we'll show info for the last selected component,
		// and we must first select the actor before we can select its individual components
		TArray<USceneComponent*> SelectedComponents;
		Selection->GetSelectedObjects<USceneComponent>(SelectedComponents);

		AllSelectedComponents.Append(SelectedComponents);
	}
	if (AllSelectedComponents.Num() == 0)
	{
		return;
	}

	// Find the exact stage actor that owns these components, if any
	AUsdStageActor* OwnerActor = nullptr;
	if (AllSelectedComponents.Num() > 0)
	{
		USceneComponent* AttachParent = AllSelectedComponents[AllSelectedComponents.Num() - 1]->GetAttachParent();
		while (AttachParent)
		{
			USceneComponent* NewAttachParent = AttachParent->GetAttachParent();
			if (!NewAttachParent)
			{
				break;
			}
			AttachParent = NewAttachParent;
		}

		if (AttachParent)
		{
			OwnerActor = Cast<AUsdStageActor>(AttachParent->GetOwner());
		}
	}
	if (!OwnerActor)
	{
		// Selected component is not something we spawned
		return;
	}

	TArray<FString> PrimPaths;
	for (USceneComponent* Component : AllSelectedComponents)
	{
		FString FoundPrimPath = OwnerActor->GetSourcePrimPath(Component);
		if (!FoundPrimPath.IsEmpty())
		{
			PrimPaths.Add(FoundPrimPath);
		}
	}

	if (OwnerActor && OwnerActor != AttachedStageActor)
	{
		AttachToStageActor(OwnerActor);
	}

	if (PrimPaths.Num() > 0)
	{
		UsdStageTreeView->SetSelectedPrimPaths(PrimPaths);

		if (UsdPrimInfoWidget)
		{
			UsdPrimInfoWidget->SetPrimPath(GetCurrentStage(), *PrimPaths[PrimPaths.Num() - 1]);
		}
	}
}

void SUsdStage::OnPostPIEStarted(bool bIsSimulating)
{
	AUsdStageActor* CurrentStageActor = ViewModel.UsdStageActor.Get();
	if (!CurrentStageActor)
	{
		return;
	}

	AUsdStageActor* PIEStageActor = Cast<AUsdStageActor>(EditorUtilities::GetSimWorldCounterpartActor(CurrentStageActor));
	if (!PIEStageActor)
	{
		return;
	}

	AttachToStageActor(PIEStageActor);
}

void SUsdStage::OnEndPIE(bool bIsSimulating)
{
	AUsdStageActor* CurrentStageActor = ViewModel.UsdStageActor.Get();
	if (!CurrentStageActor)
	{
		return;
	}

	AUsdStageActor* EditorStageActor = Cast<AUsdStageActor>(EditorUtilities::GetEditorWorldCounterpartActor(CurrentStageActor));
	if (!EditorStageActor)
	{
		return;
	}

	AttachToStageActor(EditorStageActor);
}

int32 SUsdStage::GetNaniteTriangleThresholdValue() const
{
	return CurrentNaniteThreshold;
}

void SUsdStage::OnNaniteTriangleThresholdValueChanged(int32 InValue)
{
	CurrentNaniteThreshold = InValue;
}

void SUsdStage::OnNaniteTriangleThresholdValueCommitted(int32 InValue, ETextCommit::Type InCommitType)
{
	AUsdStageActor* StageActor = GetStageActorOrCDO();
	if (!StageActor)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("NaniteTriangleThresholdCommittedTransaction", "Change Nanite triangle threshold for USD stage actor '{0}'"),
		FText::FromString(StageActor->GetActorLabel())
	));

	// c.f. comment in SUsdStage::FillCollapsingSubMenu
	TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

	StageActor->SetNaniteTriangleThreshold(InValue);
	CurrentNaniteThreshold = InValue;
	{
		StageActor->SaveConfig();
	}
}

int32 SUsdStage::GetSubdivisionLevelValue() const
{
	return CurrentSubdivisionLevel;
}

void SUsdStage::OnSubdivisionLevelValueChanged(int32 InValue)
{
	CurrentSubdivisionLevel = InValue;
}

void SUsdStage::OnSubdivisionLevelValueCommitted(int32 InValue, ETextCommit::Type InCommitType)
{
	AUsdStageActor* StageActor = GetStageActorOrCDO();
	if (!StageActor)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("SubdivisionLevelCommittedTransaction", "Change Subdivision level for USD stage actor '{0}'"),
		FText::FromString(StageActor->GetActorLabel())
	));

	// c.f. comment in SUsdStage::FillCollapsingSubMenu
	TGuardValue<bool> MaintainSelectionGuard(bUpdatingViewportSelection, true);

	StageActor->SetSubdivisionLevel(InValue);
	CurrentSubdivisionLevel = InValue;

	if (StageActor->IsTemplate())
	{
		StageActor->SaveConfig();
	}
}

UE::FUsdStageWeak SUsdStage::GetCurrentStage() const
{
	const AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (StageActor)
	{
		return StageActor->GetUsdStage();
	}

	return UE::FUsdStageWeak{};
}

AUsdStageActor* SUsdStage::GetStageActorOrCDO() const
{
	AUsdStageActor* StageActor = ViewModel.UsdStageActor.Get();
	if (!StageActor)
	{
		StageActor = GetMutableDefault<AUsdStageActor>();
	}
	ensure(StageActor);
	return StageActor;
}

#endif	  // #if USE_USD_SDK

#undef LOCTEXT_NAMESPACE
