// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDStageViewModel.h"
#include "UsdWrappers/ForwardDeclarations.h"

#include "Animation/CurveSequence.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

class AUsdStageActor;
class FLevelCollectionModel;
class FMenuBuilder;
class ISceneOutliner;
enum class EMapChangeType : uint8;
enum class EUsdInitialLoadSet : uint8;
struct FSlateBrush;
namespace UE
{
	class FUsdPrim;
	class FSdfPath;
	class FUsdAttribute;
}

#if USE_USD_SDK

class SUsdStage : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SUsdStage)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	virtual ~SUsdStage();

	// Attaches to a new stage actor.
	// When bFlashButton is true we will also blink the button on the UI to draw attention to the fact
	// that the actor changed
	void AttachToStageActor(AUsdStageActor* InUsdStageActor, bool bFlashButton = true);

	AUsdStageActor* GetAttachedStageActor() const;

	TArray<UE::FSdfLayer> GetSelectedLayers() const;
	void SetSelectedLayers(const TArray<UE::FSdfLayer>& NewSelection) const;

	TArray<UE::FUsdPrim> GetSelectedPrims() const;
	void SetSelectedPrims(const TArray<UE::FUsdPrim>& NewSelection) const;

	TArray<FString> GetSelectedPropertyNames() const;
	void SetSelectedPropertyNames(const TArray<FString>& NewSelection);

	TArray<FString> GetSelectedPropertyMetadataNames() const;
	void SetSelectedPropertyMetadataNames(const TArray<FString>& NewSelection);

	// Main menu actions.
	// For all of these, providing an empty path will cause us to pop open a dialog to let the user pick the path
	// instead.
	void FileNew();
	void FileOpen(const FString& FilePath = {});
	void FileSave(const FString& OutputFileIfUnsaved = {});
	void FileExportAllLayers(const FString& OutputDirectory = {});
	void FileExportFlattenedStage(const FString& OutputLayer = {});
	void FileExportFlattenedLayerStack(const FString& OutputLayer = {});
	void FileReload();
	void FileReset();
	void FileClose();
	void ActionsImportWithDialog();
	void ActionsImport(const FString& OutputContentFolder, UUsdStageImportOptions* Options);
	void ActionsRegenerate();
	void ActionsSelectStageActor();
	void ActionsOpenLevelSequence();
	void ExportSelectedLayers(const FString& OutputLayerOrDirectory = {});

protected:
	void SetupStageActorDelegates();
	void ClearStageActorDelegates();

	TSharedRef<SWidget> MakeMainMenu();
	TSharedRef<SWidget> MakeActorPickerMenu();
	TSharedRef<SWidget> MakeActorPickerMenuContent();
	TSharedRef<SWidget> MakeIsolateWarningButton();
	void FillFileMenu(FMenuBuilder& MenuBuilder);
	void FillActionsMenu(FMenuBuilder& MenuBuilder);
	void FillOptionsMenu(FMenuBuilder& MenuBuilder);
	void FillExportSubMenu(FMenuBuilder& MenuBuilder);
	void FillStageStateSubMenu(FMenuBuilder& MenuBuilder);
	void FillPayloadsSubMenu(FMenuBuilder& MenuBuilder);
	void FillPurposesToLoadSubMenu(FMenuBuilder& MenuBuilder);
	void FillRenderContextSubMenu(FMenuBuilder& MenuBuilder);
	void FillMaterialPurposeSubMenu(FMenuBuilder& MenuBuilder);
	void FillRootMotionSubMenu(FMenuBuilder& MenuBuilder);
	void FillFallbackCollisionTypeMenu(FMenuBuilder& MenuBuilder);
	void FillSubdivisionLevelSubMenu(FMenuBuilder& MenuBuilder);
	void FillMetadataSubMenu(FMenuBuilder& MenuBuilder);
	void FillCollapsingSubMenu(FMenuBuilder& MenuBuilder);
	void FillShareAssetsSubMenu(FMenuBuilder& MenuBuilder);
	void FillInterpolationTypeSubMenu(FMenuBuilder& MenuBuilder);
	void FillSelectionSubMenu(FMenuBuilder& MenuBuilder);
	void FillNaniteThresholdSubMenu(FMenuBuilder& MenuBuilder);
	void FillGeometryCacheImportSubMenu(FMenuBuilder& MenuBuilder);

	void OnLayerIsolated(const UE::FSdfLayer& IsolatedLayer);

	void OnPrimSelectionChanged(const TArray<FString>& PrimPath);

	void OpenStage(const TCHAR* FilePath);

	void RequestLayersTreeViewRefresh();
	void RequestFullRefresh();
	void OnSlateTick(float Time);

	void OnViewportSelectionChanged(UObject* NewSelection);

	void OnPostPIEStarted(bool bIsSimulating);
	void OnEndPIE(bool bIsSimulating);

	int32 GetNaniteTriangleThresholdValue() const;
	void OnNaniteTriangleThresholdValueChanged(int32 InValue);
	void OnNaniteTriangleThresholdValueCommitted(int32 InValue, ETextCommit::Type InCommitType);

	int32 GetSubdivisionLevelValue() const;
	void OnSubdivisionLevelValueChanged(int32 InValue);
	void OnSubdivisionLevelValueCommitted(int32 InValue, ETextCommit::Type InCommitType);

	UE::FUsdStageWeak GetCurrentStage() const;

	AUsdStageActor* GetStageActorOrCDO() const;

protected:
	TSharedPtr<class SUsdStageTreeView> UsdStageTreeView;
	TSharedPtr<class SUsdPrimInfo> UsdPrimInfoWidget;
	TSharedPtr<class SUsdLayersTreeView> UsdLayersTreeView;

	FDelegateHandle OnActorLoadedHandle;
	FDelegateHandle OnActorDestroyedHandle;
	FDelegateHandle OnStageActorPropertyChangedHandle;
	FDelegateHandle OnStageChangedHandle;
	FDelegateHandle OnStageEditTargetChangedHandle;
	FDelegateHandle OnPrimChangedHandle;
	FDelegateHandle OnSdfLayersChangedHandle;
	FDelegateHandle OnSdfLayerDirtinessChangedHandle;
	FDelegateHandle OnViewportSelectionChangedHandle;
	FDelegateHandle PostPIEStartedHandle;
	FDelegateHandle EndPIEHandle;

	FString SelectedPrimPath;

	FUsdStageViewModel ViewModel;

	FCurveSequence FlashActorPickerCurve;

	// True while we're in the middle of setting the viewport selection from the prim selection
	bool bUpdatingViewportSelection;

	// True while we're in the middle of updating the prim selection from the viewport selection
	bool bUpdatingPrimSelection;

	// We keep this menu alive instead of recreating it every time because it doesn't expose
	// setting/getting/responding to the picked world, which resets every time it is reconstructed (see UE-127879)
	// By keeping it alive it will keep its own state and we just do a full refresh when needed
	TSharedPtr<ISceneOutliner> ActorPickerMenu;

	int32 CurrentNaniteThreshold = INT32_MAX;

	int32 CurrentSubdivisionLevel = 0;

	TArray<TSharedPtr<FString>> MaterialPurposes;

	// We use our own bool to track engine exit because we use some tickers to delay work onto the next frame,
	// and during engine shutdown its possible for this delayed work to run *before* IsEngineExitRequested() actually
	// turns true, so we can't check it.
	// If that work continues (and triggers its slate updates and so on), we can run into crashes as we're
	// trying to render slate as the engine shuts down.
	// This bool is set on GEditor->OnEditorClose(), which happens earlier in the shutdown process, and can be used
	// to let our ticker lambdas gracefully early out during those shutdown scenarios.
	bool bEditorIsShuttingDown = false;
	FDelegateHandle OnEditorCloseHandle;
};

#endif	  // #if USE_USD_SDK
