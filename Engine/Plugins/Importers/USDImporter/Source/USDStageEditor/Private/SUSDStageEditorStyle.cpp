// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDStageEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH(RelativePath, ...) FSlateImageBrush(FUsdStageEditorStyle::InContent(RelativePath, ".png"), __VA_ARGS__)
#define RootToContentDir StyleSet->RootToContentDir

FString FUsdStageEditorStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("USDImporter"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr<FSlateStyleSet> FUsdStageEditorStyle::StyleSet = nullptr;
TSharedPtr<class ISlateStyle> FUsdStageEditorStyle::Get()
{
	return StyleSet;
}

FName FUsdStageEditorStyle::GetStyleSetName()
{
	static FName StyleName(TEXT("UsdStageEditor"));
	return StyleName;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FUsdStageEditorStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon14x14(14.0f, 14.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
		/** Height of items in the stage/layer tree views, prim info panel, etc. */
		StyleSet->Set("UsdStageEditor.ListItemHeight", 20.0f);

		/** Color of Prim labels that have composition arcs */
		StyleSet->Set("UsdStageEditor.PrimCompositionArcColor", FLinearColor(FColor::Orange));

		/** Highlighted color of Prim labels that have composition arcs */
		StyleSet->Set("UsdStageEditor.HighlightPrimCompositionArcColor", FLinearColor(0.2f, 0.08f, 0.0f, 1.0f));

		/**
		 * Checkmark that is always white. We can't use the editor style one in some places (which by default is styled blue)
		 * because the highlight color is also the same blue, so the checkbox would be invisible
		 */
		StyleSet->Set("UsdStageEditor.CheckBoxImage", new IMAGE_BRUSH("Common/Check", Icon14x14, FLinearColor::White));

		StyleSet->Set("UsdStageEditor.ActorPickerFlashBrush", new FSlateRoundedBoxBrush(FLinearColor::White, 4.0f));

		/**
		 * Button without a background, border or foreground color that doesn't move when pressed.
		 * No foreground color is important to let the automatic slate foreground color propagate down.
		 */
		const FButtonStyle Style = FButtonStyle()
									   .SetNormal(FSlateNoResource())
									   .SetHovered(FSlateNoResource())
									   .SetPressed(FSlateNoResource())
									   .SetDisabled(FSlateNoResource())
									   .SetNormalPadding(FMargin(2, 2, 2, 2))
									   .SetPressedPadding(FMargin(2, 2, 2, 2));
		StyleSet->Set("NoBorder", Style);
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FUsdStageEditorStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
