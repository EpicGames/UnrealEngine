// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDStageTreeView.h"

#include "SUSDStageEditorStyle.h"
#include "UnrealUSDWrapper.h"
#include "USDAttributeUtils.h"
#include "USDConversionUtils.h"
#include "USDDuplicateType.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDOptionsWindow.h"
#include "USDPrimViewModel.h"
#include "USDProjectSettings.h"
#include "USDReferenceOptions.h"
#include "USDSetUtils.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Editor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/Function.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "UsdStageTreeView"

#if USE_USD_SDK

#include "USDIncludesStart.h"
#include "pxr/usd/usdPhysics/tokens.h"
#include "pxr/usd/usdSkel/tokens.h"
#include "USDIncludesEnd.h"

namespace UE::USDStageTreeView::Private
{
	static const FText NoSpecOnLocalLayerStack = LOCTEXT(
		"NoLocalSpecToolTip",
		"This prim needs at least one spec on the stage's local layer stack for this option to be usable"
	);
}

enum class EPayloadsTrigger
{
	Load,
	Unload,
	Toggle
};

class FUsdStageNameColumn
	: public FUsdTreeViewColumn
	, public TSharedFromThis<FUsdStageNameColumn>
{
public:
	DECLARE_DELEGATE_TwoParams(FOnPrimNameCommitted, const FUsdPrimViewModelRef&, const FText&);
	FOnPrimNameCommitted OnPrimNameCommitted;

	DECLARE_DELEGATE_ThreeParams(FOnPrimNameUpdated, const FUsdPrimViewModelRef&, const FText&, FText&);
	FOnPrimNameUpdated OnPrimNameUpdated;

	TWeakPtr<SUsdStageTreeView> OwnerTree;

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		if (!InTreeItem)
		{
			return SNullWidget::NullWidget;
		}

		TSharedPtr<FUsdPrimViewModel> TreeItem = StaticCastSharedPtr<FUsdPrimViewModel>(InTreeItem);

		// clang-format off
		TSharedRef<SInlineEditableTextBlock> Item =
			SNew(SInlineEditableTextBlock)
			.Text(TreeItem->RowData, &FUsdPrimModel::GetName)
			.ColorAndOpacity(this, &FUsdStageNameColumn::GetTextColor, TreeItem)
			.OnTextCommitted(this, &FUsdStageNameColumn::OnTextCommitted, TreeItem)
			.OnVerifyTextChanged(this, &FUsdStageNameColumn::OnTextUpdated, TreeItem)
			.IsReadOnly_Lambda([TreeItem]()
			{
				return !TreeItem->bIsRenamingExistingPrim && (!TreeItem || TreeItem->UsdPrim);
			});

		TreeItem->RenameRequestEvent.BindSP(&Item.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		return SNew(SBox)
			.VAlign(VAlign_Center)
			[
				Item
			];
		// clang-format on
	}

protected:
	void OnTextCommitted(const FText& InPrimName, ETextCommit::Type InCommitInfo, TSharedPtr<FUsdPrimViewModel> TreeItem)
	{
		if (!TreeItem)
		{
			return;
		}

		OnPrimNameCommitted.ExecuteIfBound(TreeItem.ToSharedRef(), InPrimName);
	}

	bool OnTextUpdated(const FText& InPrimName, FText& ErrorMessage, TSharedPtr<FUsdPrimViewModel> TreeItem)
	{
		if (!TreeItem)
		{
			return false;
		}

		OnPrimNameUpdated.ExecuteIfBound(TreeItem.ToSharedRef(), InPrimName, ErrorMessage);

		return ErrorMessage.IsEmpty();
	}

	FSlateColor GetTextColor(TSharedPtr<FUsdPrimViewModel> TreeItem) const
	{
		FSlateColor TextColor = FSlateColor::UseForeground();
		if (!TreeItem)
		{
			return TextColor;
		}

		if (TreeItem->RowData->HasCompositionArcs())
		{
			const TSharedPtr<SUsdStageTreeView> OwnerTreePtr = OwnerTree.Pin();
			if (OwnerTreePtr && OwnerTreePtr->IsItemSelected(TreeItem.ToSharedRef()))
			{
				TextColor = FUsdStageEditorStyle::Get()->GetColor("UsdStageEditor.HighlightPrimCompositionArcColor");
			}
			else
			{
				TextColor = FUsdStageEditorStyle::Get()->GetColor("UsdStageEditor.PrimCompositionArcColor");
			}
		}

		return TextColor;
	}
};

class FUsdStagePayloadColumn
	: public FUsdTreeViewColumn
	, public TSharedFromThis<FUsdStagePayloadColumn>
{
public:
	ECheckBoxState IsChecked(const FUsdPrimViewModelPtr InTreeItem) const
	{
		ECheckBoxState CheckedState = ECheckBoxState::Unchecked;

		if (InTreeItem && InTreeItem->RowData->HasPayload())
		{
			CheckedState = InTreeItem->RowData->IsLoaded() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return CheckedState;
	}

	void OnCheckedPayload(ECheckBoxState NewCheckedState, const FUsdPrimViewModelPtr TreeItem)
	{
		if (!TreeItem)
		{
			return;
		}

		switch (NewCheckedState)
		{
			case ECheckBoxState::Checked:
				TreeItem->UsdPrim.Load();
				break;
			case ECheckBoxState::Unchecked:
				TreeItem->UsdPrim.Unload();
				break;
			default:
				break;
		}
	}

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		const TWeakPtr<FUsdPrimViewModel> TreeItem = StaticCastSharedPtr<FUsdPrimViewModel>(InTreeItem);

		// clang-format off
		TSharedRef<SWidget> Item = SNew(SCheckBox)
		.Visibility_Lambda([TreeItem]() -> EVisibility
		{
			if (const TSharedPtr<FUsdPrimViewModel> PinnedItem = TreeItem.Pin())
			{
				return PinnedItem->RowData->HasPayload() ? EVisibility::Visible : EVisibility::Collapsed;
			}

			return EVisibility::Collapsed;
		})
		.ToolTipText(LOCTEXT("TogglePayloadToolTip", "Toggle payload"))
		.IsChecked(this, &FUsdStagePayloadColumn::IsChecked, StaticCastSharedPtr<FUsdPrimViewModel>(InTreeItem))
		.OnCheckStateChanged(this, &FUsdStagePayloadColumn::OnCheckedPayload, StaticCastSharedPtr<FUsdPrimViewModel>(InTreeItem));
		// clang-format on

		return Item;
	}
};

class FUsdStageVisibilityColumn
	: public FUsdTreeViewColumn
	, public TSharedFromThis<FUsdStageVisibilityColumn>
{
public:
	FReply OnToggleVisibility(const FUsdPrimViewModelPtr TreeItem)
	{
		FScopedTransaction Transaction(
			FText::Format(LOCTEXT("VisibilityTransaction", "Toggle visibility of prim '{0}'"), FText::FromName(TreeItem->UsdPrim.GetName()))
		);

		TreeItem->ToggleVisibility();

		return FReply::Handled();
	}

	const FSlateBrush* GetBrush(const FUsdPrimViewModelPtr TreeItem, const TSharedPtr<SButton> Button) const
	{
		const bool bIsButtonHovered = Button.IsValid() && Button->IsHovered();

		if (TreeItem->RowData->IsVisible())
		{
			return bIsButtonHovered ? FAppStyle::GetBrush("Level.VisibleHighlightIcon16x") : FAppStyle::GetBrush("Level.VisibleIcon16x");
		}
		else
		{
			return bIsButtonHovered ? FAppStyle::GetBrush("Level.NotVisibleHighlightIcon16x") : FAppStyle::GetBrush("Level.NotVisibleIcon16x");
		}
	}

	FSlateColor GetForegroundColor(const FUsdPrimViewModelPtr TreeItem, const TSharedPtr<ITableRow> TableRow, const TSharedPtr<SButton> Button) const
	{
		if (!TreeItem.IsValid() || !TableRow.IsValid() || !Button.IsValid())
		{
			return FSlateColor::UseForeground();
		}

		const bool bIsRowHovered = TableRow->AsWidget()->IsHovered();
		const bool bIsButtonHovered = Button->IsHovered();
		const bool bIsRowSelected = TableRow->IsItemSelected();
		const bool bIsPrimVisible = TreeItem->RowData->IsVisible();

		if (bIsPrimVisible && !bIsRowHovered && !bIsRowSelected)
		{
			return FLinearColor::Transparent;
		}
		else if (bIsButtonHovered && !bIsRowSelected)
		{
			return FAppStyle::GetSlateColor(TEXT("Colors.ForegroundHover"));
		}

		return FSlateColor::UseForeground();
	}

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		if (!InTreeItem)
		{
			return SNullWidget::NullWidget;
		}

		const TSharedPtr<FUsdPrimViewModel> TreeItem = StaticCastSharedPtr<FUsdPrimViewModel>(InTreeItem);
		const float ItemSize = FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight");

		// clang-format off
		if (!TreeItem->HasVisibilityAttribute())
		{
			return SNew(SBox)
				.HeightOverride(ItemSize)
				.WidthOverride(ItemSize)
				.Visibility(EVisibility::Visible)
				.ToolTip(SNew(SToolTip).Text(LOCTEXT("NoGeomImageable", "Only prims with the GeomImageable schema (or derived) have the visibility attribute!")));
		}

		TSharedPtr<SButton> Button = SNew(SButton)
			.ContentPadding(0)
			.ButtonStyle(FUsdStageEditorStyle::Get(), TEXT("NoBorder"))
			.OnClicked(this, &FUsdStageVisibilityColumn::OnToggleVisibility, TreeItem)
			.ToolTip(SNew(SToolTip).Text(LOCTEXT("GeomImageable", "Toggle the visibility of this prim")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center);

		TSharedPtr<SImage> Image = SNew(SImage)
			.Image(this, &FUsdStageVisibilityColumn::GetBrush, TreeItem, Button)
			.ColorAndOpacity(this, &FUsdStageVisibilityColumn::GetForegroundColor, TreeItem, TableRow, Button);

		Button->SetContent(Image.ToSharedRef());

		return SNew(SBox)
			.HeightOverride(ItemSize)
			.WidthOverride(ItemSize)
			.Visibility(EVisibility::Visible)
			[
				Button.ToSharedRef()
			];
		// clang-format on
	}
};

class FUsdStagePrimTypeColumn : public FUsdTreeViewColumn
{
public:
	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		TSharedPtr<FUsdPrimViewModel> TreeItem = StaticCastSharedPtr<FUsdPrimViewModel>(InTreeItem);

		// clang-format off
		return SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(TreeItem->RowData, &FUsdPrimModel::GetType)
			];
		// clang-format on
	}
};

void SUsdStageTreeView::Construct(const FArguments& InArgs)
{
	SUsdTreeView::Construct(SUsdTreeView::FArguments());

	OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SUsdStageTreeView::ConstructPrimContextMenu);

	OnSelectionChanged = FOnSelectionChanged::CreateLambda(
		[this](FUsdPrimViewModelPtr UsdStageTreeItem, ESelectInfo::Type SelectionType)
		{
			FString SelectedPrimPath;

			if (UsdStageTreeItem)
			{
				SelectedPrimPath = UsdToUnreal::ConvertPath(UsdStageTreeItem->UsdPrim.GetPrimPath());
			}

			TArray<FString> SelectedPrimPaths = GetSelectedPrimPaths();
			this->OnPrimSelectionChanged.ExecuteIfBound(SelectedPrimPaths);
		}
	);

	OnExpansionChanged = FOnExpansionChanged::CreateLambda(
		[this](const FUsdPrimViewModelPtr& UsdPrimViewModel, bool bIsExpanded)
		{
			if (!UsdPrimViewModel)
			{
				return;
			}

			const UE::FUsdPrim& Prim = UsdPrimViewModel->UsdPrim;
			if (!Prim)
			{
				return;
			}

			UsdPrimViewModel->SetIsExpanded(bIsExpanded);

			// We have a special handling for the root because we'll want to manually expand it by
			// default at first but also remember if the user collapsed it or not. For all
			// other prims we truly just leave the nodes at default collapsed unless we have
			// recorded that the node should be expanded
			if (Prim.IsPseudoRoot())
			{
				RootWasExpanded = bIsExpanded;
			}
			else
			{
				const FString& PrimPath = Prim.GetPrimPath().GetString();
				if (bIsExpanded)
				{
					ExpandedPrimPaths.Add(PrimPath);
				}
				else
				{
					ExpandedPrimPaths.Remove(PrimPath);
				}
			}
		}
	);

	OnPrimSelectionChanged = InArgs._OnPrimSelectionChanged;

	PostUndoRedoHandle = FEditorDelegates::PostUndoRedo.AddLambda(
		[this]()
		{
			// This is in charge of restoring our expansion states after we undo/redo a prim rename
			RequestExpansionStateRestore();
		}
	);

	UICommandList = MakeShared<FUICommandList>();
	UICommandList->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnCutPrim),
		FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimHaveSpecOnLocalLayerStack)
	);
	UICommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnCopyPrim),
		FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnStage)
	);
	UICommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnPastePrim),
		FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::CanPastePrim)
	);
	UICommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnDuplicatePrim, EUsdDuplicateType::AllLocalLayerSpecs),
		FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnStage)
	);
	UICommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnDeletePrim),
		FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimHaveSpecOnLocalLayerStack)
	);
	UICommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnRenamePrim),
		FCanExecuteAction::CreateLambda(
			[this]()
			{
				return SUsdStageTreeView::DoesPrimHaveSpecOnLocalLayerStack() && GetSelectedItems().Num() == 1;
			}
		)
	);
}

SUsdStageTreeView::~SUsdStageTreeView()
{
	FEditorDelegates::PostUndoRedo.Remove(PostUndoRedoHandle);
}

TSharedRef<ITableRow> SUsdStageTreeView::OnGenerateRow(FUsdPrimViewModelRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUsdTreeRow<FUsdPrimViewModelRef>, InDisplayNode, OwnerTable, SharedData);
}

void SUsdStageTreeView::OnGetChildren(FUsdPrimViewModelRef InParent, TArray<FUsdPrimViewModelRef>& OutChildren) const
{
	for (const FUsdPrimViewModelRef& Child : InParent->UpdateChildren())
	{
		OutChildren.Add(Child);
	}
}

void SUsdStageTreeView::Refresh(const UE::FUsdStageWeak& NewStage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SUsdStageTreeView::Refresh);

	UE::FUsdStageWeak OldStage = RootItems.Num() > 0 ? RootItems[0]->UsdStage : UE::FUsdStageWeak();

	RootItems.Empty();

	UsdStage = NewStage;

	if (OldStage != NewStage)
	{
		RootWasExpanded.Reset();
		ExpandedPrimPaths.Reset();
	}

	if (NewStage)
	{
		if (UE::FUsdPrim RootPrim = NewStage.GetPseudoRoot())
		{
			RootItems.Add(MakeShared<FUsdPrimViewModel>(nullptr, UsdStage, RootPrim));
		}

		RequestExpansionStateRestore();
	}
}

void SUsdStageTreeView::RefreshPrim(const FString& PrimPath, bool bResync)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SUsdStageTreeView::RefreshPrim);

	FScopedUnrealAllocs UnrealAllocs;	 // RefreshPrim can be called by a delegate for which we don't know the active allocator

	FUsdPrimViewModelPtr FoundItem = GetItemFromPrimPath(PrimPath);

	if (FoundItem.IsValid())
	{
		FoundItem->RefreshData(true);

		// Item doesn't match any prim, needs to be removed
		if (!FoundItem->UsdPrim)
		{
			if (FoundItem->ParentItem)
			{
				FoundItem->ParentItem->RefreshData(true);
			}
			else
			{
				RootItems.Remove(FoundItem.ToSharedRef());
			}
		}
	}
	// We couldn't find the target prim, do a full refresh instead
	else
	{
		FWriteScopeLock StageTreeViewLock{RefreshStateLock};
		bNeedsFullUpdate = true;
	}

	if (bResync)
	{
		FWriteScopeLock StageTreeViewLock{RefreshStateLock};
		bNeedsFullUpdate = true;
	}
}

FUsdPrimViewModelPtr SUsdStageTreeView::GetItemFromPrimPath(const FString& PrimPath)
{
	FScopedUnrealAllocs UnrealAllocs;

	UE::FSdfPath UsdPrimPath(*PrimPath);
	if (!UsdStage.GetPrimAtPath(UsdPrimPath))
	{
		return nullptr;
	}

	TFunction<FUsdPrimViewModelPtr(const UE::FSdfPath&, const FUsdPrimViewModelRef&)> FindTreeItemFromPrimPath;
	FindTreeItemFromPrimPath = [&FindTreeItemFromPrimPath,
								this](const UE::FSdfPath& UsdPrimPath, const FUsdPrimViewModelRef& ItemRef) -> FUsdPrimViewModelPtr
	{
		UE::FSdfPath ItemPath = ItemRef->UsdPrim.GetPrimPath();
		if (ItemPath == UsdPrimPath)
		{
			return ItemRef;
		}
		else if (UsdPrimPath.HasPrefix(ItemPath))
		{
			// If we're past the check at the top of this function we know we have a prim for this path.
			// If we do, then we *must* be able to generate a FUsdPrimViewModelPtr for it (if we dig deep enough),
			// so let's expand ItemRef's parent on-demand, so that we generate our children that we can step into
			if (!ItemRef->ShouldGenerateChildren() && ItemRef->ParentItem)
			{
				SetItemExpansion(StaticCastSharedRef<FUsdPrimViewModel>(ItemRef->ParentItem->AsShared()), true);
			}

			for (FUsdPrimViewModelRef ChildItem : ItemRef->Children)
			{
				if (FUsdPrimViewModelPtr ChildValue = FindTreeItemFromPrimPath(UsdPrimPath, ChildItem))
				{
					return ChildValue;
				}
			}
		}

		return {};
	};

	// Search for the corresponding tree item to update
	FUsdPrimViewModelPtr FoundItem = nullptr;
	for (FUsdPrimViewModelRef RootItem : this->RootItems)
	{
		UE::FSdfPath PrimPathToSearch = UsdPrimPath;

		FoundItem = FindTreeItemFromPrimPath(PrimPathToSearch, RootItem);

		// If we haven't found an item, try finding an item for an ancestor
		while (!FoundItem.IsValid())
		{
			UE::FSdfPath ParentPrimPath = PrimPathToSearch.GetParentPath();
			if (ParentPrimPath == PrimPathToSearch)
			{
				break;
			}
			PrimPathToSearch = MoveTemp(ParentPrimPath);

			FoundItem = FindTreeItemFromPrimPath(PrimPathToSearch, RootItem);
		}

		if (FoundItem.IsValid())
		{
			break;
		}
	}

	return FoundItem;
}

void SUsdStageTreeView::SelectItemsInternal(const TArray<FUsdPrimViewModelRef>& ItemsToSelect)
{
	if (ItemsToSelect.Num() > 0)
	{
		// Clear selection without emitting events, as we'll emit new events with SetItemSelection
		// anyway. This prevents a UI blink as OnPrimSelectionChanged would otherwise fire for
		// ClearSelection() and then again right away for SetItemSelection()
		Private_ClearSelection();

		const bool bSelected = true;
		SetItemSelection(ItemsToSelect, bSelected);
		ScrollItemIntoView(ItemsToSelect.Last());
	}
	else
	{
		ClearSelection();

		// ClearSelection is not going to fire the OnSelectionChanged event in case we have nothing selected, but we
		// need to do that to refresh the prim properties panel to display the stage properties instead
		OnPrimSelectionChanged.ExecuteIfBound({});
	}
}

void SUsdStageTreeView::SetSelectedPrimPaths(const TArray<FString>& PrimPaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SetSelectedPrimPaths);

	TArray<FUsdPrimViewModelRef> ItemsToSelect;
	ItemsToSelect.Reserve(PrimPaths.Num());

	for (const FString& PrimPath : PrimPaths)
	{
		if (FUsdPrimViewModelPtr FoundItem = GetItemFromPrimPath(PrimPath))
		{
			ItemsToSelect.Add(FoundItem.ToSharedRef());
		}
	}

	SelectItemsInternal(ItemsToSelect);
}

void SUsdStageTreeView::SetSelectedPrims(const TArray<UE::FUsdPrim>& Prims)
{
	TArray<FString> PrimPaths;
	PrimPaths.Reserve(Prims.Num());
	for (const UE::FUsdPrim& Prim : Prims)
	{
		PrimPaths.Add(Prim.GetPrimPath().GetString());
	}

	SetSelectedPrimPaths(PrimPaths);
}

TArray<FString> SUsdStageTreeView::GetSelectedPrimPaths()
{
	TArray<FString> SelectedPrimPaths;
	SelectedPrimPaths.Reserve(GetNumItemsSelected());

	for (FUsdPrimViewModelRef SelectedItem : GetSelectedItems())
	{
		SelectedPrimPaths.Add(SelectedItem->UsdPrim.GetPrimPath().GetString());
	}

	return SelectedPrimPaths;
}

TArray<UE::FUsdPrim> SUsdStageTreeView::GetSelectedPrims()
{
	TArray<UE::FUsdPrim> SelectedPrims;
	SelectedPrims.Reserve(GetNumItemsSelected());

	for (FUsdPrimViewModelRef SelectedItem : GetSelectedItems())
	{
		SelectedPrims.Add(SelectedItem->UsdPrim);
	}

	return SelectedPrims;
}

void SUsdStageTreeView::SetupColumns()
{
	HeaderRowWidget->ClearColumns();

	SHeaderRow::FColumn::FArguments VisibilityColumnArguments;
	VisibilityColumnArguments.FixedWidth(24.f);

	AddColumn(TEXT("Visibility"), FText(), MakeShared<FUsdStageVisibilityColumn>(), VisibilityColumnArguments);

	{
		TSharedRef<FUsdStageNameColumn> PrimNameColumn = MakeShared<FUsdStageNameColumn>();
		PrimNameColumn->OwnerTree = SharedThis(this);
		PrimNameColumn->bIsMainColumn = true;
		PrimNameColumn->OnPrimNameCommitted.BindRaw(this, &SUsdStageTreeView::OnPrimNameCommitted);
		PrimNameColumn->OnPrimNameUpdated.BindRaw(this, &SUsdStageTreeView::OnPrimNameUpdated);

		SHeaderRow::FColumn::FArguments PrimNameColumnArguments;
		PrimNameColumnArguments.FillWidth(70.f);

		AddColumn(TEXT("Prim"), LOCTEXT("Prim", "Prim"), PrimNameColumn, PrimNameColumnArguments);
	}

	SHeaderRow::FColumn::FArguments TypeColumnArguments;
	TypeColumnArguments.FillWidth(15.f);
	AddColumn(TEXT("Type"), LOCTEXT("Type", "Type"), MakeShared<FUsdStagePrimTypeColumn>(), TypeColumnArguments);

	SHeaderRow::FColumn::FArguments PayloadColumnArguments;
	PayloadColumnArguments.FillWidth(15.f).HAlignHeader(HAlign_Center).HAlignCell(HAlign_Center);
	AddColumn(TEXT("Payload"), LOCTEXT("Payload", "Payload"), MakeShared<FUsdStagePayloadColumn>(), PayloadColumnArguments);
}

TSharedPtr<SWidget> SUsdStageTreeView::ConstructPrimContextMenu()
{
	TSharedRef<SWidget> MenuWidget = SNullWidget::NullWidget;

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder PrimOptions(bInShouldCloseWindowAfterMenuSelection, UICommandList);
	PrimOptions.BeginSection("Edit", LOCTEXT("EditText", "Edit"));
	{
		PrimOptions.AddMenuEntry(
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return GetSelectedItems().Num() == 0 ? LOCTEXT("AddTopLevelPrim", "Add Prim") : LOCTEXT("AddPrim", "Add Child");
				}
			)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return GetSelectedItems().Num() == 0 ? LOCTEXT("AddTopPrim_ToolTip", "Adds a new top-level prim")
														 : LOCTEXT("AddPrim_ToolTip", "Adds a new prim as a child of this one");
				}
			)),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.PlaceActors"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnAddChildPrim),
				FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::CanAddChildPrim)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		PrimOptions.AddMenuEntry(
			FGenericCommands::Get().Cut,
			NAME_None,
			LOCTEXT("Cut_Text", "Cut"),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return DoesPrimHaveSpecOnLocalLayerStack()
							   ? LOCTEXT("Cut_ToolTip", "Cuts the selected prim's specs from the stage's local layer stack")
							   : UE::USDStageTreeView::Private::NoSpecOnLocalLayerStack;
				}
			))
		);

		PrimOptions.AddMenuEntry(
			FGenericCommands::Get().Copy,
			NAME_None,
			LOCTEXT("Copy_Text", "Copy"),
			LOCTEXT("Copy_ToolTip", "Copies the selected prims")
		);

		PrimOptions.AddMenuEntry(
			FGenericCommands::Get().Paste,
			NAME_None,
			LOCTEXT("Paste_Text", "Paste"),
			LOCTEXT("Paste_ToolTip", "Pastes a flattened representation of the cut/copied prims as children of this prim, on the current edit target")
		);

		const bool bInOpenSubMenuOnClick = false;
		PrimOptions.AddSubMenu(
			LOCTEXT("Duplicate_Text", "Duplicate..."),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SUsdStageTreeView::FillDuplicateSubmenu),
			bInOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate")
		);

		PrimOptions.AddMenuEntry(
			FGenericCommands::Get().Delete,
			NAME_None,
			LOCTEXT("Delete_Text", "Delete"),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return DoesPrimHaveSpecOnLocalLayerStack()
							   ? LOCTEXT("Delete_ToolTip", "Deletes the selected prim's specs from the local layer stack")
							   : UE::USDStageTreeView::Private::NoSpecOnLocalLayerStack;
				}
			))
		);

		PrimOptions.AddMenuEntry(
			FGenericCommands::Get().Rename,
			NAME_None,
			LOCTEXT("Rename_Text", "Rename"),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return DoesPrimHaveSpecOnLocalLayerStack()
							   ? LOCTEXT("Rename_ToolTip", "Renames the selected prim's specs on the local layer stack")
							   : UE::USDStageTreeView::Private::NoSpecOnLocalLayerStack;
				}
			))
		);

		PrimOptions.AddSubMenu(
			LOCTEXT("Collapsing_Text", "Collapsing..."),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SUsdStageTreeView::FillCollapsingSubmenu),
			bInOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Merge")
		);
	}
	PrimOptions.EndSection();

	PrimOptions.BeginSection("Payloads", LOCTEXT("Payloads", "Payloads"));
	{
		PrimOptions.AddMenuEntry(
			LOCTEXT("TogglePayloads", "Toggle All Payloads"),
			LOCTEXT("TogglePayloads_ToolTip", "Toggles all payloads for this prim and its children"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnToggleAllPayloads, EPayloadsTrigger::Toggle), FCanExecuteAction()),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		PrimOptions.AddMenuEntry(
			LOCTEXT("LoadPayloads", "Load All Payloads"),
			LOCTEXT("LoadPayloads_ToolTip", "Loads all payloads for this prim and its children"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnToggleAllPayloads, EPayloadsTrigger::Load), FCanExecuteAction()),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		PrimOptions.AddMenuEntry(
			LOCTEXT("UnloadPayloads", "Unload All Payloads"),
			LOCTEXT("UnloadPayloads_ToolTip", "Unloads all payloads for this prim and its children"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnToggleAllPayloads, EPayloadsTrigger::Unload), FCanExecuteAction()),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	PrimOptions.EndSection();

	PrimOptions.BeginSection("Composition", LOCTEXT("Composition", "Composition"));
	{
		PrimOptions.AddMenuEntry(
			LOCTEXT("AddReference", "Add Reference"),
			LOCTEXT("AddReference_ToolTip", "Adds a reference for this prim"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnAddReference),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						return GetSelectedItems().Num() == 1 && GetSelectedItems()[0]->UsdPrim && !GetSelectedItems()[0]->UsdPrim.IsPseudoRoot();
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		PrimOptions.AddMenuEntry(
			LOCTEXT("ClearReferences", "Clear References"),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return DoesPrimHaveReferenceSpecOnLocalLayerStack()
							   ? LOCTEXT("ClearReferences_ToolTip", "Clears the references for this prim")
							   : LOCTEXT("ClearReferencesNoSpec_ToolTip", "The prim doesn't have any reference spec on the current edit target");
				}
			)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnClearReferences),
				FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimHaveReferenceSpecOnLocalLayerStack)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		PrimOptions.AddMenuEntry(
			LOCTEXT("AddPayload", "Add Payload"),
			LOCTEXT("AddPayload_ToolTip", "Adds a payload for this prim"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnAddPayload),
				FCanExecuteAction::CreateLambda(
					[this]()
					{
						return GetSelectedItems().Num() == 1 && GetSelectedItems()[0]->UsdPrim && !GetSelectedItems()[0]->UsdPrim.IsPseudoRoot();
					}
				)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		PrimOptions.AddMenuEntry(
			LOCTEXT("ClearPayloads", "Clear Payloads"),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					return DoesPrimHavePayloadSpecOnLocalLayerStack()
							   ? LOCTEXT("ClearPayloads_ToolTip", "Clears the payloads for this prim")
							   : LOCTEXT("ClearPayloadsNoSpec_ToolTip", "The prim doesn't have any payload spec on the current edit target");
				}
			)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnClearPayloads),
				FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimHavePayloadSpecOnLocalLayerStack)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	PrimOptions.EndSection();

	PrimOptions.BeginSection("Schemas", LOCTEXT("Schemas", "Schemas"));
	{
		const bool bInOpenSubMenuOnClick = false;
		PrimOptions.AddSubMenu(
			LOCTEXT("AddSchemaText", "Add schema..."),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SUsdStageTreeView::FillAddSchemaSubmenu),
			bInOpenSubMenuOnClick
		);

		PrimOptions.AddSubMenu(
			LOCTEXT("RemoveSchemaText", "Remove schema..."),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SUsdStageTreeView::FillRemoveSchemaSubmenu),
			bInOpenSubMenuOnClick
		);
	}
	PrimOptions.EndSection();

	MenuWidget = PrimOptions.MakeWidget();

	return MenuWidget;
}

void SUsdStageTreeView::OnAddChildPrim()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	// Add a new child prim
	if (MySelectedItems.Num() > 0)
	{
		for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
		{
			FUsdPrimViewModelRef TreeItem = MakeShared<FUsdPrimViewModel>(&SelectedItem.Get(), SelectedItem->UsdStage);
			SelectedItem->Children.Add(TreeItem);

			PendingRenameItem = TreeItem;
			ScrollItemIntoView(TreeItem);
		}
	}
	// Add a new top-level prim (direct child of the pseudo-root prim)
	else
	{
		FUsdPrimViewModelRef TreeItem = MakeShared<FUsdPrimViewModel>(nullptr, UsdStage);
		RootItems.Add(TreeItem);

		PendingRenameItem = TreeItem;
		ScrollItemIntoView(TreeItem);
	}

	RequestTreeRefresh();
}

void SUsdStageTreeView::OnCutPrim()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("CutPrimTransaction", "Cut prims"));

	UE::FSdfChangeBlock Block;

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(MySelectedItems.Num());

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->UsdPrim)
		{
			Prims.Add(SelectedItem->UsdPrim);
		}
	}

	UsdUtils::CutPrims(Prims);
}

void SUsdStageTreeView::OnCopyPrim()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(MySelectedItems.Num());

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->UsdPrim)
		{
			Prims.Add(SelectedItem->UsdPrim);
		}
	}

	UsdUtils::CopyPrims(Prims);
}

void SUsdStageTreeView::OnPastePrim()
{
	if (!UsdStage)
	{
		return;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("PastePrimTransaction", "Paste prims"));

	UE::FSdfChangeBlock Block;

	UE::FUsdPrim ParentPrim;

	// This happens when right-clicking the background area without selecting any prim
	if (MySelectedItems.IsEmpty())
	{
		ParentPrim = UsdStage.GetPseudoRoot();
	}
	else if (MySelectedItems.Num() > 0)
	{
		// Only paste for the first selection, for simplicity
		ParentPrim = MySelectedItems[0]->UsdPrim.GetParent();
	}

	if (!ParentPrim)
	{
		return;
	}

	// Preemptively mark the parent prims as expanded so that we can always see what we pasted
	ExpandedPrimPaths.Add(ParentPrim.GetPrimPath().GetString());

	UsdUtils::PastePrims(ParentPrim);
}

void SUsdStageTreeView::OnDuplicatePrim(EUsdDuplicateType DuplicateType)
{
	if (!UsdStage)
	{
		return;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("DuplicatePrimTransaction", "Duplicate prims"));

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(MySelectedItems.Num());

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->UsdPrim)
		{
			Prims.Add(SelectedItem->UsdPrim);
		}
	}

	UsdUtils::DuplicatePrims(Prims, DuplicateType, UsdStage.GetEditTarget());
}

void SUsdStageTreeView::OnDeletePrim()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("DeletePrimTransaction", "Delete prims"));

	UE::FSdfChangeBlock Block;

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		UsdUtils::RemoveAllLocalPrimSpecs(SelectedItem->UsdPrim);
	}
}

void SUsdStageTreeView::OnRenamePrim()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	if (MySelectedItems.Num() > 0)
	{
		FUsdPrimViewModelRef TreeItem = MySelectedItems[0];

		TreeItem->bIsRenamingExistingPrim = true;
		PendingRenameItem = TreeItem;
		RequestScrollIntoView(TreeItem);
	}
}

void SUsdStageTreeView::OnSetCollapsingPreference(UsdUtils::ECollapsingPreference Preference)
{
	if (!UsdStage)
	{
		return;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("SetCollapsingPreferenceTransaction", "Set collapsing preference"));

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		UsdUtils::SetCollapsingPreference(SelectedItem->UsdPrim, Preference);
	}
}

void SUsdStageTreeView::OnAddReference()
{
	if (!UsdStage || !UsdStage.IsEditTargetValid())
	{
		return;
	}

	TStrongObjectPtr<UUsdReferenceOptions> Options = TStrongObjectPtr<UUsdReferenceOptions>{NewObject<UUsdReferenceOptions>()};
	UUsdReferenceOptions* OptionsPtr = Options.Get();
	if (!OptionsPtr)
	{
		return;
	}

	bool bContinue = SUsdOptionsWindow::ShowOptions(*OptionsPtr, LOCTEXT("AddReferenceTitle", "Add reference"), LOCTEXT("AddReferenceAccept", "OK"));
	if (!bContinue)
	{
		return;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	if (MySelectedItems.Num() != 1)
	{
		return;
	}
	UE::FUsdPrim Referencer = MySelectedItems[0]->UsdPrim;
	if (UsdUtils::NotifyIfInstanceProxy(Referencer))
	{
		return;
	}

	// This transaction is important as adding a reference may trigger the creation of new unreal assets, which need to be
	// destroyed if we spam undo afterwards. Undoing won't remove the actual reference from the stage yet though, sadly...
	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("AddReferenceTransaction", "Add reference from prim '{0}'"), FText::FromString(Referencer.GetPrimPath().GetString()))
	);

	// AddReference expects absolute file paths, so let's try ensuring that
	FString ReferencedLayerPath = OptionsPtr->TargetFile.FilePath;
	if (OptionsPtr->bInternalReference)
	{
		ReferencedLayerPath = TEXT("");
		OptionsPtr->bUseDefaultPrim = false;
	}
	else if (FPaths::IsRelative(ReferencedLayerPath))
	{
		FString AbsoluteLayerFromBinary = FPaths::ConvertRelativePathToFull(ReferencedLayerPath);
		if (!FPaths::IsRelative(AbsoluteLayerFromBinary) && FPaths::FileExists(AbsoluteLayerFromBinary))
		{
			ReferencedLayerPath = AbsoluteLayerFromBinary;
		}
	}

	UsdUtils::AddReference(
		Referencer,
		*ReferencedLayerPath,
		TOptional<EReferencerTypeHandling>{},	 // Explicit unset optional to check project settings
		OptionsPtr->bUseDefaultPrim ? UE::FSdfPath{} : UE::FSdfPath{*OptionsPtr->TargetPrimPath},
		OptionsPtr->TimeCodeOffset,
		OptionsPtr->TimeCodeScale
	);
}

void SUsdStageTreeView::OnClearReferences()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("ClearReferenceTransaction", "Clear references to USD layers"));

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		SelectedItem->ClearReferences();
	}
}

void SUsdStageTreeView::OnAddPayload()
{
	if (!UsdStage || !UsdStage.IsEditTargetValid())
	{
		return;
	}

	TStrongObjectPtr<UUsdReferenceOptions> Options = TStrongObjectPtr<UUsdReferenceOptions>{NewObject<UUsdReferenceOptions>()};
	UUsdReferenceOptions* OptionsPtr = Options.Get();
	if (!OptionsPtr)
	{
		return;
	}

	bool bContinue = SUsdOptionsWindow::ShowOptions(*OptionsPtr, LOCTEXT("AddPayloadTitle", "Add payload"), LOCTEXT("AddPayloadAccept", "OK"));
	if (!bContinue)
	{
		return;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	if (MySelectedItems.Num() != 1)
	{
		return;
	}
	UE::FUsdPrim Referencer = MySelectedItems[0]->UsdPrim;
	if (UsdUtils::NotifyIfInstanceProxy(Referencer))
	{
		return;
	}

	// This transaction is important as adding a payload may trigger the creation of new unreal assets, which need to be
	// destroyed if we spam undo afterwards. Undoing won't remove the actual payload from the stage yet though, sadly...
	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("AddPayloadTransaction", "Add payload from prim '{0}'"), FText::FromString(Referencer.GetPrimPath().GetString()))
	);

	// AddPayload expects absolute file paths too, so let's try ensuring that
	FString ReferencedLayerPath = OptionsPtr->TargetFile.FilePath;
	if (OptionsPtr->bInternalReference)
	{
		ReferencedLayerPath = TEXT("");
		OptionsPtr->bUseDefaultPrim = false;
	}
	else if (FPaths::IsRelative(ReferencedLayerPath))
	{
		FString AbsoluteLayerFromBinary = FPaths::ConvertRelativePathToFull(ReferencedLayerPath);
		if (!FPaths::IsRelative(AbsoluteLayerFromBinary) && FPaths::FileExists(AbsoluteLayerFromBinary))
		{
			ReferencedLayerPath = AbsoluteLayerFromBinary;
		}
	}

	UsdUtils::AddPayload(
		Referencer,
		*ReferencedLayerPath,
		TOptional<EReferencerTypeHandling>{},	 // Explicit unset optional to check project settings
		OptionsPtr->bUseDefaultPrim ? UE::FSdfPath{} : UE::FSdfPath{*OptionsPtr->TargetPrimPath},
		OptionsPtr->TimeCodeOffset,
		OptionsPtr->TimeCodeScale
	);
}

void SUsdStageTreeView::OnClearPayloads()
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("ClearPayloadTransaction", "Clear payloads to USD layers"));

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		SelectedItem->ClearPayloads();
	}
}

void SUsdStageTreeView::OnApplySchema(FName SchemaName)
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("ApplySchemaTransaction", "Apply the '{0}' schema onto selected prims"), FText::FromName(SchemaName))
	);

	UE::FSdfChangeBlock Block;

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		SelectedItem->ApplySchema(SchemaName);
	}
}

void SUsdStageTreeView::OnRemoveSchema(FName SchemaName)
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (UsdUtils::NotifyIfInstanceProxy(SelectedItem->UsdPrim))
		{
			return;
		}
	}

	FScopedTransaction Transaction(
		FText::Format(LOCTEXT("RemoveSchemaTransaction", "Remove the '{0}' schema from selected prims"), FText::FromName(SchemaName))
	);

	UE::FSdfChangeBlock Block;

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		SelectedItem->RemoveSchema(SchemaName);
	}
}

bool SUsdStageTreeView::CanApplySchema(FName SchemaName)
{
	if (!UsdStage || !UsdStage.IsEditTargetValid())
	{
		return false;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->CanApplySchema(SchemaName))
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::CanRemoveSchema(FName SchemaName)
{
	if (!UsdStage || !UsdStage.IsEditTargetValid())
	{
		return false;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->CanRemoveSchema(SchemaName))
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::CanAddChildPrim() const
{
	if (!UsdStage)
	{
		return false;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	// Allow adding a new top-level prim
	if (MySelectedItems.IsEmpty())
	{
		return true;
	}
	else
	{
		// We use the "rename" text input workflow to specify the target name,
		// so this doesn't work very well for multiple prims yet
		if (MySelectedItems.Num() > 1)
		{
			return false;
		}

		// If we have something selected it must be valid
		if (!MySelectedItems[0]->UsdPrim.IsValid())
		{
			return false;
		}
	}

	return true;
}

bool SUsdStageTreeView::CanPastePrim() const
{
	if (!UsdStage)
	{
		return false;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->UsdPrim || SelectedItem->UsdPrim.IsPseudoRoot())
		{
			// Don't let us paste as siblings of the pseudoroot as that makes no sense
			return false;
		}
	}

	return UsdUtils::CanPastePrims();
}

bool SUsdStageTreeView::DoesPrimExistOnStage() const
{
	if (!UsdStage || !UsdStage.IsEditTargetValid())
	{
		return false;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->UsdPrim.IsPseudoRoot() && SelectedItem->UsdPrim.IsValid())
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::DoesPrimExistOnEditTarget() const
{
	if (!UsdStage || !UsdStage.IsEditTargetValid())
	{
		return false;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->UsdPrim || SelectedItem->UsdPrim.IsPseudoRoot())
		{
			return false;
		}

		UE::FSdfPath SpecPath = UsdUtils::GetPrimSpecPathForLayer(SelectedItem->UsdPrim, UsdStage.GetEditTarget());
		if (!SpecPath.IsAbsoluteRootPath() && !SpecPath.IsEmpty())
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::DoesPrimHaveSpecOnLocalLayerStack() const
{
	for (FUsdPrimViewModelRef SelectedItem : GetSelectedItems())
	{
		if (!SelectedItem->UsdPrim || SelectedItem->UsdPrim.IsPseudoRoot())
		{
			return false;
		}

		if (SelectedItem->HasSpecsOnLocalLayer())
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::DoesPrimHaveReferenceSpecOnLocalLayerStack() const
{
	for (FUsdPrimViewModelRef SelectedItem : GetSelectedItems())
	{
		if (!SelectedItem->UsdPrim || SelectedItem->UsdPrim.IsPseudoRoot())
		{
			return false;
		}

		if (SelectedItem->HasReferencesOnLocalLayer())
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::DoesPrimHavePayloadSpecOnLocalLayerStack() const
{
	for (FUsdPrimViewModelRef SelectedItem : GetSelectedItems())
	{
		if (!SelectedItem->UsdPrim || SelectedItem->UsdPrim.IsPseudoRoot())
		{
			return false;
		}

		if (SelectedItem->HasPayloadsOnLocalLayer())
		{
			return true;
		}
	}

	return false;
}

bool SUsdStageTreeView::DoSelectedPrimsHaveCollapsingPreference(UsdUtils::ECollapsingPreference TargetPreference) const
{
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	if (MySelectedItems.Num() == 0)
	{
		return false;
	}

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (!SelectedItem->UsdPrim || SelectedItem->UsdPrim.IsPseudoRoot())
		{
			return false;
		}

		UsdUtils::ECollapsingPreference PrimPreference = UsdUtils::GetCollapsingPreference(SelectedItem->UsdPrim);
		if (PrimPreference != TargetPreference)
		{
			return false;
		}
	}

	return true;
}

void SUsdStageTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Restore expansion states.
	//
	// We do this on tick so that we only do at most one of these per frame, and also so that we do it as delayed
	// as possible, as during some busy transitions like undo/redo we may end up creating new FUsdPrimViewModels,
	// and we only want to try restoring these expansion states after all FUsdPrimViewModels have been created
	if (bNeedExpansionStateRefresh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SUsdStageTreeView::RestoreExpansionStates);

		// We should have only one root item, and it should be the expanded by default unless it was manually collapsed
		if (RootItems.Num() > 0)
		{
			const UE::FUsdPrim& RootPrim = RootItems[0]->UsdPrim;
			if (RootPrim.IsPseudoRoot())
			{
				const bool bShouldExpand = true;

				const bool bDefaultValue = true;
				if (RootWasExpanded.Get(bDefaultValue))
				{
					SetItemExpansion(RootItems[0], bShouldExpand);
				}
				else
				{
					SetItemExpansion(RootItems[0], !bShouldExpand);
				}
			}
		}

		TFunction<void(const FUsdPrimViewModelRef&)> SetExpansionRecursive = [&](const FUsdPrimViewModelRef& Item)
		{
			if (const UE::FUsdPrim& Prim = Item->UsdPrim)
			{
				if (ExpandedPrimPaths.Contains(Prim.GetPrimPath().GetString()))
				{
					const bool bShouldExpand = true;
					SetItemExpansion(Item, bShouldExpand);
				}
			}

			for (const FUsdPrimViewModelRef& Child : Item->Children)
			{
				SetExpansionRecursive(Child);
			}
		};

		for (const FUsdPrimViewModelRef& RootItem : RootItems)
		{
			SetExpansionRecursive(RootItem);
		}

		bNeedExpansionStateRefresh = false;
	}

	SUsdTreeView<FUsdPrimViewModelRef>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SUsdStageTreeView::RequestExpansionStateRestore()
{
	bNeedExpansionStateRefresh = true;
}

void SUsdStageTreeView::OnToggleAllPayloads(EPayloadsTrigger PayloadsTrigger)
{
	if (!UsdStage)
	{
		return;
	}

	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();

	// Ideally we'd just use a UE::FSdfChangeBlock here, but for whatever reason this doesn't seem to affect the
	// notices USD emits when loading/unloading prim payloads, so we must do this via the UsdStage directly

	TSet<UE::FSdfPath> PrimsToLoad;
	TSet<UE::FSdfPath> PrimsToUnload;

	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		if (SelectedItem->UsdPrim)
		{
			TFunction<void(FUsdPrimViewModelRef)> RecursiveTogglePayloads;
			RecursiveTogglePayloads = [&RecursiveTogglePayloads, PayloadsTrigger, &PrimsToLoad, &PrimsToUnload](FUsdPrimViewModelRef InSelectedItem
									  ) -> void
			{
				UE::FUsdPrim& UsdPrim = InSelectedItem->UsdPrim;

				if (UsdPrim.HasAuthoredPayloads())
				{
					bool bPrimIsLoaded = UsdPrim.IsLoaded();

					if (PayloadsTrigger == EPayloadsTrigger::Toggle)
					{
						if (bPrimIsLoaded)
						{
							PrimsToUnload.Add(UsdPrim.GetPrimPath());
						}
						else
						{
							PrimsToLoad.Add(UsdPrim.GetPrimPath());
						}
					}
					else if (PayloadsTrigger == EPayloadsTrigger::Load && !bPrimIsLoaded)
					{
						PrimsToLoad.Add(UsdPrim.GetPrimPath());
					}
					else if (PayloadsTrigger == EPayloadsTrigger::Unload && bPrimIsLoaded)
					{
						PrimsToUnload.Add(UsdPrim.GetPrimPath());
					}
				}
				else
				{
					for (FUsdPrimViewModelRef Child : InSelectedItem->UpdateChildren())
					{
						RecursiveTogglePayloads(Child);
					}
				}
			};

			RecursiveTogglePayloads(SelectedItem);
		}
	}

	if (PrimsToLoad.Num() + PrimsToUnload.Num() > 0)
	{
		UE::FSdfChangeBlock GroupNotices;
		UsdStage.LoadAndUnload(PrimsToLoad, PrimsToUnload);
	}
}

void SUsdStageTreeView::FillDuplicateSubmenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DuplicateFlattened_Text", "Flatten composed prim"),
		LOCTEXT("DuplicateFlattened_ToolTip", "Generate a flattened duplicate of the composed prim onto the current edit target"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnDuplicatePrim, EUsdDuplicateType::FlattenComposedPrim),
			FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnStage)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DuplicateSingle_Text", "Single layer specs"),
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
			[this]()
			{
				return DoesPrimExistOnEditTarget()
						   ? LOCTEXT("DuplicateSingleValid_ToolTip", "Duplicate the prim's specs on the current edit target only")
						   : UE::USDStageTreeView::Private::NoSpecOnLocalLayerStack;
			}
		)),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnDuplicatePrim, EUsdDuplicateType::SingleLayerSpecs),
			FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnEditTarget)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("DuplicateAllLocal_Text", "All local layer specs"),
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
			[this]()
			{
				return DoesPrimHaveSpecOnLocalLayerStack()
						   ? LOCTEXT("DuplicateAllLocalValid_ToolTip", "Duplicate each of the prim's specs across the entire stage")
						   : UE::USDStageTreeView::Private::NoSpecOnLocalLayerStack;
			}
		)),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnDuplicatePrim, EUsdDuplicateType::AllLocalLayerSpecs),
			FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimHaveSpecOnLocalLayerStack)
		),
		NAME_None,
		EUserInterfaceActionType::Button
	);
}

void SUsdStageTreeView::FillCollapsingSubmenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(
		LOCTEXT("AllowCollapse", "Allow collapsing"),
		LOCTEXT("AllowCollapse_ToolTip", "Allow this prim to be collapsed and to try collapsing its subtree, regardless of its kind"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnSetCollapsingPreference, UsdUtils::ECollapsingPreference::Allow),
			FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnStage),
			FIsActionChecked::CreateSP(this, &SUsdStageTreeView::DoSelectedPrimsHaveCollapsingPreference, UsdUtils::ECollapsingPreference::Allow)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CollapseOnKind", "Default"),
		LOCTEXT(
			"CollapseOnKind_ToolTip",
			"When 'Use prim kinds for collapsing' is enabled, prims are collapsed according to their kind (default). When disabled, prims won't be collapsed by default."
		),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnSetCollapsingPreference, UsdUtils::ECollapsingPreference::Default),
			FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnStage),
			FIsActionChecked::CreateSP(this, &SUsdStageTreeView::DoSelectedPrimsHaveCollapsingPreference, UsdUtils::ECollapsingPreference::Default)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("NeverCollapse", "Never collapse"),
		LOCTEXT("NeverCollapse_ToolTip", "Never collapse this prim, regardless of its kind"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnSetCollapsingPreference, UsdUtils::ECollapsingPreference::Never),
			FCanExecuteAction::CreateSP(this, &SUsdStageTreeView::DoesPrimExistOnStage),
			FIsActionChecked::CreateSP(this, &SUsdStageTreeView::DoSelectedPrimsHaveCollapsingPreference, UsdUtils::ECollapsingPreference::Never)
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);
}

void SUsdStageTreeView::FillAddSchemaSubmenu(FMenuBuilder& MenuBuilder)
{
	const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>();
	if (!ProjectSettings)
	{
		return;
	}

	static TArray<TSharedPtr<FString>> DefaultSchemas = {
		MakeShared<FString>(UsdToUnreal::ConvertToken(UnrealIdentifiers::ControlRigAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(UnrealIdentifiers::GroomAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(UnrealIdentifiers::GroomBindingAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(UnrealIdentifiers::LiveLinkAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdShadeTokens->MaterialBindingAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->PhysicsCollisionAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->PhysicsMeshCollisionAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(UnrealIdentifiers::SparseVolumeTextureAPI)),
		MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdSkelTokens->SkelBindingAPI)),
	};

	static TArray<TSharedPtr<FString>> AllowedKnownSchemas;
	AllowedKnownSchemas.Reset(DefaultSchemas.Num() + ProjectSettings->AdditionalCustomSchemaNames.Num());

	static TSet<FString> SeenSchemas;
	SeenSchemas.Reset();
	SeenSchemas.Reserve(AllowedKnownSchemas.Num());

	// Add default list of known schemas
	for (const TSharedPtr<FString>& KnownSchema : DefaultSchemas)
	{
		SeenSchemas.Add(*KnownSchema);

		// Only show on the list the schemas that can be applied to the selected prim
		if (CanApplySchema(**KnownSchema))
		{
			AllowedKnownSchemas.Add(KnownSchema);
		}
	}

	// Add additional list of custom schemas
	for (const FString& CustomSchema : ProjectSettings->AdditionalCustomSchemaNames)
	{
		if (SeenSchemas.Contains(CustomSchema))
		{
			continue;
		}
		SeenSchemas.Add(*CustomSchema);

		// Only show on the list the schemas that can be applied to the selected prim
		if (CanApplySchema(*CustomSchema))
		{
			AllowedKnownSchemas.Add(MakeShared<FString>(CustomSchema));
		}
	}

	// Sort list for a consistent order
	AllowedKnownSchemas.Sort(
		[](const TSharedPtr<FString>& LHS, const TSharedPtr<FString>& RHS)
		{
			return *LHS < *RHS;
		}
	);

	static TSharedPtr<FString> CurrentKnownSchema = nullptr;
	if (AllowedKnownSchemas.Num() > 0)
	{
		if (!CurrentKnownSchema || !CanApplySchema(**CurrentKnownSchema))
		{
			CurrentKnownSchema = AllowedKnownSchemas[0];
		}
	}
	else
	{
		CurrentKnownSchema = nullptr;
	}

	static FText ManuallyInputText;

	static TFunction<FText()> GetCurrentSchemaNameText = []() -> FText
	{
		return !ManuallyInputText.IsEmpty() ? ManuallyInputText : CurrentKnownSchema ? FText::FromString(*CurrentKnownSchema) : FText::GetEmpty();
	};

	// clang-format off
	TSharedRef<SHorizontalBox> Box =
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(8.0f, 0.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&AllowedKnownSchemas)
			.OnGenerateWidget_Lambda([&](TSharedPtr<FString> Option)
			{
				TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
				if (Option)
				{
					Widget = SNew(STextBlock)
						.Text(FText::FromString(*Option))
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
				}

				return Widget.ToSharedRef();
			})
			.OnSelectionChanged_Lambda([](TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo)
			{
				CurrentKnownSchema = ChosenOption;
				ManuallyInputText = FText::GetEmpty();
			})
			[
				SNew(SEditableTextBox)
				.Text_Lambda([]() -> FText
				{
					return GetCurrentSchemaNameText();
				})
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.OnTextChanged_Lambda([](const FText& NewText)
				{
					ManuallyInputText = NewText;
				})
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(8.0f, 0.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.Text(LOCTEXT("AddSchemaButtonText", "Add"))
			.ToolTipText(LOCTEXT("AddSchemaButtonToolTip", "Adds the currently selected schema to the prim"))
			.IsEnabled_Lambda([this]()->bool
			{
				FName SelectedSchema = *GetCurrentSchemaNameText().ToString();
				return CanApplySchema(SelectedSchema);
			})
			.OnClicked_Lambda([this]() -> FReply
			{
				FText SelectedSchemaText = GetCurrentSchemaNameText();
				FString SelectedSchemaString = SelectedSchemaText.ToString();
				FName SelectedSchemaName = *SelectedSchemaString;

				if (CanApplySchema(SelectedSchemaName))
				{
					OnApplySchema(SelectedSchemaName);
				}

				if (!SeenSchemas.Contains(SelectedSchemaString))
				{
					UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>();
					if (ProjectSettings)
					{
						ProjectSettings->AdditionalCustomSchemaNames.AddUnique(SelectedSchemaText.ToString());
						ProjectSettings->SaveConfig();
					}
				}

				return FReply::Handled();
			})
			.ButtonStyle(&FAppStyle::Get(), "PrimaryButton")
		];
	// clang-format on

	const bool bNoIndent = true;
	MenuBuilder.AddWidget(Box, FText::GetEmpty(), bNoIndent);
}

void SUsdStageTreeView::FillRemoveSchemaSubmenu(FMenuBuilder& MenuBuilder)
{
	TSet<FString> RemovableSchemas;
	TArray<FUsdPrimViewModelRef> MySelectedItems = GetSelectedItems();
	for (FUsdPrimViewModelRef SelectedItem : MySelectedItems)
	{
		TArray<FName> Schemas = SelectedItem->UsdPrim.GetAppliedSchemas();
		for (const FName& Schema : Schemas)
		{
			RemovableSchemas.Add(Schema.ToString());
		}
	}

	TArray<FString> SortedRemovableSchemas = RemovableSchemas.Array();
	SortedRemovableSchemas.Sort();

	for (const FString& Schema : SortedRemovableSchemas)
	{
		if (!SUsdStageTreeView::CanRemoveSchema(*Schema))
		{
			continue;
		}

		MenuBuilder.AddMenuEntry(
			FText::FromString(Schema),
			FText::Format(LOCTEXT("RemoveSchemaToolTip", "Remove schema '{0}'"), FText::FromString(Schema)),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SUsdStageTreeView::OnRemoveSchema, FName(*Schema))),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
}

FReply SUsdStageTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SUsdTreeView::OnKeyDown(MyGeometry, InKeyEvent);
}

void SUsdStageTreeView::ScrollItemIntoView(FUsdPrimViewModelRef TreeItem)
{
	FUsdPrimViewModel* Parent = TreeItem->ParentItem;
	while (Parent)
	{
		SetItemExpansion(StaticCastSharedRef<FUsdPrimViewModel>(Parent->AsShared()), true);
		Parent = Parent->ParentItem;
	}

	RequestScrollIntoView(TreeItem);
}

void SUsdStageTreeView::OnTreeItemScrolledIntoView(FUsdPrimViewModelRef TreeItem, const TSharedPtr<ITableRow>& Widget)
{
	if (TreeItem == PendingRenameItem.Pin())
	{
		PendingRenameItem = nullptr;
		TreeItem->RenameRequestEvent.ExecuteIfBound();
	}
}

void SUsdStageTreeView::OnPrimNameCommitted(const FUsdPrimViewModelRef& ViewModel, const FText& InPrimName)
{
	// Reset this regardless of how we exit this function
	const bool bRenamingExistingPrim = ViewModel->bIsRenamingExistingPrim;
	ViewModel->bIsRenamingExistingPrim = false;

	// Escaped out of initially setting a prim name
	TFunction<void()> CancelInput = [&ViewModel, this]()
	{
		if (!ViewModel->UsdPrim)
		{
			if (FUsdPrimViewModel* Parent = ViewModel->ParentItem)
			{
				ViewModel->ParentItem->Children.Remove(ViewModel);
			}
			else
			{
				RootItems.Remove(ViewModel);
			}

			RequestTreeRefresh();
		}
	};

	if (InPrimName.IsEmptyOrWhitespace())
	{
		CancelInput();
		return;
	}

	if (bRenamingExistingPrim)
	{
		if (UsdUtils::NotifyIfInstanceProxy(ViewModel->UsdPrim))
		{
			CancelInput();
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("RenamePrimTransaction", "Rename a prim"));

		// e.g. "/Root/OldPrim"
		FString OldPath = ViewModel->UsdPrim.GetPrimPath().GetString();

		// e.g. "NewPrim"
		FString NewNameStr = InPrimName.ToString();

		// Preemptively preserve the prim's expansion state because RenamePrim will trigger notices from within itself
		// that will trigger refreshes of the tree view.
		//
		// Note: We don't remove the old paths in here, as that lets us undo the rename and
		// preserve our expansion states
		{
			// e.g. "/Root/NewPrim"
			FString NewPath = FString::Printf(TEXT("%s/%s"), *FPaths::GetPath(OldPath), *NewNameStr);
			UsdUtils::FCaseSensitiveStringSet EntriesToAdd;
			for (UsdUtils::FCaseSensitiveStringSet::TIterator It{ExpandedPrimPaths}; It; ++It)
			{
				// e.g. "/Root/OldPrim/SomeChild"
				TStringView SomePrimPath = *It;
				if (SomePrimPath.StartsWith(OldPath))
				{
					// e.g. "/SomeChild"
					SomePrimPath.RemovePrefix(OldPath.Len());

					// e.g. "/Root/NewPrim/SomeChild"
					EntriesToAdd.Add(NewPath + FString{SomePrimPath});
				}
			}
			ExpandedPrimPaths.Append(EntriesToAdd);
		}

		UsdUtils::RenamePrim(ViewModel->UsdPrim, *NewNameStr);
	}
	else
	{
		if (FUsdPrimViewModel* ParentModel = ViewModel->ParentItem)
		{
			if (UsdUtils::NotifyIfInstanceProxy(ParentModel->UsdPrim))
			{
				CancelInput();
				return;
			}
		}

		FScopedTransaction Transaction(LOCTEXT("AddPrimTransaction", "Add a new prim"));

		ViewModel->DefinePrim(*InPrimName.ToString());

		const bool bResync = true;

		// Renamed a child item
		if (ViewModel->ParentItem)
		{
			ViewModel->ParentItem->Children.Remove(ViewModel);

			RefreshPrim(ViewModel->ParentItem->UsdPrim.GetPrimPath().GetString(), bResync);
		}
		// Renamed a root item
		else
		{
			RefreshPrim(ViewModel->UsdPrim.GetPrimPath().GetString(), bResync);
		}
	}
}

void SUsdStageTreeView::OnPrimNameUpdated(const FUsdPrimViewModelRef& TreeItem, const FText& InPrimName, FText& ErrorMessage)
{
	FString NameStr = InPrimName.ToString();
	IUsdPrim::IsValidPrimName(NameStr, ErrorMessage);
	if (!ErrorMessage.IsEmpty())
	{
		return;
	}

	{
		const UE::FUsdStageWeak& Stage = TreeItem->UsdStage;
		if (!Stage)
		{
			return;
		}

		UE::FSdfPath ParentPrimPath;
		if (TreeItem->ParentItem)
		{
			ParentPrimPath = TreeItem->ParentItem->UsdPrim.GetPrimPath();
		}
		else
		{
			ParentPrimPath = UE::FSdfPath::AbsoluteRootPath();
		}

		UE::FSdfPath NewPrimPath = ParentPrimPath.AppendChild(*NameStr);
		const UE::FUsdPrim& Prim = Stage.GetPrimAtPath(NewPrimPath);
		if (Prim && Prim != TreeItem->UsdPrim)
		{
			ErrorMessage = LOCTEXT("DuplicatePrimName", "A Prim with this name already exists!");
			return;
		}
	}
}

#endif	  // #if USE_USD_SDK

#undef LOCTEXT_NAMESPACE
