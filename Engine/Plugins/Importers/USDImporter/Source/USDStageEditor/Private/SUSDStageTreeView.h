// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDPrimViewModel.h"
#include "USDSetUtils.h"
#include "UsdWrappers/UsdStage.h"
#include "Widgets/SUSDTreeView.h"

class AUsdStageActor;
class FUICommandList;
enum class EPayloadsTrigger;
enum class EUsdDuplicateType : uint8;
namespace UE
{
	class FUsdPrim;
}
namespace UsdUtils
{
	enum class ECollapsingPreference : uint8;
}

#if USE_USD_SDK

DECLARE_DELEGATE_OneParam(FOnPrimSelectionChanged, const TArray<FString>& /* NewSelection */);
DECLARE_DELEGATE_OneParam(FOnAddPrim, FString);

class SUsdStageTreeView : public SUsdTreeView<FUsdPrimViewModelRef>
{
public:
	SLATE_BEGIN_ARGS(SUsdStageTreeView)
	{
	}
	SLATE_EVENT(FOnPrimSelectionChanged, OnPrimSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SUsdStageTreeView() override;

	void Refresh(const UE::FUsdStageWeak& NewStage);
	void RefreshPrim(const FString& PrimPath, bool bResync);

	FUsdPrimViewModelPtr GetItemFromPrimPath(const FString& PrimPath);

	void SetSelectedPrimPaths(const TArray<FString>& PrimPaths);
	void SetSelectedPrims(const TArray<UE::FUsdPrim>& Prims);
	TArray<FString> GetSelectedPrimPaths();
	TArray<UE::FUsdPrim> GetSelectedPrims();

private:
	virtual TSharedRef<ITableRow> OnGenerateRow(FUsdPrimViewModelRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable) override;
	virtual void OnGetChildren(FUsdPrimViewModelRef InParent, TArray<FUsdPrimViewModelRef>& OutChildren) const override;

	// Required so that we can use the cut/copy/paste/etc. shortcuts
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void ScrollItemIntoView(FUsdPrimViewModelRef TreeItem);
	virtual void OnTreeItemScrolledIntoView(FUsdPrimViewModelRef TreeItem, const TSharedPtr<ITableRow>& Widget) override;

	void OnPrimNameCommitted(const FUsdPrimViewModelRef& TreeItem, const FText& InPrimName);
	void OnPrimNameUpdated(const FUsdPrimViewModelRef& TreeItem, const FText& InPrimName, FText& ErrorMessage);

	virtual void SetupColumns() override;
	TSharedPtr<SWidget> ConstructPrimContextMenu();

	void OnToggleAllPayloads(EPayloadsTrigger PayloadsTrigger);

	void FillDuplicateSubmenu(FMenuBuilder& MenuBuilder);
	void FillCollapsingSubmenu(FMenuBuilder& MenuBuilder);
	void FillAddSchemaSubmenu(FMenuBuilder& MenuBuilder);
	void FillRemoveSchemaSubmenu(FMenuBuilder& MenuBuilder);

	void OnAddChildPrim();
	void OnCutPrim();
	void OnCopyPrim();
	void OnPastePrim();
	void OnDuplicatePrim(EUsdDuplicateType DuplicateType);
	void OnDeletePrim();
	void OnRenamePrim();
	void OnSetCollapsingPreference(UsdUtils::ECollapsingPreference Preference);

	void OnAddReference();
	void OnClearReferences();

	void OnAddPayload();
	void OnClearPayloads();

	void OnApplySchema(FName SchemaName);
	void OnRemoveSchema(FName SchemaName);
	bool CanApplySchema(FName SchemaName);
	bool CanRemoveSchema(FName SchemaName);

	bool CanAddChildPrim() const;
	bool CanPastePrim() const;
	bool DoesPrimExistOnStage() const;
	bool DoesPrimExistOnEditTarget() const;
	bool DoesPrimHaveSpecOnLocalLayerStack() const;
	bool DoesPrimHaveReferenceSpecOnLocalLayerStack() const;
	bool DoesPrimHavePayloadSpecOnLocalLayerStack() const;
	bool DoSelectedPrimsHaveCollapsingPreference(UsdUtils::ECollapsingPreference Preference) const;

	void RequestExpansionStateRestore();
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	void SelectItemsInternal(const TArray<FUsdPrimViewModelRef>& ItemsToSelect);

public:
	// We update only on slate tick, but can receive a USD notice at any point, from any thread.
	// These variables are used to control what needs to be refreshed on the next tick

	FString PrimPathToSelect;

	bool bNeedsFullUpdate = false;

	// We append requests to refresh prims on the stage tree view to the end of this array as they arrive,
	// and then perform them all at once during OnSlateTick.
	// Elements map from prim path to whether the update neeeds to be a resync or not.
	TArray<TPair<FString, bool>> PrimsToRefresh;

	// Use a lock because some of the notices that will update these refresh variables can be
	// sent from TBB threads
	mutable FRWLock RefreshStateLock;

private:
	// Should always be valid, we keep the one we're given on Refresh()
	UE::FUsdStageWeak UsdStage;

	TWeakPtr<FUsdPrimViewModel> PendingRenameItem;

	// So that we can store these across refreshes
	UsdUtils::FCaseSensitiveStringSet ExpandedPrimPaths;
	TOptional<bool> RootWasExpanded;
	bool bNeedExpansionStateRefresh = false;

	FOnPrimSelectionChanged OnPrimSelectionChanged;

	FDelegateHandle PostUndoRedoHandle;

	TSharedPtr<FUICommandList> UICommandList;
};

#endif	  // #if USE_USD_SDK
