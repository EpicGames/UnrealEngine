// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/IUSDTreeViewItem.h"

enum class EUsdDragDropOpType
{
	None,
	Prims,
	Layers,
	Attributes
};

class FUsdDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FUsdDragDropOp, FDecoratedDragDropOp)

	EUsdDragDropOpType OpType;
	TSet<TSharedRef<IUsdTreeViewItem>> DraggedItems;
};
