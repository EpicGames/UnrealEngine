// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageEditorBlueprintLibrary.h"

#include "USDStageActor.h"
#include "USDStageEditorModule.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdPrim.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageEditorBlueprintLibrary)

bool UUsdStageEditorBlueprintLibrary::OpenStageEditor()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.OpenStageEditor();
}

bool UUsdStageEditorBlueprintLibrary::CloseStageEditor()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.CloseStageEditor();
}

bool UUsdStageEditorBlueprintLibrary::IsStageEditorOpened()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.IsStageEditorOpened();
}

AUsdStageActor* UUsdStageEditorBlueprintLibrary::GetAttachedStageActor()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.GetAttachedStageActor();
}

bool UUsdStageEditorBlueprintLibrary::SetAttachedStageActor(AUsdStageActor* NewActor)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.SetAttachedStageActor(NewActor);
}

TArray<FString> UUsdStageEditorBlueprintLibrary::GetSelectedLayerIdentifiers()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	TArray<UE::FSdfLayer> Layers = StageEditorModule.GetSelectedLayers();

	TArray<FString> LayerIdentifiers;
	LayerIdentifiers.Reserve(Layers.Num());

	for (const UE::FSdfLayer& Layer : Layers)
	{
		LayerIdentifiers.Add(Layer.GetIdentifier());
	}

	return LayerIdentifiers;
}

void UUsdStageEditorBlueprintLibrary::SetSelectedLayerIdentifiers(const TArray<FString>& NewSelection)
{
	TArray<UE::FSdfLayer> Layers;
	for (const FString& Identifier : NewSelection)
	{
		Layers.Add(UE::FSdfLayer::FindOrOpen(*Identifier));
	}

	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.SetSelectedLayers(Layers);
}

TArray<FString> UUsdStageEditorBlueprintLibrary::GetSelectedPrimPaths()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	TArray<UE::FUsdPrim> Prims = StageEditorModule.GetSelectedPrims();

	TArray<FString> PrimPaths;
	PrimPaths.Reserve(Prims.Num());

	for (const UE::FUsdPrim& Prim : Prims)
	{
		PrimPaths.Add(Prim.GetPrimPath().GetString());
	}

	return PrimPaths;
}

void UUsdStageEditorBlueprintLibrary::SetSelectedPrimPaths(const TArray<FString>& NewSelection)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	const AUsdStageActor* StageActor = StageEditorModule.GetAttachedStageActor();
	if (!StageActor)
	{
		return;
	}

	UE::FUsdStage Stage = StageActor->GetUsdStage();
	if (!Stage)
	{
		return;
	}

	TArray<UE::FUsdPrim> Prims;
	Prims.Reserve(NewSelection.Num());

	for (const FString& PrimPath : NewSelection)
	{
		if (UE::FUsdPrim SelectedPrim = Stage.GetPrimAtPath(UE::FSdfPath{*PrimPath}))
		{
			Prims.Add(SelectedPrim);
		}
	}

	StageEditorModule.SetSelectedPrims(Prims);
}

TArray<FString> UUsdStageEditorBlueprintLibrary::GetSelectedPropertyNames()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.GetSelectedPropertyNames();
}

void UUsdStageEditorBlueprintLibrary::SetSelectedPropertyNames(const TArray<FString>& NewSelection)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.SetSelectedPropertyNames(NewSelection);
}

TArray<FString> UUsdStageEditorBlueprintLibrary::GetSelectedPropertyMetadataNames()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	return StageEditorModule.GetSelectedPropertyMetadataNames();
}

void UUsdStageEditorBlueprintLibrary::SetSelectedPropertyMetadataNames(const TArray<FString>& NewSelection)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.SetSelectedPropertyMetadataNames(NewSelection);
}

void UUsdStageEditorBlueprintLibrary::FileNew()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileNew();
}

void UUsdStageEditorBlueprintLibrary::FileOpen(const FString& FilePath)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileOpen(FilePath);
}

void UUsdStageEditorBlueprintLibrary::FileSave(const FString& OutputFilePathIfUnsaved)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileSave(OutputFilePathIfUnsaved);
}

void UUsdStageEditorBlueprintLibrary::FileExportAllLayers(const FString& OutputDirectory)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileExportAllLayers(OutputDirectory);
}

void UUsdStageEditorBlueprintLibrary::FileExportFlattenedStage(const FString& OutputLayer)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileExportFlattenedStage(OutputLayer);
}

void UUsdStageEditorBlueprintLibrary::FileExportFlattenedLayerStack(const FString& OutputLayer)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileExportFlattenedLayerStack(OutputLayer);
}

void UUsdStageEditorBlueprintLibrary::FileReload()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileReload();
}

void UUsdStageEditorBlueprintLibrary::FileReset()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileReset();
}

void UUsdStageEditorBlueprintLibrary::FileClose()
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.FileClose();
}

void UUsdStageEditorBlueprintLibrary::ActionsImport(const FString& OutputContentFolder, UUsdStageImportOptions* Options)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.ActionsImport(OutputContentFolder, Options);
}

void UUsdStageEditorBlueprintLibrary::ExportSelectedLayers(const FString& OutputDirectory)
{
	IUsdStageEditorModule& StageEditorModule = FModuleManager::GetModuleChecked<IUsdStageEditorModule>("USDStageEditor");
	StageEditorModule.ExportSelectedLayers(OutputDirectory);
}
