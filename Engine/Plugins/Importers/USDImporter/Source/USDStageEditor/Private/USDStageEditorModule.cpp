// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageEditorModule.h"

#include "SUSDSaveDialog.h"
#include "SUSDStage.h"
#include "SUSDStageEditorStyle.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDProjectSettings.h"
#include "USDStageActor.h"
#include "USDUtilitiesModule.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdPrim.h"

#include "Editor/TransBuffer.h"
#include "Editor/UnrealEdEngine.h"
#include "EngineUtils.h"
#include "Framework/Docking/TabManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "UnrealEdGlobals.h"
#include "UObject/ObjectSaveContext.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "UsdStageEditorModule"

namespace UE::UsdStageEditorModule::Private
{
	const static FTabId UsdStageEditorTabID{TEXT("USDStage")};

	void SaveStageActorLayersForWorld(UWorld* World, bool bForClosing, AUsdStageActor* TargetStageActor = nullptr)
	{
#if USE_USD_SDK
		if (!World)
		{
			return;
		}

		UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>();
		if (!Settings)
		{
			return;
		}

		// Reentrant guard here because if we ever save an anonymous layer we'll update the stage actors that use it
		// to point to the new (saved) layer, which will internally close the anonymous stage and get us back in here
		static bool bIsReentrant = false;
		if (bIsReentrant)
		{
			return;
		}
		TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

		bool bPrompt = false;
		switch (bForClosing ? Settings->ShowSaveLayersDialogWhenClosing : Settings->ShowSaveLayersDialogWhenSaving)
		{
			case EUsdSaveDialogBehavior::NeverSave:
			{
				// Don't even do anything if we're not going to save anyway
				return;
			}
			case EUsdSaveDialogBehavior::AlwaysSave:
			{
				break;
			}
			default:
			case EUsdSaveDialogBehavior::ShowPrompt:
			{
				bPrompt = true;
				break;
			}
		}

		TArray<AUsdStageActor*> StageActorsToVisit;
		if (TargetStageActor)
		{
			if (TargetStageActor->GetWorld() == World)
			{
				StageActorsToVisit.Add(TargetStageActor);
			}
		}
		else
		{
			for (TActorIterator<AUsdStageActor> It{World}; It; ++It)
			{
				StageActorsToVisit.Add(*It);
			}
		}

		// For now lets only care about stages opened on stage actors. The user could have additional stages,
		// like opened via Python or custom C++ plugins, but lets ignore those
		TMap<FString, FUsdSaveDialogRowData> RowsByIdentifier;
		for (AUsdStageActor* StageActor : StageActorsToVisit)
		{
			if (!StageActor)
			{
				continue;
			}

			UE::FUsdStage UsdStage = static_cast<const AUsdStageActor*>(StageActor)->GetUsdStage();
			if (!UsdStage)
			{
				continue;
			}

			TArray<UE::FSdfLayer> UsedLayers = UsdStage.GetUsedLayers();
			RowsByIdentifier.Reserve(RowsByIdentifier.Num() + UsedLayers.Num());

			for (const UE::FSdfLayer& UsedLayer : UsedLayers)
			{
				// This comment is written to the layer when we're in the process of saving a memory-only
				// stage, and indicates that this layer is already saved (even though it will show as dirty and
				// anonymous)
				if (UsedLayer.IsDirty() && UsedLayer.GetComment() != UnrealIdentifiers::LayerSavedComment)
				{
					FUsdSaveDialogRowData& RowData = RowsByIdentifier.FindOrAdd(UsedLayer.GetIdentifier());
					RowData.Layer = UsedLayer;
					RowData.ConsumerStages.AddUnique(UsdStage);
					RowData.ConsumerActors.Add(StageActor);
				}
			}
		}

		if (RowsByIdentifier.Num() == 0)
		{
			return;
		}

		TArray<FUsdSaveDialogRowData> Rows;
		RowsByIdentifier.GenerateValueArray(Rows);

		if (bPrompt)
		{
			Rows.Sort(
				[](const FUsdSaveDialogRowData& Left, const FUsdSaveDialogRowData& Right)
				{
					return (Left.Layer && Right.Layer) ? Left.Layer.GetIdentifier() < Right.Layer.GetIdentifier()
													   // This shouldn't ever happen but just do something consistent here instead anyway
													   : Left.ConsumerStages.Num() < Right.ConsumerStages.Num();
				}
			);

			// clang-format off
			const FText WindowTitle = LOCTEXT("SaveDialogTitle", "Save USD Layers");
			const FText DescriptionText = bForClosing
				? LOCTEXT("CloseDialogDescTextText", "Before closing these USD Stages, do you want to save these USD layers to disk?")
				: LOCTEXT("SaveDialogDescTextText", "Since you're saving the Level, do you want to save these USD layers to disk?");
			// clang-format on

			bool bShouldSave = true;
			bool bShouldPromptAgain = true;
			Rows = SUsdSaveDialog::ShowDialog(Rows, WindowTitle, DescriptionText, &bShouldSave, &bShouldPromptAgain);

			EUsdSaveDialogBehavior* bSetting = bForClosing ? &Settings->ShowSaveLayersDialogWhenClosing : &Settings->ShowSaveLayersDialogWhenSaving;

			*bSetting = bShouldPromptAgain ? EUsdSaveDialogBehavior::ShowPrompt
						: bShouldSave	   ? EUsdSaveDialogBehavior::AlwaysSave
										   : EUsdSaveDialogBehavior::NeverSave;

			Settings->SaveConfig();
		}

		for (const FUsdSaveDialogRowData& ReturnedRow : Rows)
		{
			UE::FSdfLayer PinnedLayer = ReturnedRow.Layer;
			if (ReturnedRow.bSaveLayer && PinnedLayer)
			{
				bool bSaved = false;

				if (PinnedLayer.IsAnonymous())
				{
					// For now we only allow anonymous stages, and not individual layers, so we don't have to
					// patch up anything
					TOptional<FString> UsdFilePath = UsdUtils::BrowseUsdFile(UsdUtils::EBrowseFileMode::Save);
					if (UsdFilePath)
					{
						bSaved = PinnedLayer.Export(*UsdFilePath.GetValue());

						// If any stage actors were pointing at the in-memory versions of these stages, update them
						// to point to the saved versions
						if (bSaved && !bForClosing)
						{
							// Even though we're potentially going to load actors and assets here, we don't need to
							// use a scoped transaction as the Save All command already clears the transaction buffer
							// anyway, and we won't get in here when closing.

							FString ExpectedIdentifier = UnrealIdentifiers::IdentifierPrefix + PinnedLayer.GetIdentifier();
							for (AUsdStageActor* StageActor : ReturnedRow.ConsumerActors)
							{
								if (StageActor->RootLayer.FilePath == ExpectedIdentifier
									&& static_cast<const AUsdStageActor*>(StageActor)->GetUsdStage())
								{
									StageActor->SetRootLayer(*UsdFilePath.GetValue());
								}
							}
						}
					}
				}
				else
				{
					const bool bForce = true;
					bSaved = PinnedLayer.Save(bForce);
				}

				if (!bSaved)
				{
					USD_LOG_WARNING(TEXT("Failed to save layer '%s'"), *PinnedLayer.GetIdentifier());
				}
			}
		}
#endif	  // USE_USD_SDK
	}

	bool ShowReferenceHandlingDialog(const FText& DialogText, EReferencerTypeHandling& OutChosenHandling)
	{
		UUsdProjectSettings* Settings = GetMutableDefault<UUsdProjectSettings>();
		if (!Settings)
		{
			return false;
		}

		const FText Title = LOCTEXT("MismatchedTypeNamesText", "USD: Mismatched type names");
		USD_LOG_USERWARNING(FText::FromString(DialogText.ToString().Replace(TEXT("\n\n"), TEXT(" "))));

		FScopedUnrealAllocs UEAllocs;

		bool bDontPromptAgain = false;
		TOptional<EReferencerTypeHandling> ChosenHandling;

		// clang-format off
			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(Title)
				.ClientSize(FVector2D{600, 200})
				.SizingRule(ESizingRule::FixedSize);

			TSharedPtr< SHorizontalBox > ButtonsBox = SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				.HAlign(HAlign_Left)
				.Padding(2, 0)
				[
					SNew(SCheckBox)
					.IsChecked(bDontPromptAgain) // If we're showing the prompt we know we have this unchecked
					.OnCheckStateChanged_Lambda([&bDontPromptAgain](ECheckBoxState NewState)
					{
						bDontPromptAgain = NewState == ECheckBoxState::Checked;
					})
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DontPromptAgainText", "Don't prompt again"))
						.AutoWrapText(true)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get(), "PrimaryButton")
					.TextStyle(&FAppStyle::Get(), "PrimaryButtonText")
					.Text(LOCTEXT("ClearReferencerTypeText", "Clear referencer"))
					.ToolTipText(LOCTEXT("ClearReferencerTypeSubText", "Remove the authored type opinion from the referencer prim, turning it into a typeless prim. This will let the type name opinion from the target prim dictate the type of the composed prim."))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([&ChosenHandling, &Window]() -> FReply
					{
						ChosenHandling = EReferencerTypeHandling::ClearReferencerType;
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get(), "Button")
					.TextStyle(&FAppStyle::Get(), "ButtonText")
					.Text(LOCTEXT("MatchReferencedTypeText", "Match referenced"))
					.ToolTipText(LOCTEXT("MatchReferencedTypeSubText", "Force the referencer prim to have the same type as the referenced prim."))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([&ChosenHandling, &Window]() -> FReply
					{
						ChosenHandling = EReferencerTypeHandling::MatchReferencedType;
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2, 0)
				[
					SNew(SButton)
					.ButtonStyle(&FAppStyle::Get(), "Button")
					.TextStyle(&FAppStyle::Get(), "ButtonText")
					.Text(LOCTEXT("IgnoreText", "Ignore"))
					.ToolTipText(LOCTEXT("IgnoreSubText", "Add the reference or payload anyway, ignoring the difference in type names"))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.OnClicked_Lambda([&ChosenHandling, &Window]() -> FReply
					{
						ChosenHandling = EReferencerTypeHandling::Ignore;
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
				];

			TSharedRef<SBorder> Border = SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(16))
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(0.0f,0.0f,0.0f,8.0f)
				[
					SNew(STextBlock)
					.Text(DialogText)
					.AutoWrapText(true)
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 16.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Bottom)
				[
					ButtonsBox.ToSharedRef()
				]
			];
		// clang-format on

		Window->SetContent(Border);
		Window->SetWidgetToFocusOnActivate(Border);

		GEditor->EditorAddModalWindow(Window);

		if (ChosenHandling.IsSet())
		{
			OutChosenHandling = ChosenHandling.GetValue();
			ensure(OutChosenHandling != EReferencerTypeHandling::ShowPrompt);	 // We only set chosen handling when pressing a button

			if (bDontPromptAgain)
			{
				Settings->ReferencerTypeHandling = ChosenHandling.GetValue();
				Settings->SaveConfig();
			}

			return true;
		}

		return false;
	}

#if USE_USD_SDK
	TSharedPtr<SUsdStage> GetUsdStageEditor(bool bOpenIfNeeded = true)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		TSharedPtr<SDockTab> Tab = bOpenIfNeeded ? LevelEditorTabManager->TryInvokeTab(UsdStageEditorTabID)
												 : LevelEditorTabManager->FindExistingLiveTab(UsdStageEditorTabID);

		if (Tab)
		{
			if (TSharedPtr<SBorder> ContentBorder = StaticCastSharedRef<SBorder>(Tab->GetContent()))
			{
				if (TSharedPtr<SUsdStage> UsdStageEditor = StaticCastSharedRef<SUsdStage>(ContentBorder->GetContent()))
				{
					return UsdStageEditor;
				}
			}
		}

		return nullptr;
	}
#endif	  // USE_USD_SDK
}	 // namespace UE::UsdStageEditorModule::Private

class FUsdStageEditorModule : public IUsdStageEditorModule
{
public:
#if USE_USD_SDK
	static TSharedRef<SDockTab> SpawnUsdStageTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		LLM_SCOPE_BYTAG(Usd);

		// clang-format off
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			.Label(LOCTEXT("USDStageEditorTab", "USD Stage Editor"))
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SUsdStage)
				]
			];
		// clang-format on
	}

	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Usd);

		FUsdStageEditorStyle::Initialize();

		IUsdUtilitiesModule& UtilitiesModule = FModuleManager::Get().LoadModuleChecked<IUsdUtilitiesModule>(TEXT("UsdUtilities"));
		UtilitiesModule.OnReferenceHandlingDialog.BindStatic(UE::UsdStageEditorModule::Private::ShowReferenceHandlingDialog);

		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda(
			[]()
			{
				FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
				TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

				const FSlateIcon LayersIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.USDStage");

				// clang-format off
				LevelEditorTabManager->RegisterTabSpawner(
					UE::UsdStageEditorModule::Private::UsdStageEditorTabID.TabType,
					FOnSpawnTab::CreateStatic(&FUsdStageEditorModule::SpawnUsdStageTab)
				)
				.SetDisplayName(LOCTEXT("USDStageEditorMenuItem", "USD Stage Editor"))
				.SetTooltipText(LOCTEXT("USDStageEditorTooltip", "Open the USD Stage Editor tab. Use this to open and manage USD Stages without importing."))
				.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory())
				.SetIcon(LayersIcon);
				// clang-format on
			}
		);

		// Prompt to save modified USD layers when closing the editor
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		EditorCanCloseDelegate = MainFrame.RegisterCanCloseEditor(IMainFrameModule::FMainFrameCanCloseEditor::CreateLambda(
			[]() -> bool
			{
				if (GEditor && GEngine)
				{
					UWorld* EditorWorld = nullptr;
					for (const FWorldContext& Context : GEngine->GetWorldContexts())
					{
						if (Context.WorldType == EWorldType::Editor)
						{
							EditorWorld = Context.World();
						}
					}

					const bool bForClosing = true;
					UE::UsdStageEditorModule::Private::SaveStageActorLayersForWorld(EditorWorld, bForClosing);
				}

				// We won't actually ever block the save
				return true;
			}
		));

		// Prompt to save modified USD Layers when closing stageactor stages
		StageActorLoadedHandle = AUsdStageActor::OnActorLoaded.AddLambda(
			[this](AUsdStageActor* StageActor)
			{
				if (!StageActor)
				{
					return;
				}

				// We never want to prompt when undoing or redoing.
				// We have to subscribe to this here as the UTransBuffer doesn't exist by the time the module is
				// initializing
				if (UTransBuffer* TransBuffer = GUnrealEd ? Cast<UTransBuffer>(GUnrealEd->Trans) : nullptr)
				{
					if (!OnTransactionStateChangedHandle.IsValid())
					{
						OnTransactionStateChangedHandle = TransBuffer->OnTransactionStateChanged().AddLambda(
							[this](const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState)
							{
								if (InTransactionState == ETransactionStateEventType::UndoRedoStarted)
								{
									bUndoRedoing = true;
								}
								else if (InTransactionState == ETransactionStateEventType::UndoRedoFinalized)
								{
									bUndoRedoing = false;
								}
							}
						);
					}
				}

				TWeakObjectPtr<AUsdStageActor> WeakActor = StageActor;
				StageActor->OnPreStageChanged.AddLambda(
					[this, WeakActor]()
					{
						AUsdStageActor* StageActor = WeakActor.Get();
						if (!bUndoRedoing && StageActor && static_cast<const AUsdStageActor*>(StageActor)->GetUsdStage())
						{
							const bool bForClosing = true;
							UE::UsdStageEditorModule::Private::SaveStageActorLayersForWorld(StageActor->GetWorld(), bForClosing, StageActor);
						}
					}
				);
			}
		);

		// Prompt to save modified USD layers when saving the world
		PreSaveWorldEditorDelegateHandle = FEditorDelegates::PreSaveWorldWithContext.AddLambda(
			[](UWorld* World, FObjectPreSaveContext InContext)
			{
				// Detect if we should actually do anything (check for autosaves, cooking, etc.)
				if (InContext.IsFromAutoSave() || InContext.IsProceduralSave())
				{
					return;
				}

				const bool bForClosing = false;
				UE::UsdStageEditorModule::Private::SaveStageActorLayersForWorld(World, bForClosing);
			}
		);

		OpenStageEditorClickedHandle = AUsdStageActor::OnOpenStageEditorClicked.AddLambda(
			[](AUsdStageActor* StageActor)
			{
				FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
				TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
				if (TSharedPtr<SDockTab> Tab = LevelEditorTabManager->TryInvokeTab(FTabId{TEXT("USDStage")}))
				{
					if (TSharedPtr<SBorder> ContentBorder = StaticCastSharedRef<SBorder>(Tab->GetContent()))
					{
						if (TSharedPtr<SUsdStage> UsdStageEditor = StaticCastSharedRef<SUsdStage>(ContentBorder->GetContent()))
						{
							UsdStageEditor->AttachToStageActor(StageActor);
						}
					}
				}
			}
		);
	}

	virtual void ShutdownModule() override
	{
		AUsdStageActor::OnOpenStageEditorClicked.Remove(OpenStageEditorClickedHandle);

		FEditorDelegates::PreSaveWorldWithContext.Remove(PreSaveWorldEditorDelegateHandle);

		AUsdStageActor::OnActorLoaded.Remove(StageActorLoadedHandle);

		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		MainFrame.UnregisterCanCloseEditor(EditorCanCloseDelegate);

		if (LevelEditorTabManagerChangedHandle.IsValid() && FModuleManager::Get().IsModuleLoaded("LevelEditor"))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			LevelEditorModule.OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
		}

		IUsdUtilitiesModule& UtilitiesModule = FModuleManager::Get().LoadModuleChecked<IUsdUtilitiesModule>(TEXT("UsdUtilities"));
		UtilitiesModule.OnReferenceHandlingDialog.Unbind();

		FUsdStageEditorStyle::Shutdown();
	}

private:
	bool bUndoRedoing = false;

	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle PreSaveWorldEditorDelegateHandle;
	FDelegateHandle EditorCanCloseDelegate;
	FDelegateHandle StageActorLoadedHandle;
	FDelegateHandle OpenStageEditorClickedHandle;
	FDelegateHandle OnTransactionStateChangedHandle;
#endif	  // USE_USD_SDK
};

bool IUsdStageEditorModule::OpenStageEditor() const
{
#if USE_USD_SDK
	return UE::UsdStageEditorModule::Private::GetUsdStageEditor().IsValid();
#else
	return false;
#endif	  // USE_USD_SDK
}

bool IUsdStageEditorModule::CloseStageEditor() const
{
#if USE_USD_SDK
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	if (TSharedPtr<SDockTab> Tab = LevelEditorTabManager->FindExistingLiveTab(UE::UsdStageEditorModule::Private::UsdStageEditorTabID))
	{
		return Tab->RequestCloseTab();
	}
#endif	  // USE_USD_SDK

	return false;
}

bool IUsdStageEditorModule::IsStageEditorOpened() const
{
#if USE_USD_SDK
	const bool bOpenIfNeeded = false;
	return UE::UsdStageEditorModule::Private::GetUsdStageEditor(bOpenIfNeeded).IsValid();
#else
	return false;
#endif	  // USE_USD_SDK
}

AUsdStageActor* IUsdStageEditorModule::GetAttachedStageActor() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		return UsdStageEditor->GetAttachedStageActor();
	}
#endif	  // USE_USD_SDK

	return nullptr;
}

bool IUsdStageEditorModule::SetAttachedStageActor(AUsdStageActor* NewActor) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->AttachToStageActor(NewActor);
		return true;
	}
#endif	  // USE_USD_SDK

	return false;
}

TArray<UE::FSdfLayer> IUsdStageEditorModule::GetSelectedLayers() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		return UsdStageEditor->GetSelectedLayers();
	}
#endif	  // USE_USD_SDK

	return {};
}

void IUsdStageEditorModule::SetSelectedLayers(const TArray<UE::FSdfLayer>& NewSelection) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		return UsdStageEditor->SetSelectedLayers(NewSelection);
	}
#endif	  // USE_USD_SDK
}

TArray<UE::FUsdPrim> IUsdStageEditorModule::GetSelectedPrims() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		return UsdStageEditor->GetSelectedPrims();
	}
#endif	  // USE_USD_SDK

	return {};
}

void IUsdStageEditorModule::SetSelectedPrims(const TArray<UE::FUsdPrim>& NewSelection) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->SetSelectedPrims(NewSelection);
	}
#endif	  // USE_USD_SDK
}

TArray<FString> IUsdStageEditorModule::GetSelectedPropertyNames() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		return UsdStageEditor->GetSelectedPropertyNames();
	}
#endif	  // USE_USD_SDK

	return {};
}

void IUsdStageEditorModule::SetSelectedPropertyNames(const TArray<FString>& NewSelection) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->SetSelectedPropertyNames(NewSelection);
	}
#endif	  // USE_USD_SDK
}

TArray<FString> IUsdStageEditorModule::GetSelectedPropertyMetadataNames() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		return UsdStageEditor->GetSelectedPropertyMetadataNames();
	}
#endif	  // USE_USD_SDK

	return {};
}

void IUsdStageEditorModule::SetSelectedPropertyMetadataNames(const TArray<FString>& NewSelection) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->SetSelectedPropertyMetadataNames(NewSelection);
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileNew() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileNew();
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileOpen(const FString& FilePath) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileOpen(FilePath);
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileSave(const FString& OutputFilePathIfUnsaved) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileSave(OutputFilePathIfUnsaved);
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileExportAllLayers(const FString& OutputDirectory) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileExportAllLayers(OutputDirectory);
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileExportFlattenedStage(const FString& OutputLayer) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileExportFlattenedStage(OutputLayer);
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileExportFlattenedLayerStack(const FString& OutputLayer) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileExportFlattenedLayerStack(OutputLayer);
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileReload() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileReload();
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileReset() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileReset();
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::FileClose() const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->FileClose();
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::ActionsImport(const FString& OutputContentFolder, UUsdStageImportOptions* Options) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		if (OutputContentFolder.IsEmpty())
		{
			UsdStageEditor->ActionsImportWithDialog();
		}
		else
		{
			UsdStageEditor->ActionsImport(OutputContentFolder, Options);
		}
	}
#endif	  // USE_USD_SDK
}

void IUsdStageEditorModule::ExportSelectedLayers(const FString& OutputLayerOrDirectory) const
{
#if USE_USD_SDK
	if (TSharedPtr<SUsdStage> UsdStageEditor = UE::UsdStageEditorModule::Private::GetUsdStageEditor())
	{
		UsdStageEditor->ExportSelectedLayers(OutputLayerOrDirectory);
	}
#endif	  // USE_USD_SDK
}

IMPLEMENT_MODULE_USD(FUsdStageEditorModule, USDStageEditor);

#undef LOCTEXT_NAMESPACE
