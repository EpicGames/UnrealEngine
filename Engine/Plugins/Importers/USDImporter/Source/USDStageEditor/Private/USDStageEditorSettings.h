// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "USDStageEditorSettings.generated.h"

UCLASS(config = EditorPerProjectUserSettings)
class UUsdStageEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Whether our prim selection in SUSDStageTreeView is kept synchronized with the viewport selection */
	UPROPERTY(config, EditAnywhere, Category = USD)
	bool bSelectionSynced = true;

	/** Whether to automatically set a layer as edit target when isolating it */
	UPROPERTY(config, EditAnywhere, Category = USD)
	bool bIsolateLayerSyncedWithEditTarget = true;
};
