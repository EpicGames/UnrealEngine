// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDIntegrationsPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SUSDIntegrationsPanel)

#if USE_USD_SDK

#include "SUSDStageEditorStyle.h"
#include "USDAttributeUtils.h"
#include "USDIntegrationUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "USDValueConversion.h"

#include "UsdWrappers/SdfChangeBlock.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/VtValue.h"

#include "ISinglePropertyView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkTransformRole.h"
#include "ScopedTransaction.h"
#include "SLiveLinkSubjectRepresentationPicker.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "SUSDIntegrationsPanel"

namespace UE::SUsdIntergrationsPanel::Private
{
	const FMargin LeftRowPadding(6.0f, 0.0f, 2.0f, 0.0f);
	const FMargin RightRowPadding(3.0f, 0.0f, 2.0f, 0.0f);

	const TCHAR* NormalFont = TEXT("PropertyWindow.NormalFont");
}

void SUsdIntegrationsPanelRow::Construct(const FArguments& InArgs, TSharedPtr<UE::FUsdAttribute> InAttr, const TSharedRef<STableViewBase>& OwnerTable)
{
	Attribute = InAttr;

	SMultiColumnTableRow<TSharedPtr<UE::FUsdAttribute>>::Construct(SMultiColumnTableRow<TSharedPtr<UE::FUsdAttribute>>::FArguments(), OwnerTable);
}

TSharedRef<SWidget> SUsdIntegrationsPanelRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SWidget> ColumnWidget = SNullWidget::NullWidget;

	bool bIsLeftColumn = true;
	FOptionalSize RowHeight = FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight");

	FName AttributeName = Attribute->GetName();

	using DisplayTextForPropertiesEntry = TPairInitializer<const FName&, const FText&>;
	const static TMap<FName, FText> DisplayTextForProperties({
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealAnimBlueprintPath),
			LOCTEXT("AnimBlueprintPathText", "Anim Blueprint asset")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigPath),
			LOCTEXT("ControlRigPathText", "Control Rig asset")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig),
			LOCTEXT("UseFKControlRigText", "Use FKControlRig")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReduceKeys),
			LOCTEXT("ControlRigReduceKeysText", "Control Rig key reduction")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReductionTolerance),
			LOCTEXT("ControlRigReduceToleranceText", "Control Rig key reduction tolerance")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkSubjectName),
			LOCTEXT("SubjectNameText", "Live Link subject name")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkEnabled),
			LOCTEXT("LiveLinkEnabledText", "Enable LiveLink")
		),
	});

	const static TMap<FName, FText> ToolTipTextForProperties({
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealAnimBlueprintPath),
			LOCTEXT("AnimBlueprintPathToolTip", "Anim Blueprint asset to use on the component to connect with Live Link")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigPath),
			LOCTEXT("ControlRigPathToolTip", "Control Rig Blueprint asset to use")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig),
			LOCTEXT("UseFKControlRigToolTip", "Whether to use a generated FKControlRig instead of the Control Rig Blueprint asset")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReduceKeys),
			LOCTEXT("ControlRigReduceKeysToolTip", "Whether to enable key reduction when generating Control Rig tracks from the USD animation data")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReductionTolerance),
			LOCTEXT("ControlRigReduceToleranceToolTip", "Tolerance to use for the key reduction pass")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkSubjectName),
			LOCTEXT("SubjectNameToolTip", "Which Live Link subject to use for this component")
		),
		DisplayTextForPropertiesEntry(
			*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkEnabled),
			LOCTEXT(
				"LiveLinkEnabledToolTip",
				"If checked will cause the component to be animated with the pose data coming from the Anim Blueprint. If unchecked will cause the "
				"component to be animated with the regular skeleton animation data coming from the USD Stage"
			)
		),
	});

	const FText* ToolTipText = ToolTipTextForProperties.Find(AttributeName);
	if (!ToolTipText)
	{
		ToolTipText = &FText::GetEmpty();
	}

	if (ColumnName == TEXT("PropertyName"))
	{
		if (const FText* TextToDisplay = DisplayTextForProperties.Find(AttributeName))
		{
			// clang-format off
			SAssignNew(ColumnWidget, STextBlock)
			.Text(*TextToDisplay)
			.Font(FAppStyle::GetFontStyle(UE::SUsdIntergrationsPanel::Private::NormalFont))
			.ToolTipText(*ToolTipText);
			// clang-format on
		}
		else
		{
			ensure(false);
			ColumnWidget = SNullWidget::NullWidget;
		}
	}
	else
	{
		bIsLeftColumn = false;

		TSharedPtr<UE::FUsdAttribute> AttributeCopy = Attribute;
		if (!AttributeCopy || !*AttributeCopy)
		{
			return ColumnWidget;
		}

		if (AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealAnimBlueprintPath))
		{
			if (UUsdIntegrationsPanelPropertyDummy* Dummy = GetMutableDefault<UUsdIntegrationsPanelPropertyDummy>())
			{
				// Let the object picker row be as tall as it wants
				RowHeight = FOptionalSize();

				FSinglePropertyParams Params;
				Params.Font = FAppStyle::GetFontStyle(UE::SUsdIntergrationsPanel::Private::NormalFont);
				Params.NamePlacement = EPropertyNamePlacement::Hidden;

				UE::FVtValue Value;
				if (AttributeCopy->Get(Value) && !Value.IsEmpty())
				{
					Dummy->AnimBPProperty = FSoftObjectPath(UsdUtils::Stringify(Value)).TryLoad();
				}
				else
				{
					Dummy->AnimBPProperty = nullptr;
				}

				FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
				TSharedPtr<class ISinglePropertyView> PropertyView = PropertyEditor.CreateSingleProperty(
					Dummy,
					GET_MEMBER_NAME_CHECKED(UUsdIntegrationsPanelPropertyDummy, AnimBPProperty),
					Params
				);

				FSimpleDelegate PropertyChanged = FSimpleDelegate::CreateLambda(
					[AttributeCopy]()
					{
						UE::FUsdPrim Prim = AttributeCopy->GetPrim();
						if (UsdUtils::NotifyIfInstanceProxy(Prim))
						{
							return;
						}

						UUsdIntegrationsPanelPropertyDummy* Dummy = GetMutableDefault<UUsdIntegrationsPanelPropertyDummy>();

						FString Path = (Dummy && Dummy->AnimBPProperty) ? Dummy->AnimBPProperty->GetPathName() : FString{};
						std::string UsdPath = UnrealToUsd::ConvertString(*Path).Get();

						// We need transactions for these because the notices emitted for them are upgraded to resyncs by the stage
						// actor, which means they may generate new assets and components
						FScopedTransaction Transaction(
							FText::Format(LOCTEXT("AnimBluepringTransaction", "Changed AnimBluepring path to '{0}'"), FText::FromString(Path))
						);

						UE::FVtValue Value;
						if (AttributeCopy && *AttributeCopy && UsdUtils::SetUnderlyingValue(Value, UsdPath) && !Value.IsEmpty())
						{
							UE::FSdfChangeBlock Block;
							AttributeCopy->Set(Value);
							UsdUtils::NotifyIfOverriddenOpinion(*AttributeCopy);
						}
					}
				);
				PropertyView->SetOnPropertyValueChanged(PropertyChanged);

				ColumnWidget = PropertyView.ToSharedRef();
			}
			else
			{
				ColumnWidget = SNullWidget::NullWidget;
			}
		}
		else if (AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigPath))
		{
			if (UUsdIntegrationsPanelPropertyDummy* Dummy = GetMutableDefault<UUsdIntegrationsPanelPropertyDummy>())
			{
				// Let the object picker row be as tall as it wants
				RowHeight = FOptionalSize();

				FSinglePropertyParams Params;
				Params.Font = FAppStyle::GetFontStyle(UE::SUsdIntergrationsPanel::Private::NormalFont);
				Params.NamePlacement = EPropertyNamePlacement::Hidden;

				UE::FVtValue Value;
				if (AttributeCopy->Get(Value) && !Value.IsEmpty())
				{
					Dummy->ControlRigProperty = FSoftObjectPath(UsdUtils::Stringify(Value)).TryLoad();
				}
				else
				{
					Dummy->ControlRigProperty = nullptr;
				}

				FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
				TSharedPtr<class ISinglePropertyView> PropertyView = PropertyEditor.CreateSingleProperty(
					Dummy,
					GET_MEMBER_NAME_CHECKED(UUsdIntegrationsPanelPropertyDummy, ControlRigProperty),
					Params
				);

				FSimpleDelegate PropertyChanged = FSimpleDelegate::CreateLambda(
					[AttributeCopy]()
					{
						UE::FUsdPrim Prim = AttributeCopy->GetPrim();
						if (UsdUtils::NotifyIfInstanceProxy(Prim))
						{
							return;
						}

						UUsdIntegrationsPanelPropertyDummy* Dummy = GetMutableDefault<UUsdIntegrationsPanelPropertyDummy>();

						FString Path = (Dummy && Dummy->ControlRigProperty) ? Dummy->ControlRigProperty->GetPathName() : FString{};
						std::string UsdPath = UnrealToUsd::ConvertString(*Path).Get();

						FScopedTransaction Transaction(
							FText::Format(LOCTEXT("ControlRigPathTransaction", "Changed ControlRig blueprint path to '{0}'"), FText::FromString(Path))
						);

						UE::FVtValue Value;
						if (AttributeCopy && *AttributeCopy && UsdUtils::SetUnderlyingValue(Value, UsdPath) && !Value.IsEmpty())
						{
							UE::FSdfChangeBlock Block;
							AttributeCopy->Set(Value);
							UsdUtils::NotifyIfOverriddenOpinion(*AttributeCopy);
						}
					}
				);
				PropertyView->SetOnPropertyValueChanged(PropertyChanged);

				ColumnWidget = PropertyView.ToSharedRef();

				// Disable the widget if we're using an FKControlRig instead
				if (UE::FUsdAttribute FKEnabledAttr = Attribute->GetPrim().GetAttribute(
						*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig)
					))
				{
					UE::FVtValue FKEnabledValue;
					if (FKEnabledAttr.Get(FKEnabledValue))
					{
						const bool bDefaultValue = false;
						if (UsdUtils::GetUnderlyingValue<bool>(FKEnabledValue).Get(bDefaultValue))
						{
							ColumnWidget->SetEnabled(false);
						}
					}
				}
			}
			else
			{
				ColumnWidget = SNullWidget::NullWidget;
			}
		}
		else if (AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReduceKeys))
		{
			// clang-format off
			SAssignNew(ColumnWidget, SBox)
			.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([AttributeCopy]() ->ECheckBoxState
				{
					UE::FVtValue Value;
					if (AttributeCopy && *AttributeCopy && AttributeCopy->Get(Value))
					{
						if (TOptional<bool> UnderlyingValue = UsdUtils::GetUnderlyingValue<bool>(Value))
						{
							return UnderlyingValue.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([AttributeCopy](ECheckBoxState NewValue)
				{
					UE::FUsdPrim Prim = AttributeCopy->GetPrim();
					if (UsdUtils::NotifyIfInstanceProxy(Prim))
					{
						return;
					}

					FScopedTransaction Transaction(
						FText::Format(LOCTEXT("ControlRigReduceKeysTransaction", "Changed ControlRig reduce keys property to '{0}'"),
							FText::FromString(NewValue == ECheckBoxState::Checked ? TEXT("true") : TEXT("false"))
						)
					);

					UE::FVtValue Value;
					if (AttributeCopy && *AttributeCopy && UsdUtils::SetUnderlyingValue(Value, NewValue == ECheckBoxState::Checked) && !Value.IsEmpty())
					{
						UE::FSdfChangeBlock Block;
						AttributeCopy->Set(Value);
						UsdUtils::NotifyIfOverriddenOpinion(*AttributeCopy);
					}
				})
			];
			// clang-format on
		}
		else if (AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReductionTolerance))
		{
			// clang-format off
			SAssignNew(ColumnWidget, SBox)
			.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
			.VAlign(VAlign_Center)
			[
				SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::GetFontStyle(UE::SUsdIntergrationsPanel::Private::NormalFont))
				.OnValueCommitted_Lambda([AttributeCopy](float NewValue, ETextCommit::Type CommitType)
				{
					UE::FUsdPrim Prim = AttributeCopy->GetPrim();
					if (UsdUtils::NotifyIfInstanceProxy(Prim))
					{
						return;
					}

					FScopedTransaction Transaction(
						FText::Format(LOCTEXT("ControlRigReduceToleranceTransaction", "Changed ControlRig reduce tolerance to '{0}'"),
							NewValue
						)
					);

					UE::FVtValue Value;
					if (AttributeCopy && *AttributeCopy && UsdUtils::SetUnderlyingValue(Value, NewValue) && !Value.IsEmpty())
					{
						UE::FSdfChangeBlock Block;
						AttributeCopy->Set(Value);
						UsdUtils::NotifyIfOverriddenOpinion(*AttributeCopy);
					}
				})
				.Value_Lambda([AttributeCopy]() -> float
				{
					UE::FVtValue Value;
					if (AttributeCopy && *AttributeCopy && AttributeCopy->Get(Value))
					{
						if (TOptional<float> UnderlyingValue = UsdUtils::GetUnderlyingValue<float>(Value))
						{
							return UnderlyingValue.GetValue();
						}
					}

					return 0.0f;
				})
			];
			// clang-format on
		}
		else if (AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkSubjectName))
		{
			// SLiveLinkSubjectRepresentationPicker is also a little bit larger than one of our rows, so let it expand a bit
			RowHeight = FOptionalSize();

			// clang-format off
			SAssignNew(ColumnWidget, SLiveLinkSubjectRepresentationPicker)
			.ShowRole(false)
			.ShowSource(false)
			.Font(FAppStyle::GetFontStyle(UE::SUsdIntergrationsPanel::Private::NormalFont))
			.Value_Lambda([AttributeCopy]()
			{
				SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole Result;
				if (AttributeCopy && *AttributeCopy)
				{
					const UE::FUsdPrim& Prim = AttributeCopy->GetPrim();
					Result.Role = (Prim.IsA(TEXT("SkelRoot")) || Prim.IsA(TEXT("Skeleton")))
									  ? ULiveLinkAnimationRole::StaticClass()
									  : ULiveLinkTransformRole::StaticClass();

					UE::FVtValue Value;
					if (AttributeCopy->Get(Value))
					{
						if (TOptional<std::string> UnderlyingValue = UsdUtils::GetUnderlyingValue<std::string>(Value))
						{
							Result.Subject = FLiveLinkSubjectName{
								FName{
									*UsdToUnreal::ConvertString(UnderlyingValue.GetValue())
								}
							};
						}
					}
				}

				return Result;
			})
			.OnValueChanged_Lambda([AttributeCopy](SLiveLinkSubjectRepresentationPicker::FLiveLinkSourceSubjectRole NewValue)
			{
				UE::FUsdPrim Prim = AttributeCopy->GetPrim();
				if (UsdUtils::NotifyIfInstanceProxy(Prim))
				{
					return;
				}

				FScopedUsdAllocs UsdAllocs;

				std::string UsdValue = UnrealToUsd::ConvertString(*(NewValue.Subject.ToString())).Get();

				FScopedTransaction Transaction(
					FText::Format(LOCTEXT("SubjectNameTransaction", "Changed Live Link subject name to '{0}'"),
						FText::FromString(*(NewValue.Subject.ToString()))
					)
				);

				UE::FVtValue Value;
				if (AttributeCopy && *AttributeCopy && UsdUtils::SetUnderlyingValue(Value, UsdValue) && !Value.IsEmpty())
				{
					UE::FSdfChangeBlock Block;
					AttributeCopy->Set(Value);
					UsdUtils::NotifyIfOverriddenOpinion(*AttributeCopy);
				}
			});
			// clang-format on
		}
		else if (AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkEnabled) || AttributeName == *UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig))
		{
			// clang-format off
			SAssignNew(ColumnWidget, SBox)
			.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([AttributeCopy]()
				{
					UE::FVtValue Value;
					if (AttributeCopy && *AttributeCopy && AttributeCopy->Get(Value))
					{
						if (TOptional<bool> UnderlyingValue = UsdUtils::GetUnderlyingValue<bool>(Value))
						{
							return UnderlyingValue.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([AttributeCopy, AttributeName](ECheckBoxState NewState)
				{
					UE::FUsdPrim Prim = AttributeCopy->GetPrim();
					if (UsdUtils::NotifyIfInstanceProxy(Prim))
					{
						return;
					}

					FScopedTransaction Transaction(
						FText::Format(LOCTEXT("IntegrationBoolTransaction", "Changed attribute '{0}' to "),
							FText::FromName(AttributeName),
							FText::FromString(NewState == ECheckBoxState::Checked ? TEXT("true") : TEXT("false"))
						)
					);

					UE::FVtValue Value;
					if (AttributeCopy && *AttributeCopy && UsdUtils::SetUnderlyingValue(Value, NewState == ECheckBoxState::Checked) && !Value.IsEmpty())
					{
						UE::FSdfChangeBlock Block;
						AttributeCopy->Set(Value);
						UsdUtils::NotifyIfOverriddenOpinion(*AttributeCopy);
					}
				})
			];
			// clang-format on
		}
		else
		{
			ensure(false);
		}
	}

	// clang-format off
	return SNew(SBox)
		.HeightOverride(RowHeight)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(bIsLeftColumn ? UE::SUsdIntergrationsPanel::Private::LeftRowPadding : UE::SUsdIntergrationsPanel::Private::RightRowPadding)
			.AutoWidth()
			[
				ColumnWidget
			]
		];
	// clang-format on
}

void SUsdIntegrationsPanel::Construct(const FArguments& InArgs)
{
	// clang-format off
	SAssignNew(HeaderRowWidget, SHeaderRow)

	+SHeaderRow::Column(FName(TEXT("PropertyName")))
	.DefaultLabel(NSLOCTEXT("SUsdIntegrationsPanel", "Integrations", "Integrations"))
	.FillWidth(30.0f)

	+SHeaderRow::Column(FName(TEXT("Value")))
	.DefaultLabel(FText::GetEmpty())
	.FillWidth(70.0f);

	SListView::Construct
	(
		SListView::FArguments()
		.ListItemsSource(&ViewModel.Attributes)
		.OnGenerateRow(this, &SUsdIntegrationsPanel::OnGenerateRow)
		.HeaderRow(HeaderRowWidget)
	);

	SetVisibility(EVisibility::Collapsed); // Start hidden until SetPrimPath displays us
	// clang-format on
}

void SUsdIntegrationsPanel::SetPrimPath(const UE::FUsdStageWeak& InUsdStage, const TCHAR* InPrimPath)
{
	ViewModel.UpdateAttributes(InUsdStage, InPrimPath);

	SetVisibility(ViewModel.Attributes.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);

	RequestListRefresh();
}

TSharedRef<ITableRow> SUsdIntegrationsPanel::OnGenerateRow(TSharedPtr<UE::FUsdAttribute> InAttr, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUsdIntegrationsPanelRow, InAttr, OwnerTable);
}

#undef LOCTEXT_NAMESPACE
#endif	  // USE_USD_SDK
