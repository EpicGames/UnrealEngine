// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDIntegrationsViewModel.h"
#include "UsdWrappers/ForwardDeclarations.h"

#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"

#include "SUSDIntegrationsPanel.generated.h"

class SHeaderRow;
namespace UE
{
	class FUsdAttribute;
}

// We need an actual UObject and UPROPERTY to use the property editor module and generate one of the
// standard object picker widgets, so we'll be using the CDO of this class to do that
UCLASS(Abstract, Transient, MinimalAPI)
class UUsdIntegrationsPanelPropertyDummy : public UObject
{
public:
	GENERATED_BODY()

	// Ideally these would be FSoftObjectPaths, but FPropertyEditorModule doesn't support generating
	// widgets for those properties yet
	UPROPERTY(EditAnywhere, Transient, Category = Dummy, meta = (AllowedClasses = "/Script/Engine.AnimBlueprint"))
	TObjectPtr<UObject> AnimBPProperty;

	UPROPERTY(EditAnywhere, Transient, Category = Dummy, meta = (AllowedClasses = "/Script/ControlRigDeveloper.ControlRigEditorAssetInterface,/Script/ControlRig.ControlRigRuntimeAssetInterface"))
	TObjectPtr<UObject> ControlRigProperty;
};

#if USE_USD_SDK

class SUsdIntegrationsPanelRow : public SMultiColumnTableRow<TSharedPtr<UE::FUsdAttribute>>
{
public:
	SLATE_BEGIN_ARGS(SUsdIntegrationsPanelRow)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<UE::FUsdAttribute> InAttr, const TSharedRef<STableViewBase>& OwnerTable);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<UE::FUsdAttribute> Attribute;
};

// We don't really need a list view here since we'll mostly always know exactly what attributes are going to
// be displayed here beforehand, but doing so is a simple way of ensuring a consistent look between this panel
// and the variants/references panels, that *do* need to be lists
class SUsdIntegrationsPanel : public SListView<TSharedPtr<UE::FUsdAttribute>>
{
	SLATE_BEGIN_ARGS(SUsdIntegrationsPanel)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath);

protected:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<UE::FUsdAttribute> InAttr, const TSharedRef<STableViewBase>& OwnerTable);

private:
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	FUsdIntegrationsViewModel ViewModel;
};

#endif	  // USE_USD_SDK
