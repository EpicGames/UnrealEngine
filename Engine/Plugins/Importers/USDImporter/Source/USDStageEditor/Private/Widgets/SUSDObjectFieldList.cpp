// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDObjectFieldList.h"

#include "SUSDStageEditorStyle.h"
#include "UnrealUSDWrapper.h"

#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"

#if USE_USD_SDK

#define LOCTEXT_NAMESPACE "SUsdObjectFieldList"

namespace USDObjectFieldListConstants
{
	const FMargin LeftRowPadding(6.0f, 2.5f, 2.0f, 2.5f);
	const FMargin RightRowPadding(3.0f, 2.5f, 2.0f, 2.5f);
	const FMargin ComboBoxItemPadding(3.0f, 0.0f, 2.0f, 0.0f);
	const FMargin NumericEntryBoxItemPadding(0.0f, 0.0f, 2.0f, 0.0f);
	const float DesiredNumericEntryBoxWidth = 80.0f;
	const float DesiredTextEntryBoxWidth = 80.0f;

	const TCHAR* NormalFont = TEXT("PropertyWindow.NormalFont");
}

class SUsdObjectFieldRow;

namespace USDObjectFieldListImpl
{
	enum class EWidgetType : uint8
	{
		None,
		Bool,
		U8,
		I32,
		U32,
		I64,
		U64,
		F32,
		F64,
		Text,
		Dropdown,
	};

	enum class EWidgetLabelTypes
	{
		NoLabel,
		RGBA,
		XYZW,
	};

	static TMap<FString, TArray<TSharedPtr<FString>>> TokenDropdownOptions;

	void ResetOptions(const FString& TokenName)
	{
		TokenDropdownOptions.Remove(TokenName);
	}

	TArray<TSharedPtr<FString>>* GetTokenDropdownOptions(const FUsdObjectFieldViewModel& ViewModel)
	{
		if (TArray<TSharedPtr<FString>>* FoundOptions = TokenDropdownOptions.Find(ViewModel.Label))
		{
			return FoundOptions;
		}

		TArray<TSharedPtr<FString>> Options = ViewModel.GetDropdownOptions();
		if (Options.Num() == 0)
		{
			// We don't know the options for this property, so return nullptr so that it can become a regular text input box
			return nullptr;
		}

		return &USDObjectFieldListImpl::TokenDropdownOptions.Add(ViewModel.Label, Options);
	}

	EWidgetLabelTypes GetLabelType(const UsdUtils::FConvertedVtValue& FieldValue, const FString& ValueRole)
	{
		if (FieldValue.Entries.Num() < 1)
		{
			return EWidgetLabelTypes::NoLabel;
		}

		int32 NumComponents = FieldValue.Entries[0].Num();
		EWidgetLabelTypes LabelType = EWidgetLabelTypes::NoLabel;
		if (ValueRole.StartsWith(TEXT("color"), ESearchCase::IgnoreCase))
		{
			LabelType = EWidgetLabelTypes::RGBA;
		}
		else if (NumComponents > 1 && NumComponents <= 4 && FieldValue.SourceType != UsdUtils::EUsdBasicDataTypes::Matrix2d)
		{
			LabelType = EWidgetLabelTypes::XYZW;
		}

		return LabelType;
	}

	EWidgetType GetWidgetType(UsdUtils::EUsdBasicDataTypes SourceType)
	{
		using namespace UsdUtils;

		switch (SourceType)
		{
			case EUsdBasicDataTypes::Bool:
				return EWidgetType::Bool;
			case EUsdBasicDataTypes::Uchar:
				return EWidgetType::U8;
			case EUsdBasicDataTypes::Int:
			case EUsdBasicDataTypes::Int2:
			case EUsdBasicDataTypes::Int3:
			case EUsdBasicDataTypes::Int4:
				return EWidgetType::I32;
			case EUsdBasicDataTypes::Uint:
				return EWidgetType::U32;
			case EUsdBasicDataTypes::Int64:
				return EWidgetType::I64;
			case EUsdBasicDataTypes::Uint64:
				return EWidgetType::U64;
			case EUsdBasicDataTypes::Half:
			case EUsdBasicDataTypes::Half2:
			case EUsdBasicDataTypes::Half3:
			case EUsdBasicDataTypes::Half4:
			case EUsdBasicDataTypes::Quath:
			case EUsdBasicDataTypes::Float:
			case EUsdBasicDataTypes::Float2:
			case EUsdBasicDataTypes::Float3:
			case EUsdBasicDataTypes::Float4:
			case EUsdBasicDataTypes::Quatf:
				return EWidgetType::F32;
			case EUsdBasicDataTypes::Double:
			case EUsdBasicDataTypes::Double2:
			case EUsdBasicDataTypes::Double3:
			case EUsdBasicDataTypes::Double4:
			case EUsdBasicDataTypes::Timecode:
			case EUsdBasicDataTypes::Matrix2d:
			case EUsdBasicDataTypes::Matrix3d:
			case EUsdBasicDataTypes::Matrix4d:
			case EUsdBasicDataTypes::Quatd:
				return EWidgetType::F64;
			case EUsdBasicDataTypes::Token:
				return EWidgetType::Dropdown;
			case EUsdBasicDataTypes::String:
			case EUsdBasicDataTypes::Asset:
				return EWidgetType::Text;
			default:
				break;
		}

		return EWidgetType::None;
	}

	/** Always returns `NumLabels` widgets, regardless of LabelTypes. Those may be the NullWidget though */
	TArray<TSharedRef<SWidget>> GetNumericEntryBoxLabels(int32 NumLabels, EWidgetLabelTypes LabelType)
	{
		const static TArray<const FLinearColor*> Colors = {
			&SNumericEntryBox<int32>::RedLabelBackgroundColor,
			&SNumericEntryBox<int32>::GreenLabelBackgroundColor,
			&SNumericEntryBox<int32>::BlueLabelBackgroundColor,
			&FLinearColor::White};

		if (NumLabels > 4)
		{
			LabelType = EWidgetLabelTypes::NoLabel;
		}

		TArray<TSharedRef<SWidget>> Labels;
		Labels.Reserve(NumLabels);

		switch (LabelType)
		{
			case EWidgetLabelTypes::RGBA:
				for (int32 Index = 0; Index < NumLabels; ++Index)
				{
					Labels.Add(SNumericEntryBox<int32>::BuildNarrowColorLabel(*Colors[Index]));
				}
				break;
			case EWidgetLabelTypes::XYZW:
				for (int32 Index = 0; Index < NumLabels; ++Index)
				{
					Labels.Add(SNumericEntryBox<int32>::BuildNarrowColorLabel(*Colors[Index]));
				}
				break;
			case EWidgetLabelTypes::NoLabel:
			default:
				for (int32 Index = 0; Index < NumLabels; ++Index)
				{
					Labels.Add(SNullWidget::NullWidget);
				}
				break;
		}

		return Labels;
	}
}	 // namespace USDObjectFieldListImpl

class SUsdObjectFieldRow : public SMultiColumnTableRow<TSharedPtr<FUsdObjectFieldViewModel>>
{
	SLATE_BEGIN_ARGS(SUsdObjectFieldRow)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(
		const FArguments& InArgs,
		const TSharedPtr<FUsdObjectFieldViewModel>& InUsdObjectField,
		const TSharedRef<STableViewBase>& OwnerTable
	);
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void SetUsdObjectField(const TSharedPtr<FUsdObjectFieldViewModel>& InUsdObjectField);

protected:
	FText GetLabel() const
	{
		return FText::FromString(Field->Label);
	}
	FText GetType() const
	{
		const static FText TypeTexts[] = {FText::FromString(TEXT("M")), FText::FromString(TEXT("A")), FText::FromString(TEXT("R"))};

		const int32 Index = static_cast<int32>(Field->Type);
		return Index >= 0 && Index < static_cast<int32>(EObjectFieldType::MAX) ? TypeTexts[Index] : FText::GetEmpty();
	}

	FText GetTypeToolTip() const
	{
		const static FText ToolTipTexts[] = {
			LOCTEXT("MetadataTooltip", "Metadata"),
			LOCTEXT("AttributeTooltip", "Attribute"),
			LOCTEXT("RelationshipTooltip", "Relationship")};

		const int32 Index = static_cast<int32>(Field->Type);
		return Index >= 0 && Index < static_cast<int32>(EObjectFieldType::MAX) ? ToolTipTexts[Index] : FText::GetEmpty();
	}

	// Optional here because SNumericEntryBox uses optional values as input
	template<typename T>
	TOptional<T> GetValue(int32 ComponentIndex) const
	{
		if (Field->Value.Entries.Num() == 1)	// Ignore arrays for now
		{
			if (Field->Value.Entries[0].IsValidIndex(ComponentIndex))
			{
				if (T* Value = Field->Value.Entries[0][ComponentIndex].TryGet<T>())
				{
					return *Value;
				}
			}
		}

		return {};
	}

	// Other overloads as checkbox/text widgets don't use optional values as input, and these should only ever have one component anyway
	FText GetValueText() const
	{
		if (Field->Value.Entries.Num() == 1)
		{
			if (Field->Value.Entries[0].Num() > 0)
			{
				if (FString* Value = Field->Value.Entries[0][0].TryGet<FString>())
				{
					return FText::FromString(*Value);
				}
			}
		}

		return FText::GetEmpty();
	}

	ECheckBoxState GetValueBool() const
	{
		if (Field->Value.Entries.Num() == 1)
		{
			if (Field->Value.Entries[0].Num() > 0)
			{
				if (bool* Value = Field->Value.Entries[0][0].TryGet<bool>())
				{
					return *Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
			}
		}

		return ECheckBoxState::Undetermined;
	}

private:
	template<typename T>
	TArray<TAttribute<TOptional<T>>> GetAttributeArray();

	TSharedRef<SWidget> GenerateTextWidget(
		const TAttribute<FText>& Attribute,
		const TAttribute<FText>& ToolTipAttribute = {},
		ETextJustify::Type TextJustify = ETextJustify::Left
	);
	TSharedRef<SWidget> GenerateEditableTextWidget(const TAttribute<FText>& Attribute, bool bIsReadOnly);
	TSharedRef<SWidget> GenerateCheckboxWidget(const TAttribute<ECheckBoxState>& Attribute);

	template<typename T>
	TSharedRef<SWidget> GenerateSpinboxWidgets(
		const TArray<TAttribute<TOptional<T>>>& Attribute,
		USDObjectFieldListImpl::EWidgetLabelTypes LabelType = USDObjectFieldListImpl::EWidgetLabelTypes::NoLabel
	);

private:
	template<typename T>
	void OnSpinboxValueChanged(T NewValue, int32 ComponentIndex);

	template<typename T>
	void OnSpinboxValueCommitted(T InNewValue, ETextCommit::Type CommitType, int32 ComponentIndex);

	void OnComboBoxSelectionChanged(TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo);
	void OnTextBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitType);
	void OnCheckBoxCheckStateChanged(ECheckBoxState NewState);

private:
	TSharedPtr<FUsdObjectFieldViewModel> Field;
};

void SUsdObjectFieldRow::Construct(
	const FArguments& InArgs,
	const TSharedPtr<FUsdObjectFieldViewModel>& InUsdObjectField,
	const TSharedRef<STableViewBase>& OwnerTable
)
{
	SetUsdObjectField(InUsdObjectField);

	SMultiColumnTableRow<TSharedPtr<FUsdObjectFieldViewModel>>::Construct(
		SMultiColumnTableRow<TSharedPtr<FUsdObjectFieldViewModel>>::FArguments(),
		OwnerTable
	);
}

TSharedRef<SWidget> SUsdObjectFieldRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	using namespace USDObjectFieldListImpl;

	TSharedRef<SWidget> ColumnWidget = SNullWidget::NullWidget;

	if (ColumnName == ObjectFieldColumnIds::TypeColumn)
	{
		ColumnWidget = GenerateTextWidget({this, &SUsdObjectFieldRow::GetType}, {this, &SUsdObjectFieldRow::GetTypeToolTip}, ETextJustify::Center);
	}
	else if (ColumnName == ObjectFieldColumnIds::NameColumn)
	{
		ColumnWidget = GenerateTextWidget({this, &SUsdObjectFieldRow::GetLabel});
	}
	else
	{
		EWidgetType WidgetType = GetWidgetType(Field->Value.SourceType);
		EWidgetLabelTypes LabelType = GetLabelType(Field->Value, Field->ValueRole);

		switch (WidgetType)
		{
			case EWidgetType::Bool:
				ColumnWidget = GenerateCheckboxWidget({this, &SUsdObjectFieldRow::GetValueBool});
				break;
			case EWidgetType::U8:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<uint8>(), LabelType);
				break;
			case EWidgetType::I32:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<int32>(), LabelType);
				break;
			case EWidgetType::U32:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<uint32>(), LabelType);
				break;
			case EWidgetType::I64:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<int64>(), LabelType);
				break;
			case EWidgetType::U64:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<uint64>(), LabelType);
				break;
			case EWidgetType::F32:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<float>(), LabelType);
				break;
			case EWidgetType::F64:
				ColumnWidget = GenerateSpinboxWidgets(GetAttributeArray<double>(), LabelType);
				break;
			case EWidgetType::Dropdown:
			{
				TArray<TSharedPtr<FString>>* Options = GetTokenDropdownOptions(*Field);

				// Show a dropdown if we know the available options for that token
				if (Options)
				{
					// clang-format off
					SAssignNew(ColumnWidget, SBox)
					.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SComboBox<TSharedPtr<FString>>)
						.OptionsSource(Options)
						.OnGenerateWidget_Lambda([&](TSharedPtr<FString> Option)
						{
							return SUsdObjectFieldRow::GenerateTextWidget(FText::FromString(*Option));
						})
						.OnSelectionChanged(this, &SUsdObjectFieldRow::OnComboBoxSelectionChanged)
						[
							// Having an editable text box inside the combobox allows the user to pick through the most common ones but to
							// also specify a custom kind/purpose/etc. if they want to
							SNew(SEditableTextBox)
							.Text(this, &SUsdObjectFieldRow::GetValueText)
							.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
							.Padding(FMargin(3.0f))
							// Fixed foreground color or else it will flip when our row is selected, and we already have a background
							.ForegroundColor(FAppStyle::GetSlateColor(TEXT("Colors.ForegroundHover")))
							.OnTextCommitted(this, &SUsdObjectFieldRow::OnTextBoxTextCommitted)
						]
					];
					// clang-format on

					break;
				}

				// If we don't have any options we intentionally fall down into the 'Text' case
			}
			case EWidgetType::Text:
			{
				ColumnWidget = SUsdObjectFieldRow::GenerateEditableTextWidget({this, &SUsdObjectFieldRow::GetValueText}, Field->bReadOnly);
				break;
			}
			case EWidgetType::None:
			default:
				ColumnWidget = SNullWidget::NullWidget;
				break;
		}
	}

	// clang-format off
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			ColumnWidget
		];
	// clang-format on
}

void SUsdObjectFieldRow::SetUsdObjectField(const TSharedPtr<FUsdObjectFieldViewModel>& InUsdObjectField)
{
	Field = InUsdObjectField;
}

template<typename T>
TArray<TAttribute<TOptional<T>>> SUsdObjectFieldRow::GetAttributeArray()
{
	using AttrType = TAttribute<TOptional<T>>;

	TArray<AttrType> Attributes;
	if (Field->Value.Entries.Num() == 1)
	{
		for (int32 ComponentIndex = 0; ComponentIndex < Field->Value.Entries[0].Num(); ++ComponentIndex)
		{
			Attributes.Add(AttrType::Create(AttrType::FGetter::CreateSP(this, &SUsdObjectFieldRow::template GetValue<T>, ComponentIndex)));
		}
	}

	return Attributes;
}

TSharedRef<SWidget> SUsdObjectFieldRow::GenerateTextWidget(
	const TAttribute<FText>& Attribute,
	const TAttribute<FText>& ToolTipAttribute,
	ETextJustify::Type TextJustify
)
{
	// clang-format off
	return SNew(STextBlock)
		.Text(Attribute)
		.Justification(TextJustify)
		.ToolTipText(ToolTipAttribute)
		.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
		.Margin(USDObjectFieldListConstants::RightRowPadding);
	// clang-format on
}

TSharedRef<SWidget> SUsdObjectFieldRow::GenerateEditableTextWidget(const TAttribute<FText>& Attribute, bool bIsReadOnly)
{
	// clang-format off
	return SNew(SBox)
		.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
		.MinDesiredWidth(USDObjectFieldListConstants::DesiredTextEntryBoxWidth)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(USDObjectFieldListConstants::DesiredTextEntryBoxWidth)
			.Text(Attribute)
			.IsReadOnly(bIsReadOnly)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.OnTextCommitted(this, &SUsdObjectFieldRow::OnTextBoxTextCommitted)
			// Fixed foreground color or else it will flip when our row is selected, and we already have a background
			.ForegroundColor(FAppStyle::GetSlateColor(bIsReadOnly ? TEXT("Colors.Foreground") : TEXT("Colors.ForegroundHover")))
		];
	// clang-format on
}

TSharedRef<SWidget> SUsdObjectFieldRow::GenerateCheckboxWidget(const TAttribute<ECheckBoxState>& Attribute)
{
	// clang-format off
	return SNew(SBox)
		.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SCheckBox)
			.IsChecked(Attribute)
			.OnCheckStateChanged(this, &SUsdObjectFieldRow::OnCheckBoxCheckStateChanged)
		];
	// clang-format on
}

template<typename T>
TSharedRef<SWidget> SUsdObjectFieldRow::GenerateSpinboxWidgets(
	const TArray<TAttribute<TOptional<T>>>& Attributes,
	USDObjectFieldListImpl::EWidgetLabelTypes LabelType
)
{
	if (Attributes.Num() == 0)
	{
		return SNullWidget::NullWidget;
	}

	TArray<TSharedRef<SWidget>> Labels = USDObjectFieldListImpl::GetNumericEntryBoxLabels(Attributes.Num(), LabelType);

	// If we're some type of matrix attribute, we'll want to display multiple rows
	const int32 NumRows = LabelType != USDObjectFieldListImpl::EWidgetLabelTypes::NoLabel ? 1	 // If we have labels we're definitely not a matrix
																								 // type
						  : Attributes.Num() == 4  ? 2
						  : Attributes.Num() == 9  ? 3
						  : Attributes.Num() == 16 ? 4
												   : 1;

	const int32 NumColumns = Attributes.Num() / NumRows;

	// clang-format off
	TSharedPtr<SVerticalBox> VertBox = SNew(SVerticalBox);
	for (int32 RowIndex = 0; RowIndex <NumRows; ++RowIndex)
	{
		TSharedPtr<SHorizontalBox> HorizBox = SNew(SHorizontalBox);
		for (int32 ColumnIndex = 0; ColumnIndex <NumColumns; ++ColumnIndex)
		{
			const int32 ComponentIndex = RowIndex * NumColumns + ColumnIndex;
			const TAttribute<TOptional<T>>& Attribute = Attributes[ComponentIndex];

			TSharedRef<SWidget> EntryBox = SNew(SNumericEntryBox<T>)
				.AllowSpin(true)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.ShiftMultiplier(10.f)
				.SupportDynamicSliderMaxValue(true)
				.SupportDynamicSliderMinValue(true)
				.OnValueChanged(this, &SUsdObjectFieldRow::OnSpinboxValueChanged<T>, ComponentIndex)
				.OnValueCommitted(this, &SUsdObjectFieldRow::OnSpinboxValueCommitted<T>, ComponentIndex)
				.Value(Attribute)
				.MinValue(TOptional<T>())
				.MaxValue(TOptional<T>())
				.MinSliderValue(TOptional<T>())
				.MaxSliderValue(TOptional<T>())
				.SliderExponent(T(1))
				.Delta(T(0))
				.LabelPadding(FMargin(3))
				.LabelLocation(SNumericEntryBox<T>::ELabelLocation::Inside)
				.Label()
				[
					Labels[ComponentIndex]
				];

			HorizBox->AddSlot()
				.Padding(USDObjectFieldListConstants::NumericEntryBoxItemPadding)
				[
					SNew(SBox)
					.MinDesiredWidth(USDObjectFieldListConstants::DesiredNumericEntryBoxWidth)
					.MaxDesiredWidth(USDObjectFieldListConstants::DesiredNumericEntryBoxWidth)
					[
						EntryBox
					]
				];
		}

		VertBox->AddSlot()
			[
				SNew(SBox)
				.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
				.VAlign(VAlign_Center)
				[
					HorizBox.ToSharedRef()
				]
			];
	}

	return SNew(SBox)
		.MinDesiredWidth(USDObjectFieldListConstants::DesiredNumericEntryBoxWidth)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			VertBox.ToSharedRef()
		];
	// clang-format on
}

template<typename T>
void SUsdObjectFieldRow::OnSpinboxValueChanged(T NewValue, int32 ComponentIndex)
{
	Field->Value.Entries[0][ComponentIndex].Set<T>(NewValue);
}

template<typename T>
void SUsdObjectFieldRow::OnSpinboxValueCommitted(T InNewValue, ETextCommit::Type CommitType, int32 ComponentIndex)
{
	if (CommitType == ETextCommit::OnCleared || Field->bReadOnly)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("SetUsdAttributeTransaction", "Set attribute '{0}'"), FText::FromString(Field->Label)));

	UsdUtils::FConvertedVtValue NewValue = Field->Value;
	NewValue.Entries[0][ComponentIndex].Set<T>(InNewValue);

	Field->SetAttributeValue(NewValue);
}

void SUsdObjectFieldRow::OnComboBoxSelectionChanged(TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo)
{
	if (Field->bReadOnly)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("SetUsdAttributeTransaction", "Set attribute '{0}'"), FText::FromString(Field->Label)));

	UsdUtils::FConvertedVtValue NewValue = Field->Value;
	NewValue.Entries[0][0] = UsdUtils::FConvertedVtValueComponent(TInPlaceType<FString>(), *ChosenOption);

	Field->SetAttributeValue(NewValue);
}

void SUsdObjectFieldRow::OnTextBoxTextCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnCleared || Field->bReadOnly)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("SetUsdAttributeTransaction", "Set attribute '{0}'"), FText::FromString(Field->Label)));

	UsdUtils::FConvertedVtValue NewValue = Field->Value;
	NewValue.Entries[0][0] = UsdUtils::FConvertedVtValueComponent(TInPlaceType<FString>(), NewText.ToString());

	Field->SetAttributeValue(NewValue);
}

void SUsdObjectFieldRow::OnCheckBoxCheckStateChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Undetermined || Field->bReadOnly)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(LOCTEXT("SetUsdAttributeTransaction", "Set attribute '{0}'"), FText::FromString(Field->Label)));

	UsdUtils::FConvertedVtValue NewValue = Field->Value;
	NewValue.Entries[0][0] = UsdUtils::FConvertedVtValueComponent(TInPlaceType<bool>(), NewState == ECheckBoxState::Checked);

	Field->SetAttributeValue(NewValue);
}

void SUsdObjectFieldList::Construct(const FArguments& InArgs)
{
	// Clear map as usd file may have additional Kinds now
	USDObjectFieldListImpl::ResetOptions(TEXT("Kind"));

	// clang-format off
	SAssignNew(HeaderRowWidget, SHeaderRow)

	+SHeaderRow::Column(ObjectFieldColumnIds::TypeColumn)
	.DefaultLabel(FText::GetEmpty())
	.FixedWidth(24.0f)
	.SortMode(this, &SUsdObjectFieldList::GetColumnSortMode, ObjectFieldColumnIds::TypeColumn)
	.OnSort(this, &SUsdObjectFieldList::Sort)

	+SHeaderRow::Column(ObjectFieldColumnIds::NameColumn)
	.DefaultLabel(InArgs._NameColumnText)
	.FillWidth(40.f)
	.SortMode(this, &SUsdObjectFieldList::GetColumnSortMode, ObjectFieldColumnIds::NameColumn)
	.OnSort(this, &SUsdObjectFieldList::Sort)

	+SHeaderRow::Column(ObjectFieldColumnIds::ValueColumn)
	.DefaultLabel(LOCTEXT("ValueColumnText", "Value"))
	.FillWidth(60.f);

	SListView::Construct
	(
		SListView::FArguments()
		.ListItemsSource(&ViewModel.Fields)
		.OnGenerateRow(this, &SUsdObjectFieldList::OnGenerateRow)
		.HeaderRow(HeaderRowWidget)
	);
	// clang-format on

	OnSelectionChanged = InArgs._OnSelectionChanged;
}

TSharedRef<ITableRow> SUsdObjectFieldList::OnGenerateRow(
	TSharedPtr<FUsdObjectFieldViewModel> InDisplayNode,
	const TSharedRef<STableViewBase>& OwnerTable
)
{
	return SNew(SUsdObjectFieldRow, InDisplayNode, OwnerTable);
}

void SUsdObjectFieldList::GenerateFieldList(const UE::FUsdStageWeak& UsdStage, const TCHAR* InObjectPath)
{
	const float TimeCode = 0.f;
	ViewModel.Refresh(UsdStage, InObjectPath, TimeCode);
}

void SUsdObjectFieldList::Sort(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	ViewModel.CurrentSortColumn = ColumnId;
	ViewModel.CurrentSortMode = NewSortMode;
	ViewModel.Sort();

	RequestListRefresh();
}

EColumnSortMode::Type SUsdObjectFieldList::GetColumnSortMode(const FName ColumnId) const
{
	if (ViewModel.CurrentSortColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return ViewModel.CurrentSortMode;
}

void SUsdObjectFieldList::SetObjectPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* InObjectPath)
{
	GenerateFieldList(UsdStage, InObjectPath);

	RequestListRefresh();
}

TArray<FString> SUsdObjectFieldList::GetSelectedFieldNames() const
{
	TArray<FString> SelectedProperties;

	TArray<TSharedPtr<FUsdObjectFieldViewModel>> SelectedViewModels = GetSelectedItems();
	SelectedProperties.Reserve(SelectedViewModels.Num());

	for (const TSharedPtr<FUsdObjectFieldViewModel>& SelectedItem : SelectedViewModels)
	{
		if (SelectedItem)
		{
			SelectedProperties.Add(SelectedItem->Label);
		}
	}

	return SelectedProperties;
}

void SUsdObjectFieldList::SetSelectedFieldNames(const TArray<FString>& NewSelection)
{
	TSet<FString> NewSelectionSet{NewSelection};

	Private_ClearSelection();

	TArray<TSharedPtr<FUsdObjectFieldViewModel>> NewItemSelection;
	NewItemSelection.Reserve(NewSelection.Num());

	for (const TSharedPtr<FUsdObjectFieldViewModel>& Item : ViewModel.Fields)
	{
		if (Item && NewSelectionSet.Contains(Item->Label))
		{
			NewItemSelection.Add(Item);
		}
	}

	const bool bSelected = true;
	SetItemSelection(NewItemSelection, bSelected);
}

UE::FUsdStageWeak SUsdObjectFieldList::GetUsdStage() const
{
	return ViewModel.GetUsdStage();
}

FString SUsdObjectFieldList::GetObjectPath() const
{
	return ViewModel.GetObjectPath();
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
