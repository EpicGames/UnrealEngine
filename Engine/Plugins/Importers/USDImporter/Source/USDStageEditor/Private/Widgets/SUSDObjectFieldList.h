// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDObjectFieldViewModel.h"
#include "Widgets/Views/SListView.h"

#if USE_USD_SDK

class SUsdObjectFieldList : public SListView<TSharedPtr<FUsdObjectFieldViewModel>>
{
public:
	SLATE_BEGIN_ARGS(SUsdObjectFieldList)
	{
	}
	SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
	SLATE_ATTRIBUTE(FText, NameColumnText)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetObjectPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* InObjectPath);

	TArray<FString> GetSelectedFieldNames() const;
	void SetSelectedFieldNames(const TArray<FString>& NewSelection);

	UE::FUsdStageWeak GetUsdStage() const;
	FString GetObjectPath() const;

protected:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FUsdObjectFieldViewModel> InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable);
	void GenerateFieldList(const UE::FUsdStageWeak& UsdStage, const TCHAR* InObjectPath);

	void Sort(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

private:
	FUsdObjectFieldsViewModel ViewModel;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
};

#endif	  // USE_USD_SDK
