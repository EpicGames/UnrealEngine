// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDReferencesList.h"

#include "SUSDStageEditorStyle.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "USDStageActor.h"

#if USE_USD_SDK

#define LOCTEXT_NAMESPACE "USDReferencesList"

namespace UsdReferencesListConstants
{
	const FMargin RowPadding(6.0f, 2.5f, 2.0f, 2.5f);

	const TCHAR* NormalFont = TEXT("PropertyWindow.NormalFont");
}

void SUsdReferenceRow::Construct(const FArguments& InArgs, TSharedPtr<FUsdReference> InReference, const TSharedRef<STableViewBase>& OwnerTable)
{
	Reference = InReference;

	SMultiColumnTableRow<TSharedPtr<FUsdReference>>::Construct(SMultiColumnTableRow<TSharedPtr<FUsdReference>>::FArguments(), OwnerTable);
}

TSharedRef<SWidget> SUsdReferenceRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SWidget> ColumnWidget = SNullWidget::NullWidget;

	// clang-format off
	if (ColumnName == TEXT("ReferenceType"))
	{
		SAssignNew(ColumnWidget, STextBlock)
		.Text(FText::FromString(Reference->bIsPayload ? TEXT("P") : TEXT("R")))
		.ToolTipText(FText::FromString(Reference->bIsPayload ? TEXT("Payload") : TEXT("Reference")))
		.Justification(ETextJustify::Center)
		.Font(FAppStyle::GetFontStyle(UsdReferencesListConstants::NormalFont));
	}

	else if (ColumnName == TEXT("AssetPath"))
	{
		SAssignNew(ColumnWidget, STextBlock)
		.Text(FText::FromString(Reference->AssetPath.IsEmpty() ? TEXT("(internal reference)") : Reference->AssetPath))
		.Margin(FMargin(3.0f, 0.0f, 0.0f, 0.0f))
		.Font(FAppStyle::GetFontStyle(UsdReferencesListConstants::NormalFont));
	}

	else if (ColumnName == TEXT("PrimPath"))
	{
		SAssignNew(ColumnWidget, STextBlock)
		.Text(FText::FromString(Reference->PrimPath.IsEmpty() ? TEXT("(default prim)") : Reference->PrimPath))
		.Margin(FMargin(3.0f, 0.0f, 0.0f, 0.0f))
		.Font(FAppStyle::GetFontStyle(UsdReferencesListConstants::NormalFont));
	}

	return SNew(SBox)
		.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			ColumnWidget
		];
	// clang-format on
}

void SUsdReferencesList::Construct(const FArguments& InArgs)
{
	// clang-format off
	SAssignNew(HeaderRowWidget, SHeaderRow)

	+SHeaderRow::Column(FName(TEXT("ReferenceType")))
	.DefaultLabel(FText::GetEmpty())
	.FixedWidth(24.0f)

	+SHeaderRow::Column(FName(TEXT("AssetPath")))
	.DefaultLabel(LOCTEXT("ReferencedPath", "Referenced layers"))
	.FillWidth(100.f)

	+SHeaderRow::Column(FName(TEXT("PrimPath")))
	.DefaultLabel(LOCTEXT("ReferencedPrim", "Referenced prims"))
	.FillWidth(100.f);

	SListView::Construct
	(
		SListView::FArguments()
		.ListItemsSource(&ViewModel.References)
		.OnGenerateRow(this, &SUsdReferencesList::OnGenerateRow)
		.HeaderRow(HeaderRowWidget)
	);

	OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SUsdReferencesList::ConstructLayerContextMenu);

	SetVisibility(EVisibility::Collapsed); // Start hidden until SetPrimPath displays us
	// clang-format on
}

TSharedRef<ITableRow> SUsdReferencesList::OnGenerateRow(TSharedPtr<FUsdReference> InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUsdReferenceRow, InDisplayNode, OwnerTable);
}

TSharedPtr<SWidget> SUsdReferencesList::ConstructLayerContextMenu()
{
	TSharedRef<SWidget> MenuWidget = SNullWidget::NullWidget;

	FMenuBuilder LayerOptions(true, nullptr);
	LayerOptions.BeginSection(
		"Reference",
		TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
			[this]()
			{
				bool bIsPayload = false;
				for (const TSharedPtr<FUsdReference>& SelectedItem : GetSelectedItems())
				{
					if (SelectedItem->bIsPayload)
					{
						bIsPayload = true;
						break;
					}
				}

				return bIsPayload ? LOCTEXT("Payload_Text", "Payload") : LOCTEXT("Reference_Text", "Reference");
			}
		))
	);
	{
		LayerOptions.AddMenuEntry(
			LOCTEXT("RemoveReference", "Remove"),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda(
				[this]()
				{
					const bool bCanRemove = CanRemoveReference();
					return bCanRemove ? LOCTEXT("RemoveReference_ToolTip", "Remove this reference or payload")
									  : LOCTEXT(
										  "CantRemoveReference_ToolTip",
										  "Cannot remove a reference that was introduced from across another reference"
									  );
				}
			)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdReferencesList::RemoveReference),
				FCanExecuteAction::CreateSP(this, &SUsdReferencesList::CanRemoveReference)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		LayerOptions.AddMenuEntry(
			LOCTEXT("ReloadReference", "Reload"),
			LOCTEXT("ReloadReference_ToolTip", "Reloads this reference or payload"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SUsdReferencesList::ReloadReference),
				FCanExecuteAction::CreateSP(this, &SUsdReferencesList::CanReloadReference)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	LayerOptions.EndSection();

	MenuWidget = LayerOptions.MakeWidget();

	return MenuWidget;
}

bool SUsdReferencesList::CanRemoveReference() const
{
	// From https://openusd.org/dev/usdfaq.html#list-edited-composition-arcs:
	// "The rule, therefore, for meaningfully deleting composition arcs, is that you can only remove an arc
	// if it was introduced in the same layerStack, as discussed with an example here. This means you cannot
	// delete a reference that was introduced from across another reference."
	for (const TSharedPtr<FUsdReference>& SelectedItem : GetSelectedItems())
	{
		if (!SelectedItem->bIntroducedInLocalLayerStack)
		{
			return false;
		}
	}

	return true;
}

void SUsdReferencesList::RemoveReference()
{
	FScopedTransaction Transaction(LOCTEXT("RemoveReferenceTransaction", "Remove reference"));

	for (const TSharedPtr<FUsdReference>& SelectedItem : GetSelectedItems())
	{
		ViewModel.RemoveReference(SelectedItem);
	}
}

bool SUsdReferencesList::CanReloadReference() const
{
	return true;
}

void SUsdReferencesList::ReloadReference()
{
	FScopedTransaction Transaction(LOCTEXT("ReloadReferenceTransaction", "Reload reference"));

	for (const TSharedPtr<FUsdReference>& SelectedItem : GetSelectedItems())
	{
		ViewModel.ReloadReference(SelectedItem);
	}
}

void SUsdReferencesList::SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath)
{
	ViewModel.UpdateReferences(UsdStage, PrimPath);

	SetVisibility(ViewModel.References.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);

	RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE

#endif	  // #if USE_USD_SDK
