// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

#if USE_USD_SDK

#include "USDReferencesViewModel.h"
#include "UsdWrappers/ForwardDeclarations.h"

class SUsdReferenceRow : public SMultiColumnTableRow<TSharedPtr<FUsdReference>>
{
public:
	SLATE_BEGIN_ARGS(SUsdReferenceRow)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FUsdReference> InReference, const TSharedRef<STableViewBase>& OwnerTable);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	FString PrimPath;
	TSharedPtr<FUsdReference> Reference;
};

class SUsdReferencesList : public SListView<TSharedPtr<FUsdReference>>
{
	SLATE_BEGIN_ARGS(SUsdReferencesList)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath);

protected:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FUsdReference> InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr<SWidget> ConstructLayerContextMenu();

	bool CanRemoveReference() const;
	void RemoveReference();

	bool CanReloadReference() const;
	void ReloadReference();

private:
	FUsdReferencesViewModel ViewModel;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
};

#endif	  // #if USE_USD_SDK
