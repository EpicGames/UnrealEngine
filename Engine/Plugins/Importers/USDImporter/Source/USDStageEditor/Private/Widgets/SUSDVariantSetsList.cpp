// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUSDVariantSetsList.h"

#include "Styling/AppStyle.h"
#include "SUSDStageEditorStyle.h"
#include "Widgets/Input/STextComboBox.h"

#if USE_USD_SDK

namespace UsdVariantSetsListConstants
{
	const FMargin LeftRowPadding(6.0f, 0.0f, 2.0f, 0.0f);
	const FMargin RightRowPadding(3.0f, 0.0f, 2.0f, 0.0f);

	const TCHAR* NormalFont = TEXT("PropertyWindow.NormalFont");
}

void SUsdVariantRow::Construct(
	const FArguments& InArgs,
	TSharedPtr<FUsdVariantSetViewModel> InVariantSet,
	const TSharedRef<STableViewBase>& OwnerTable
)
{
	OnVariantSelectionChanged = InArgs._OnVariantSelectionChanged;

	VariantSet = InVariantSet;

	SMultiColumnTableRow<TSharedPtr<FUsdVariantSetViewModel>>::Construct(
		SMultiColumnTableRow<TSharedPtr<FUsdVariantSetViewModel>>::FArguments(),
		OwnerTable
	);
}

TSharedRef<SWidget> SUsdVariantRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SWidget> ColumnWidget = SNullWidget::NullWidget;

	bool bIsLeftRow = true;

	if (ColumnName == TEXT("VariantSetName"))
	{
		// clang-format off
		SAssignNew(ColumnWidget, STextBlock)
		.Text(FText::FromString(VariantSet->SetName))
		.Font(FAppStyle::GetFontStyle(UsdVariantSetsListConstants::NormalFont));
		// clang-format on
	}
	else
	{
		bIsLeftRow = false;

		TSharedPtr<FString>* InitialSelectionPtr = VariantSet->Variants.FindByPredicate(
			[VariantSelection = VariantSet->VariantSelection](const TSharedPtr<FString>& A)
			{
				return A->Equals(*VariantSelection, ESearchCase::IgnoreCase);
			}
		);

		TSharedPtr<FString> InitialSelection = InitialSelectionPtr ? *InitialSelectionPtr : TSharedPtr<FString>();

		// clang-format off
		SAssignNew(ColumnWidget, STextComboBox)
		.OptionsSource(&VariantSet->Variants)
		.InitiallySelectedItem(InitialSelection)
		.OnSelectionChanged(this, &SUsdVariantRow::OnSelectionChanged)
		.Font(FAppStyle::GetFontStyle(UsdVariantSetsListConstants::NormalFont));
		// clang-format on
	}

	// clang-format off
	return SNew(SBox)
		.HeightOverride(FUsdStageEditorStyle::Get()->GetFloat("UsdStageEditor.ListItemHeight"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(bIsLeftRow ? UsdVariantSetsListConstants::LeftRowPadding : UsdVariantSetsListConstants::RightRowPadding)
			.AutoWidth()
			[
				ColumnWidget
			]
		];
	// clang-format on
}

void SUsdVariantRow::OnSelectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	OnVariantSelectionChanged.ExecuteIfBound(VariantSet.ToSharedRef(), NewValue);
}

void SVariantsList::Construct(const FArguments& InArgs)
{
	// clang-format off
	SAssignNew(HeaderRowWidget, SHeaderRow)

	+SHeaderRow::Column(FName(TEXT("VariantSetName")))
	.DefaultLabel(NSLOCTEXT("USDVariantSetsList", "VariantSetName", "Variants"))
	.FillWidth(25.f)

	+SHeaderRow::Column(FName(TEXT("VariantSetSelection")))
	.DefaultLabel(FText::GetEmpty())
	.FillWidth(75.f);

	SListView::Construct
	(
		SListView::FArguments()
		.ListItemsSource(&ViewModel.VariantSets)
		.OnGenerateRow(this, &SVariantsList::OnGenerateRow)
		.HeaderRow(HeaderRowWidget)
	);

	SetVisibility(EVisibility::Collapsed); // Start hidden until SetPrimPath displays us
	// clang-format on
}

TSharedRef<ITableRow> SVariantsList::OnGenerateRow(TSharedPtr<FUsdVariantSetViewModel> InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUsdVariantRow, InDisplayNode, OwnerTable).OnVariantSelectionChanged(this, &SVariantsList::OnVariantSelectionChanged);
}

void SVariantsList::SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* InPrimPath)
{
	ViewModel.UpdateVariantSets(UsdStage, InPrimPath);

	SetVisibility(ViewModel.VariantSets.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);

	RequestListRefresh();
}

void SVariantsList::OnVariantSelectionChanged(const TSharedRef<FUsdVariantSetViewModel>& VariantSet, const TSharedPtr<FString>& NewValue)
{
	VariantSet->SetVariantSelection(NewValue);
}

#endif	  // #if USE_USD_SDK
