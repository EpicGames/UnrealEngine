// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDVariantSetsViewModel.h"
#include "Widgets/Views/SListView.h"

#if USE_USD_SDK

class SUsdVariantRow : public SMultiColumnTableRow<TSharedPtr<FUsdVariantSetViewModel>>
{
public:
	DECLARE_DELEGATE_TwoParams(FOnVariantSelectionChanged, const TSharedRef<FUsdVariantSetViewModel>&, const TSharedPtr<FString>&);

public:
	SLATE_BEGIN_ARGS(SUsdVariantRow)
		: _OnVariantSelectionChanged()
	{
	}

	SLATE_EVENT(FOnVariantSelectionChanged, OnVariantSelectionChanged)

	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<FUsdVariantSetViewModel> InVariantSet, const TSharedRef<STableViewBase>& OwnerTable);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:
	FOnVariantSelectionChanged OnVariantSelectionChanged;

protected:
	void OnSelectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

private:
	FString PrimPath;
	TSharedPtr<FUsdVariantSetViewModel> VariantSet;
};

class SVariantsList : public SListView<TSharedPtr<FUsdVariantSetViewModel>>
{
	SLATE_BEGIN_ARGS(SVariantsList)
	{
	}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	void SetPrimPath(const UE::FUsdStageWeak& UsdStage, const TCHAR* InPrimPath);

protected:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FUsdVariantSetViewModel> InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable);

	void OnVariantSelectionChanged(const TSharedRef<FUsdVariantSetViewModel>& VariantSet, const TSharedPtr<FString>& NewValue);

private:
	FUsdVariantSetsViewModel ViewModel;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
};

#endif	  // #if USE_USD_SDK
