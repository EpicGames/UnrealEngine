// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "USDStageEditorBlueprintLibrary.generated.h"

#define UE_API USDSTAGEEDITOR_API

class AUsdStageActor;
class UUsdStageImportOptions;

/** Library of functions that can be used from scripting to interact with the USD Stage Editor UI */
UCLASS(MinimalAPI, meta = (ScriptName = "UsdStageEditorLibrary"))
class UUsdStageEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Opens the the USD Stage Editor window, or focus it in case it is already open.
	 * @return True if a stage editor window is now opened and available.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Stage Editor")
	static UE_API bool OpenStageEditor();

	/**
	 * Closes the USD Stage Editor window if it is opened. Does nothing in case it is already closed.
	 * @return True if a stage editor window was closed by this action.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Stage Editor")
	static UE_API bool CloseStageEditor();

	/**
	 * Checks to see if an USD Stage Editor window is currently opened.
	 * @return True if a stage editor window is now opened and available.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Stage Editor")
	static UE_API bool IsStageEditorOpened();

	/**
	 * Gets which actor is currently attached to the USD Stage Editor, if any.
	 * May return nullptr in case no actor is attached.
	 * @return The attached stage actor.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Stage Actor")
	static UE_API AUsdStageActor* GetAttachedStageActor();

	/**
	 * Sets which actor is currently attached to the USD Stage Editor.
	 * Provide None/nullptr to clear the attached stage actor.
	 * @param NewActor - The actor to select
	 * @return True in case the new actor was attached to the USD Stage Editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Stage Actor")
	static UE_API bool SetAttachedStageActor(AUsdStageActor* NewActor);

	/**
	 * Returns the full identifiers of all layers that are currently selected on the USD Stage Editor.
	 * @return The list of identifiers (e.g. ["c:/MyFolder/root.usda", "c:/MyFolder/sublayer.usda"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API TArray<FString> GetSelectedLayerIdentifiers();

	/**
	 * Sets the USD Stage Editor layer selection to all occurences of the layers with identifiers
	 * in NewSelection. Provide an empty array to clear the selection.
	 * @param NewSelection - The list of identifiers to select
	 *                       (e.g. ["c:/MyFolder/root.usda", "c:/MyFolder/sublayer.usda"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API void SetSelectedLayerIdentifiers(const TArray<FString>& NewSelection);

	/**
	 * Returns the full paths to all prims currently selected on the USD Stage Editor.
	 * @return The paths of selected prims (e.g. ["/ParentPrim/Mesh", "/Root"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API TArray<FString> GetSelectedPrimPaths();

	/**
	 * Sets the USD Stage Editor prim selection to the prims with paths contained in NewSelection.
	 * Provide an empty array to clear the selection.
	 * @param NewSelection - The list of prim paths to select (e.g. ["/ParentPrim/Mesh", "/Root"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API void SetSelectedPrimPaths(const TArray<FString>& NewSelection);

	/**
	 * Returns the names of the currently selected properties on the right panel of the USD Stage Editor.
	 * @return The names of selected properties (e.g. ["points", "displayColor"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API TArray<FString> GetSelectedPropertyNames();

	/**
	 * Sets the USD Stage Editor property selection to the properties with names contained in NewSelection.
	 * Provide an empty array to clear the selection.
	 * @param NewSelection - The list of property names to select (e.g. ["points", "displayColor"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API void SetSelectedPropertyNames(const TArray<FString>& NewSelection);

	/**
	 * Returns the names of the currently selected property metadata entries on the right panel of the USD Stage
	 * Editor.
	 * @return The names of selected metadata (e.g. ["documentation", "typeName"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API TArray<FString> GetSelectedPropertyMetadataNames();

	/**
	 * Sets the USD Stage Editor property metadata selection to the entries with names contained in NewSelection.
	 * Provide an empty array to clear the selection.
	 * @param NewSelection - The list of property names to select (e.g. ["documentation", "typeName"])
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Selection")
	static UE_API void SetSelectedPropertyMetadataNames(const TArray<FString>& NewSelection);

	/**
	 * Creates a new memory-only layer and opens an USD Stage with that layer as its root.
	 * Corresponds to the "File -> New" action on the USD Stage Editor menu bar.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileNew();

	/**
	 * Opens an USD Stage from a file on disk.
	 * Corresponds to the "File -> Open" action on the USD Stage Editor menu bar.
	 * @param FilePath - File path to the USD layer to open (e.g. "C:/Folder/MyFile.usda").
	 *                   If this path is the empty string a dialog will be shown to let the user pick the file.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileOpen(const FString& FilePath);

	/**
	 * Saves the currently opened USD Stage back to disk, or to a new file in case it hasn't been saved yet.
	 * Corresponds to the "File -> Save" action on the USD Stage Editor menu bar.
	 * @param OutputFilePathIfUnsaved - File path (e.g. "C:/Folder/MyFile.usda") to use when the currently opened
	 *									stage hasn't been saved yet.
	 *                                  If this path is the empty string a dialog will be shown to let the user pick
	 *                                  the file.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileSave(const FString& OutputFilePathIfUnsaved);

	/**
	 * Exports all layers of the currently opened USD Stage to brand new files in a new location.
	 * Corresponds to the "File -> Export -> All Layers" action on the USD Stage Editor menu bar.
	 * @param OutputDirectory - Directory path (e.g. "C:/ExportFolder/") to receive the exported files.
	 *                          If this path is the empty string a dialog will be shown to let the user pick the
	 *                          directory.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileExportAllLayers(const FString& OutputDirectory);

	/**
	 * Exports the currently opened USD Stage to a single "flattened" USD layer.
	 * Corresponds to the "File -> Export -> Flattened stage" action on the USD Stage Editor menu bar.
	 * @param OutputLayer - File path (e.g. "C:/ExportFolder/out.usda") to export the flattened layer to.
	 *                      If this path is the empty string a dialog will be shown to let the user pick the file.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileExportFlattenedStage(const FString& OutputLayer);

	/**
	 * Exports the currently opened USD Stage to a single "flattened" USD layer.
	 * Corresponds to the "File -> Export -> Flattened layer stack" action on the USD Stage Editor menu bar.
	 * @param OutputLayer - File path (e.g. "C:/ExportFolder/out.usda") to export the flattened layer to.
	 *                      If this path is the empty string a dialog will be shown to let the user pick the file.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileExportFlattenedLayerStack(const FString& OutputLayer);

	/**
	 * Reloads all layers of the current USD Stage.
	 * Corresponds to the "File -> Reload" action on the USD Stage Editor menu bar.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileReload();

	/**
	 * Resets the state of the current USD Stage (which layers are muted, the current edit target, etc.).
	 * Corresponds to the "File -> Reset state" action on the USD Stage Editor menu bar.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileReset();

	/**
	 * Closes the currently opened USD Stage (by clearing the attached Stage Actor's RootLayer property).
	 * Corresponds to the "File -> Close" action on the USD Stage Editor menu bar.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void FileClose();

	/**
	 * Imports the currently opened USD Stage into persistent UE assets, actors and components on the level.
	 * Corresponds to the "Actions -> Import" action on the USD Stage Editor menu bar.
	 * @param OutputContentFolder - Content path (e.g. "/Game/Imports") to receive the imported assets.
	 *								If this path is the empty string a dialog will be shown to let the user pick
	 *								the folder.
	 * @papam Options - Options object to use for the import.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void ActionsImport(const FString& OutputContentFolder, UUsdStageImportOptions* Options);

	/**
	 * Exports the currently selected layers on the USD Stage Editor to brand new files in a new location.
	 * Corresponds to right-clicking selected layers on the USD Stage Editor and picking "Export".
	 * @param OutputDirectory - Directory path (e.g. "C:/ExportFolder/") to receive the exported files.
	 *                          If this path is the empty string a dialog will be shown to let the user pick the
	 *                          directory.
	 */
	UFUNCTION(BlueprintCallable, Category = "USD|Menu Actions")
	static UE_API void ExportSelectedLayers(const FString& OutputDirectory);
};

#undef UE_API
