// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/ForwardDeclarations.h"

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"

class AUsdStageActor;
class UUsdStageImportOptions;
namespace UE
{
	class FUsdPrim;
	class FUsdAttribute;
}

class IUsdStageEditorModule : public IModuleInterface
{
public:

	/**
	 * Check out USDStageEditorBlueprintLibrary.h for documentation regarding these functions.
	 */

	bool OpenStageEditor() const;
	bool CloseStageEditor() const;
	bool IsStageEditorOpened() const;

	AUsdStageActor* GetAttachedStageActor() const;
	bool SetAttachedStageActor(AUsdStageActor* NewActor) const;

	TArray<UE::FSdfLayer> GetSelectedLayers() const;
	void SetSelectedLayers(const TArray<UE::FSdfLayer>& NewSelection) const;

	TArray<UE::FUsdPrim> GetSelectedPrims() const;
	void SetSelectedPrims(const TArray<UE::FUsdPrim>& NewSelection) const;

	TArray<FString> GetSelectedPropertyNames() const;
	void SetSelectedPropertyNames(const TArray<FString>& NewSelection) const;

	TArray<FString> GetSelectedPropertyMetadataNames() const;
	void SetSelectedPropertyMetadataNames(const TArray<FString>& NewSelection) const;

	// For all of these, providing an empty path will cause us to pop open a dialog to let the user pick the path
	// instead.
	void FileNew() const;
	void FileOpen(const FString& FilePath) const;
	void FileSave(const FString& OutputFilePathIfUnsaved) const;
	void FileExportAllLayers(const FString& OutputDirectory) const;
	void FileExportFlattenedStage(const FString& OutputLayer) const;
	void FileExportFlattenedLayerStack(const FString& OutputLayer) const;
	void FileReload() const;
	void FileReset() const;
	void FileClose() const;
	void ActionsImport(const FString& OutputContentFolder, UUsdStageImportOptions* Options) const;
	void ExportSelectedLayers(const FString& OutputLayerOrDirectory) const;
};
