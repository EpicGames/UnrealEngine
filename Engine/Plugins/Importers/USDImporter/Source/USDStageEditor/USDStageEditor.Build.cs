// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDStageEditor : ModuleRules
	{
		public USDStageEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DesktopPlatform",
					"DesktopWidgets",
					"EditorFramework",
					"Engine",
					"InputCore",
					"LevelEditor",
					"LevelSequence",
					"LiveLinkEditor", // For SLiveLinkSubjectRepresentationPicker
					"LiveLinkInterface", // For the ULiveLinkAnimation/TransformRole classes
					"Projects", // So that we can use the IPluginManager, required for our custom style
					"SceneOutliner",
					"Slate",
					"SlateCore",
					"UnrealEd",
					"UnrealUSDWrapper",
					"USDClasses",
					"USDSchemas",
					"USDStage",
					"USDStageEditorViewModels",
					"USDStageImporter",
					"USDUtilities",
					"WorkspaceMenuStructure",
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
