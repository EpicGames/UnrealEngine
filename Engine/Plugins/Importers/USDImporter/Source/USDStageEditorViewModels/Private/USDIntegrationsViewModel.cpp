// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDIntegrationsViewModel.h"

#include "USDIntegrationUtils.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

void FUsdIntegrationsViewModel::UpdateAttributes(const UE::FUsdStageWeak& InUsdStage, const TCHAR* InPrimPath)
{
#if USE_USD_SDK
	UsdStage = InUsdStage;
	PrimPath = InPrimPath;

	Attributes.Reset();

	if (!UsdStage || PrimPath.IsEmpty() || UE::FSdfPath{InPrimPath}.IsAbsoluteRootPath())
	{
		return;
	}

	TArray<FString> AttributeNames;

	TFunction<void(bool, bool)> AddLiveLinkAttributes = [&AttributeNames](bool bHasLiveLink, bool bIsSkeletal)
	{
		if (bHasLiveLink)
		{
			if (bIsSkeletal)
			{
				AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealAnimBlueprintPath));
			}

			AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkSubjectName));
			AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealLiveLinkEnabled));
		}
	};

	TFunction<void(bool, bool)> AddControlRigAttributes = [&AttributeNames](bool bHasControlRig, bool bIsSkeletal)
	{
		if (bHasControlRig && bIsSkeletal)
		{
			AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigPath));
			AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealUseFKControlRig));
			AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReduceKeys));
			AttributeNames.Add(*UsdToUnreal::ConvertToken(UnrealIdentifiers::UnrealControlRigReductionTolerance));
		}
	};

	if (UE::FUsdPrim Prim = InUsdStage.GetPrimAtPath(UE::FSdfPath{InPrimPath}))
	{
		if (!Prim.IsPseudoRoot())
		{
			const bool bHasLiveLink = UsdUtils::PrimHasSchema(Prim, UnrealIdentifiers::LiveLinkAPI);
			const bool bHasControlRig = UsdUtils::PrimHasSchema(Prim, UnrealIdentifiers::ControlRigAPI);
			const bool bIsSkeletal = Prim.IsA(TEXT("SkelRoot")) || Prim.IsA(TEXT("Skeleton"));

			AddLiveLinkAttributes(bHasLiveLink, bIsSkeletal);
			AddControlRigAttributes(bHasControlRig, bIsSkeletal);

			for (const FString& AttributeName : AttributeNames)
			{
				if (UE::FUsdAttribute Attr = Prim.GetAttribute(*AttributeName))
				{
					Attributes.Add(MakeShared<UE::FUsdAttribute>(Attr));
				}
			}
		}
	}
#endif	  // USE_USD_SDK
}
