// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDLayersViewModel.h"

#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/layerStack.h"
#include "pxr/usd/sdf/layer.h"
#include "pxr/usd/sdf/layerTree.h"
#include "pxr/usd/sdf/layerUtils.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

FUsdLayerViewModel::FUsdLayerViewModel(
	FUsdLayerViewModel* InParentItem,
	const UE::FUsdStageWeak& InUsdStage,
	const UE::FUsdStageWeak& InIsolatedStage,
	const FString& InLayerIdentifier
)
	: LayerModel(MakeShared<FUsdLayerModel>())
	, ParentItem(InParentItem)
	, UsdStage(InUsdStage)
	, IsolatedStage(InIsolatedStage)
	, LayerIdentifier(InLayerIdentifier)
{
	RefreshData();
}

bool FUsdLayerViewModel::IsValid() const
{
	return (bool)UsdStage && (!ParentItem || !ParentItem->LayerIdentifier.Equals(LayerIdentifier, ESearchCase::Type::IgnoreCase));
}

TArray<TSharedRef<FUsdLayerViewModel>> FUsdLayerViewModel::GetChildren()
{
	if (!IsValid())
	{
		return {};
	}

	bool bNeedsRefresh = false;

#if USE_USD_SDK
	{
		FScopedUsdAllocs UsdAllocs;

		pxr::SdfLayerRefPtr UsdLayer(GetLayer());

		if (UsdLayer)
		{
			int32 SubLayerIndex = 0;
			for (const std::string& SubLayerPath : UsdLayer->GetSubLayerPaths())
			{
				const FString SubLayerIdentifier = UsdToUnreal::ConvertString(pxr::SdfComputeAssetPathRelativeToLayer(UsdLayer, SubLayerPath));

				if (!Children.IsValidIndex(SubLayerIndex)
					|| !Children[SubLayerIndex]->LayerIdentifier.Equals(SubLayerIdentifier, ESearchCase::Type::IgnoreCase))
				{
					Children.Reset();
					bNeedsRefresh = true;
					break;
				}

				++SubLayerIndex;
			}

			if (!bNeedsRefresh && SubLayerIndex < Children.Num())
			{
				Children.Reset();
				bNeedsRefresh = true;
			}
		}
	}
#endif	  // #if USE_USD_SDK

	if (bNeedsRefresh)
	{
		FillChildren();
	}

	return Children;
}

void FUsdLayerViewModel::FillChildren()
{
	Children.Reset();

	if (!IsValid())
	{
		return;
	}

#if USE_USD_SDK
	{
		FScopedUsdAllocs UsdAllocs;

		pxr::SdfLayerRefPtr UsdLayer(GetLayer());

		if (UsdLayer)
		{
			TSet<FString> AllLayerIdentifiers;

			FUsdLayerViewModel* CurrentItem = this;
			while (CurrentItem)
			{
				AllLayerIdentifiers.Add(CurrentItem->LayerIdentifier);
				CurrentItem = CurrentItem->ParentItem;
			}

			for (std::string SubLayerPath : UsdLayer->GetSubLayerPaths())
			{
				FString AssetPathRelativeToLayer = UsdToUnreal::ConvertString(pxr::SdfComputeAssetPathRelativeToLayer(UsdLayer, SubLayerPath));

				// Prevent infinite recursions if a sublayer refers to a parent of the same hierarchy
				if (!AllLayerIdentifiers.Contains(AssetPathRelativeToLayer))
				{
					Children.Add(MakeShared<FUsdLayerViewModel>(this, UsdStage, IsolatedStage, AssetPathRelativeToLayer));
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK
}

void FUsdLayerViewModel::RefreshData()
{
	if (!IsValid())
	{
		return;
	}

	Children = GetChildren();

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdStageRefPtr UsdStageRef{UsdStage};
	pxr::UsdStageRefPtr IsolatedStageRef{IsolatedStage};
	pxr::UsdStageRefPtr ActiveStage = IsolatedStage ? IsolatedStage : UsdStageRef;

	const std::string UsdLayerIdentifier = UnrealToUsd::ConvertString(*LayerIdentifier).Get();

	LayerModel->DisplayName = UsdToUnreal::ConvertString(pxr::SdfLayer::GetDisplayNameFromIdentifier(UsdLayerIdentifier));

	pxr::SdfLayerRefPtr UsdLayer = UE::FSdfLayer::FindOrOpen(*LayerIdentifier);
	if (UsdLayer)
	{
		// Don't show session layers (always anonymous) as dirty, but if the root layer of the stage itself is anonymous
		// it may be a File->New stage, so we want to show it as dirty to indicate it hasn't been saved yet
		LayerModel->bIsDirty = UsdLayer->IsDirty() && (!UsdLayer->IsAnonymous() || ActiveStage->GetRootLayer() == UsdLayer);
	}

	const bool bIsIsolatedRootLayer = IsolatedStageRef && IsolatedStageRef->GetRootLayer() == UsdLayer;

	// Note: Originally this was a proper check to see if UsdLayer was inside IsolatedStage's layer stack. However, this
	// breaks down if the layer is also muted: When that happens the stages basically pretend the layer doesn't exist,
	// and so any muted layer on the base stage would behave as if it was "non-isolated"
	LayerModel->bIsInIsolatedStage = !IsolatedStageRef || bIsIsolatedRootLayer || (ParentItem && ParentItem->IsInIsolatedStage());

	const pxr::SdfLayerHandle& EditTargetLayer = UsdStageRef->GetEditTarget().GetLayer();
	const pxr::SdfLayerHandle& FocusedEditTarget = IsolatedStageRef ? IsolatedStageRef->GetEditTarget().GetLayer() : pxr::SdfLayerHandle{};

	LayerModel->bIsEditTarget = (IsolatedStageRef) ? LayerModel->bIsInIsolatedStage && FocusedEditTarget == UsdLayer : EditTargetLayer == UsdLayer;

	LayerModel->bIsMuted = (IsolatedStageRef && LayerModel->bIsInIsolatedStage)
							   // If we isolating, we're only muted if we're muted on the isolated stage
							   ? IsolatedStageRef->IsLayerMuted(UsdLayerIdentifier)
							   // Otherwise we're muted any time we're muted in the base stage
							   : UsdStageRef->IsLayerMuted(UsdLayerIdentifier);

	for (const TSharedRef<FUsdLayerViewModel>& Child : Children)
	{
		Child->RefreshData();
	}
#endif	  // #if USE_USD_SDK
}

UE::FSdfLayer FUsdLayerViewModel::GetLayer() const
{
	return UE::FSdfLayer::FindOrOpen(*LayerIdentifier);
}

FText FUsdLayerViewModel::GetDisplayName() const
{
	FString DisplayName = LayerModel->DisplayName;

	if (IsLayerDirty())
	{
		DisplayName += TEXT("*");
	}

	return FText::FromString(DisplayName);
}

bool FUsdLayerViewModel::IsLayerMuted() const
{
	return LayerModel->bIsMuted;
}

bool FUsdLayerViewModel::CanMuteLayer() const
{
	if (!IsValid())
	{
		return false;
	}

	UE::FUsdStageWeak AffectedStage = IsolatedStage ? IsolatedStage : UsdStage;

	return !AffectedStage.GetRootLayer().GetIdentifier().Equals(LayerIdentifier, ESearchCase::Type::IgnoreCase) && !LayerModel->bIsEditTarget;
}

void FUsdLayerViewModel::ToggleMuteLayer()
{
	if (!IsValid() || !CanMuteLayer())
	{
		return;
	}

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	const TUsdStore<std::string> UsdLayerIdentifier = UnrealToUsd::ConvertString(*LayerIdentifier);

	pxr::UsdStageRefPtr CurrentStage(IsolatedStage ? IsolatedStage : UsdStage);

	if (CurrentStage->IsLayerMuted(UsdLayerIdentifier.Get()))
	{
		CurrentStage->UnmuteLayer(UsdLayerIdentifier.Get());
	}
	else
	{
		CurrentStage->MuteLayer(UsdLayerIdentifier.Get());
	}

	// We have to refresh all of our view models here, because this layer can appear more than once on the layer
	// stack and we want all of them to show the updated muted/unmuted icon
	FUsdLayerViewModel* Ancestor = this;
	while (true)
	{
		FUsdLayerViewModel* AncestorParent = Ancestor->ParentItem;
		if (!AncestorParent)
		{
			break;
		}

		Ancestor = AncestorParent;
	}
	if (Ancestor)
	{
		Ancestor->RefreshData();
	}
#endif	  // #if USE_USD_SDK
}

bool FUsdLayerViewModel::IsEditTarget() const
{
	return LayerModel->bIsEditTarget;
}

bool FUsdLayerViewModel::CanEditLayer() const
{
	return !LayerModel->bIsMuted;
}

bool FUsdLayerViewModel::EditLayer()
{
	UE::FSdfLayer Layer(GetLayer());

	UE::FUsdStageWeak AffectedStage = IsolatedStage ? IsolatedStage : UsdStage;

	if (!AffectedStage || !Layer || !CanEditLayer())
	{
		return false;
	}

	AffectedStage.SetEditTarget(Layer);
	RefreshData();

	return true;
}

bool FUsdLayerViewModel::IsInIsolatedStage() const
{
	return LayerModel->bIsInIsolatedStage;
}

void FUsdLayerViewModel::AddSubLayer(const TCHAR* SubLayerIdentifier)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;
	UsdUtils::InsertSubLayer(pxr::SdfLayerRefPtr(GetLayer()), SubLayerIdentifier);
#endif	  // #if USE_USD_SDK
}

void FUsdLayerViewModel::NewSubLayer(const TCHAR* SubLayerIdentifier)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;
	UsdUtils::CreateNewLayer(pxr::UsdStageRefPtr(UsdStage), pxr::SdfLayerRefPtr(GetLayer()), SubLayerIdentifier);
#endif	  // #if USE_USD_SDK
}

bool FUsdLayerViewModel::RemoveSubLayer(int32 SubLayerIndex)
{
	bool bLayerRemoved = false;

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::SdfLayerRefPtr UsdLayer(GetLayer());

	if (UsdLayer)
	{
		UsdLayer->RemoveSubLayerPath(SubLayerIndex);
		bLayerRemoved = true;
	}
#endif	  // #if USE_USD_SDK

	return bLayerRemoved;
}

bool FUsdLayerViewModel::IsLayerDirty() const
{
	return LayerModel->bIsDirty;
}

bool FUsdLayerViewModel::CanReload() const
{
	return !GetLayer().IsAnonymous();
}

void FUsdLayerViewModel::Reload()
{
	const bool bForce = true;
	GetLayer().Reload(bForce);
}
