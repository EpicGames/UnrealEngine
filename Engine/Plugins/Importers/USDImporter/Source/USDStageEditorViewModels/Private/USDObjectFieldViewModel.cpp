// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDObjectFieldViewModel.h"

#include "UnrealUSDWrapper.h"
#include "USDAttributeUtils.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"
#include "USDValueConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/VtValue.h"

#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Widgets/Notifications/SNotificationList.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/base/tf/stringUtils.h"
#include "pxr/base/vt/value.h"
#include "pxr/usd/kind/registry.h"
#include "pxr/usd/sdf/types.h"
#include "pxr/usd/usd/attribute.h"
#include "pxr/usd/usd/common.h"
#include "pxr/usd/usd/editContext.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/relationship.h"
#include "pxr/usd/usdGeom/tokens.h"
#include "pxr/usd/usdPhysics/tokens.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#define LOCTEXT_NAMESPACE "USDObjectFieldViewModel"

FUsdObjectFieldViewModel::FUsdObjectFieldViewModel(FUsdObjectFieldsViewModel* InOwner)
	: Owner(InOwner)
{
}

TArray<TSharedPtr<FString>> FUsdObjectFieldViewModel::GetDropdownOptions() const
{
#if USE_USD_SDK
	if (Label == TEXT("kind"))
	{
		TArray<TSharedPtr<FString>> Options;
		{
			FScopedUsdAllocs Allocs;

			std::vector<pxr::TfToken> Kinds = pxr::KindRegistry::GetAllKinds();
			Options.Reserve(Kinds.size());

			for (const pxr::TfToken& Kind : Kinds)
			{
				Options.Add(MakeShared<FString>(UsdToUnreal::ConvertToken(Kind)));
			}

			// They are supposed to be in an unspecified order, so let's make them consistent
			Options.Sort(
				[](const TSharedPtr<FString>& A, const TSharedPtr<FString>& B)
				{
					return A.IsValid() && B.IsValid() && (*A < *B);
				}
			);
		}
		return Options;
	}
	else if (Label == UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->purpose))
	{
		return TArray<TSharedPtr<FString>>{
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->default_)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->proxy)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->render)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->guide)),
		};
	}
	else if (Label == UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->upAxis))
	{
		return TArray<TSharedPtr<FString>>{
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->y)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->z)),
		};
	}
	else if (Label == UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->visibility))
	{
		return TArray<TSharedPtr<FString>>{
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->inherited)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdGeomTokens->invisible)),
		};
	}
	else if (Label == UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->physicsApproximation))
	{
		return TArray<TSharedPtr<FString>>{
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->none)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->convexDecomposition)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->convexHull)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->boundingSphere)),
			MakeShared<FString>(UsdToUnreal::ConvertToken(pxr::UsdPhysicsTokens->boundingCube)),
			// meshSimplification will get mapped functionally to convexDecomposition
		};
	}

#endif	  // #if USE_USD_SDK

	return {};
}

void FUsdObjectFieldViewModel::SetAttributeValue(const UsdUtils::FConvertedVtValue& InValue)
{
	Owner->SetFieldValue(Label, InValue);
}

template<typename T>
void FUsdObjectFieldsViewModel::CreateField(
	EObjectFieldType Type,
	const FString& FieldName,
	const T& Value,
	UsdUtils::EUsdBasicDataTypes SourceType,
	const FString& ValueRole,
	bool bReadOnly
)
{
	UsdUtils::FConvertedVtValue VtValue;
	VtValue.Entries = {{UsdUtils::FConvertedVtValueComponent{TInPlaceType<T>(), Value}}};
	VtValue.SourceType = SourceType;

	FUsdObjectFieldViewModel Property(this);
	Property.Type = Type;
	Property.Label = FieldName;
	Property.Value = VtValue;
	Property.ValueRole = ValueRole;
	Property.bReadOnly = bReadOnly;

	Fields.Add(MakeSharedUnreal<FUsdObjectFieldViewModel>(MoveTemp(Property)));
}

void FUsdObjectFieldsViewModel::CreateField(EObjectFieldType Type, const FString& FieldName, const UsdUtils::FConvertedVtValue& Value, bool bReadOnly)
{
	FUsdObjectFieldViewModel Property(this);
	Property.Type = Type;
	Property.Label = FieldName;
	Property.Value = Value;
	Property.bReadOnly = bReadOnly;

	Fields.Add(MakeSharedUnreal<FUsdObjectFieldViewModel>(MoveTemp(Property)));
}

void FUsdObjectFieldsViewModel::SetFieldValue(const FString& FieldName, const UsdUtils::FConvertedVtValue& InValue)
{
	bool bSuccess = false;

#if USE_USD_SDK
	if (!UsdStage)
	{
		return;
	}

	// Avoid a reentry here because of this sequence of events:
	//  - The user changes one of the prim property input boxes and presses Enter, calling e.g. OnSpinboxValueCommitted;
	//  - We set the new value directly on the prim in the stage (within this function), triggering a USD stage change;
	//  - USD sends a change notice;
	//  - We respond to the notice on AUsdStageActor::OnUsdObjectsChanged;
	//  - The body of that function uses a FScopedSlowTask to show a progress bar;
	//  - The progress bar being displayed steals focus from the input box;
	//  - The input box losing focus automatically triggers OnSpinboxValueCommitted again!
	//  - We set the new value directly on the prim in the stage, again. Even though it's the same value, we get a new notice!
	//  - etc.
	static bool bIsReentrant = false;
	if (bIsReentrant)
	{
		return;
	}
	TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

	// Transact here as setting this attribute may trigger USD events that affect assets/components
	FScopedTransaction Transaction(FText::Format(
		LOCTEXT("SetFieldValue", "Set value for field '{0}' of prim '{1}'"),
		FText::FromString(FieldName),
		FText::FromString(ObjectPath)
	));

	const bool bIsStageAttribute = ObjectPath == TEXT("/") || ObjectPath.IsEmpty();
	bool bIsPropertyPath = false;

	UE::FVtValue VtValue;
	if (UnrealToUsd::ConvertValue(InValue, VtValue))
	{
		if (bIsStageAttribute)
		{
			FScopedUsdAllocs UsdAllocs;

			// To set stage metadata the edit target must be the root or session layer
			pxr::UsdStageRefPtr Stage{UsdStage};
			pxr::UsdEditContext EditContext{Stage, Stage->GetRootLayer()};
			bSuccess = UsdStage.SetMetadata(*FieldName, VtValue);
		}
		else if (UE::FUsdPrim UsdPrim = UsdStage.GetPrimAtPath(UE::FSdfPath(*ObjectPath)))
		{
			if (UsdUtils::NotifyIfInstanceProxy(UsdPrim))
			{
				return;
			}

			FScopedUsdAllocs UsdAllocs;

			pxr::UsdPrim PxrUsdPrim{UsdPrim};
			pxr::TfToken FieldNameToken = UnrealToUsd::ConvertToken(*FieldName).Get();

			// Single value, single component of FString
			if (FieldName == TEXT("kind") && InValue.Entries.Num() == 1 && InValue.Entries[0].Num() == 1 && InValue.Entries[0][0].IsType<FString>())
			{
				FString Kind = InValue.Entries[0][0].Get<FString>();
				if (Kind.IsEmpty())
				{
					bSuccess = IUsdPrim::ClearKind(UsdPrim);
				}
				else
				{
					bSuccess = IUsdPrim::SetKind(UsdPrim, UnrealToUsd::ConvertToken(*Kind).Get());
				}
			}
			else if (UE::FUsdAttribute Attribute = UsdPrim.GetAttribute(*FieldName))
			{
				bSuccess = Attribute.Set(VtValue);
				UsdUtils::NotifyIfOverriddenOpinion(Attribute);
			}
			else if (pxr::UsdRelationship Relationship = PxrUsdPrim.GetRelationship(FieldNameToken))
			{
				if (InValue.Entries.Num() == 1 && InValue.Entries[0].Num() == 1 && InValue.Entries[0][0].IsType<FString>())
				{
					const FString& NewValue = InValue.Entries[0][0].Get<FString>();

					// We only allow editing relationship attributes that have a single target anyway
					bSuccess = Relationship.SetTargets(std::vector<pxr::SdfPath>{UnrealToUsd::ConvertPath(*NewValue).Get()});
					UsdUtils::NotifyIfOverriddenOpinion(Relationship);
				}
			}
			else if (PxrUsdPrim.HasMetadata(FieldNameToken))
			{
				bSuccess = PxrUsdPrim.SetMetadata(FieldNameToken, VtValue.GetUsdValue());
			}
		}
		// Trying to set property metadata
		else if (!ObjectPath.IsEmpty())
		{
			FScopedUsdAllocs Allocs;

			pxr::SdfPath UsdPath = UnrealToUsd::ConvertPath(*ObjectPath).Get();
			if (UsdPath.IsPropertyPath())
			{
				bIsPropertyPath = true;

				pxr::SdfPath PxrPrimPath = UsdPath.GetPrimPath();
				pxr::TfToken PropertyName = UsdPath.GetNameToken();

				pxr::UsdStageRefPtr PxrUsdStage{UsdStage};
				if (pxr::UsdPrim Prim = PxrUsdStage->GetPrimAtPath(PxrPrimPath))
				{
					if (UsdUtils::NotifyIfInstanceProxy(Prim))
					{
						return;
					}

					if (pxr::UsdProperty Property = Prim.GetProperty(PropertyName))
					{
						pxr::TfToken FieldNameToken = UnrealToUsd::ConvertToken(*FieldName).Get();
						if (Property.HasMetadata(FieldNameToken))
						{
							bSuccess = Property.SetMetadata(FieldNameToken, VtValue.GetUsdValue());
						}
					}
				}
			}
		}
	}

	if (!bSuccess)
	{
		const FText ErrorMessage = FText::Format(
			LOCTEXT("FailToSetFieldMessage", "Failed to set '{0}' on {1} '{2}'"),
			FText::FromString(FieldName),
			FText::FromString(
				bIsStageAttribute ? TEXT("stage")
				: bIsPropertyPath ? TEXT("property")
								  : TEXT("prim")
			),
			FText::FromString(bIsStageAttribute ? UsdStage.GetRootLayer().GetRealPath() : ObjectPath)
		);

		FNotificationInfo ErrorToast(ErrorMessage);
		ErrorToast.ExpireDuration = 5.0f;
		ErrorToast.bFireAndForget = true;
		ErrorToast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		FSlateNotificationManager::Get().AddNotification(ErrorToast);

		USD_LOG_USERWARNING(ErrorMessage);
	}
#endif	  // #if USE_USD_SDK
}

void FUsdObjectFieldsViewModel::Refresh(const UE::FUsdStageWeak& InUsdStage, const TCHAR* InObjectPath, float TimeCode)
{
	FScopedUnrealAllocs UnrealAllocs;

	UsdStage = InUsdStage;
	ObjectPath = InObjectPath;

	Fields.Reset();

#if USE_USD_SDK
	TFunction<void(const FString&, const pxr::VtValue&)> DisplayFieldAsString = [this](const FString& FieldName, const pxr::VtValue& Value)
	{
		FString Stringified;

		if (Value.IsArrayValued())
		{
			Stringified = FString::Printf(TEXT("%zu elements: "), Value.GetArraySize());
		}

		// This array it's too large to even stringify fast enough, so for now just show the element count
		if (Value.IsArrayValued() && Value.GetArraySize() > 5000)
		{
			Stringified += TEXT("[too many entries to expand]");
		}
		else
		{
			Stringified += UsdToUnreal::ConvertString(pxr::TfStringify(Value));

			// STextBlock can get very slow calculating its desired size for very long string so chop it if needed
			const int32 MaxValueLength = 300;
			if (Stringified.Len() > MaxValueLength)
			{
				Stringified.LeftInline(MaxValueLength);
				Stringified.Append(TEXT("...]"));
			}
		}

		const bool bAttrReadOnly = true;
		CreateField(EObjectFieldType::Metadata, FieldName, Stringified, UsdUtils::EUsdBasicDataTypes::String, TEXT(""), bAttrReadOnly);
	};

	// Lambda to add rows to the model based on USD object metadata. Extracted here because we can reuse it for
	// properties and prims
	TFunction<void(pxr::UsdObject)> AddMetadataFields = [this, &DisplayFieldAsString](pxr::UsdObject Object)
	{
		std::map<class pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan> MetadataMap = Object.GetAllMetadata();
		for (std::map<class pxr::TfToken, pxr::VtValue, pxr::TfDictionaryLessThan>::iterator Iter = MetadataMap.begin(); Iter != MetadataMap.end();
			 ++Iter)
		{
			const pxr::TfToken& Key = Iter->first;
			const pxr::VtValue& Value = Iter->second;

			FString MetadataName = UsdToUnreal::ConvertToken(Key);

			UsdUtils::FConvertedVtValue ConvertedValue;
			if (UsdToUnreal::ConvertValue(UE::FVtValue{Value}, ConvertedValue))
			{
				if (Value.IsArrayValued())
				{
					DisplayFieldAsString(MetadataName, Value);
				}
				else
				{
					const bool bReadOnly = false;
					CreateField(EObjectFieldType::Metadata, MetadataName, ConvertedValue, bReadOnly);
				}
			}
			// If we don't know how to display this type, try stringifying it
			else
			{
				DisplayFieldAsString(MetadataName, Value);
			}
		}
	};

	if (UsdStage)
	{
		// Show info about the stage
		if (ObjectPath.Equals(TEXT("/")) || ObjectPath.IsEmpty())
		{
			const bool bReadOnly = true;
			const FString Role = TEXT("");
			CreateField(
				EObjectFieldType::Metadata,
				TEXT("path"),
				UsdStage.GetRootLayer().GetRealPath(),
				UsdUtils::EUsdBasicDataTypes::String,
				Role,
				bReadOnly
			);

			FScopedUsdAllocs UsdAllocs;

			pxr::UsdStageRefPtr Stage{UsdStage};

			std::vector<pxr::TfToken> TokenVector = pxr::SdfSchema::GetInstance().GetMetadataFields(pxr::SdfSpecType::SdfSpecTypePseudoRoot);
			for (const pxr::TfToken& Token : TokenVector)
			{
				pxr::VtValue VtValue;

				if (Token == pxr::SdfFieldKeys->TimeCodesPerSecond)
				{
					// There is some fallback logic here (e.g. using framesPerSecond), so don't fetch
					// the raw metadata value generically as that will ignore it
					VtValue = Stage->GetTimeCodesPerSecond();
				}
				else
				{
					bool bSuccess = Stage->GetMetadata(Token, &VtValue);
					if (!bSuccess)
					{
						VtValue = pxr::VtValue{};
					}
				}

				FString FieldName = UsdToUnreal::ConvertToken(Token);

				UsdUtils::FConvertedVtValue ConvertedValue;
				if (!UsdToUnreal::ConvertValue(UE::FVtValue{VtValue}, ConvertedValue))
				{
					continue;
				}

				const bool bAttrReadOnly = ConvertedValue.bIsArrayValued;
				CreateField(EObjectFieldType::Metadata, FieldName, ConvertedValue, bAttrReadOnly);
			}
		}
		// Show info about a prim
		else if (UE::FUsdPrim UsdPrim = UsdStage.GetPrimAtPath(UE::FSdfPath{*ObjectPath}))
		{
			// For now we can't rename/reparent prims through this
			const bool bPrimReadOnly = true;
			const FString Role = TEXT("");
			CreateField(
				EObjectFieldType::Metadata,
				TEXT("name"),
				UsdPrim.GetName().ToString(),
				UsdUtils::EUsdBasicDataTypes::String,
				Role,
				bPrimReadOnly
			);
			CreateField(EObjectFieldType::Metadata, TEXT("path"), ObjectPath, UsdUtils::EUsdBasicDataTypes::String, Role, bPrimReadOnly);

			// If the prim doesn't have any "kind" authored show a combobox anyway so that we can use this to author new kinds
			if (!UsdUtils::HasAuthoredKind(UsdPrim))
			{
				CreateField(
					EObjectFieldType::Metadata,
					TEXT("kind"),
					UsdToUnreal::ConvertToken(IUsdPrim::GetKind(UsdPrim)),
					UsdUtils::EUsdBasicDataTypes::Token
				);
			}

			FScopedUsdAllocs UsdAllocs;

			pxr::UsdPrim PxrUsdPrim{UsdPrim};

			for (const pxr::UsdAttribute& PrimAttribute : PxrUsdPrim.GetAttributes())
			{
				FString AttributeName = UsdToUnreal::ConvertString(PrimAttribute.GetName().GetString());

				pxr::VtValue VtValue;
				PrimAttribute.Get(&VtValue, TimeCode);

				// Just show arrays as readonly strings for now
				if (VtValue.IsArrayValued())
				{
					FString Stringified = FString::Printf(TEXT("%zu elements: "), VtValue.GetArraySize());

					// This array it's too large to even stringify fast enough, so for now just show the element count
					if (VtValue.GetArraySize() > 5000)
					{
						Stringified += TEXT("[too many entries to expand]");
					}
					else
					{
						Stringified += UsdToUnreal::ConvertString(pxr::TfStringify(VtValue));

						// STextBlock can get very slow calculating its desired size for very long string so chop it if needed
						const int32 MaxValueLength = 300;
						if (Stringified.Len() > MaxValueLength)
						{
							Stringified.LeftInline(MaxValueLength);
							Stringified.Append(TEXT("...]"));
						}
					}

					const bool bAttrReadOnly = true;
					CreateField(
						EObjectFieldType::Attribute,
						AttributeName,
						Stringified,
						UsdUtils::EUsdBasicDataTypes::String,
						TEXT(""),
						bAttrReadOnly
					);
				}
				else
				{
					UsdUtils::FConvertedVtValue ConvertedValue;
					if (UsdToUnreal::ConvertValue(UE::FVtValue{VtValue}, ConvertedValue))
					{
						const bool bAttrReadOnly = false;
						CreateField(EObjectFieldType::Attribute, AttributeName, ConvertedValue, bAttrReadOnly);
					}

					if (PrimAttribute.HasAuthoredConnections())
					{
						const FString ConnectionAttributeName = AttributeName + TEXT(":connect");

						pxr::SdfPathVector ConnectedSources;
						PrimAttribute.GetConnections(&ConnectedSources);

						for (pxr::SdfPath& ConnectedSource : ConnectedSources)
						{
							UsdUtils::FConvertedVtValueEntry Entry;
							Entry.Emplace(TInPlaceType<FString>(), UsdToUnreal::ConvertPath(ConnectedSource));

							UsdUtils::FConvertedVtValue ConnectionPropertyValue;
							ConnectionPropertyValue.SourceType = UsdUtils::EUsdBasicDataTypes::String;
							ConnectionPropertyValue.Entries = {Entry};

							const bool bConnectionValueReadOnly = true;
							CreateField(EObjectFieldType::Attribute, ConnectionAttributeName, ConnectionPropertyValue, bConnectionValueReadOnly);
						}
					}
				}
			}

			for (const pxr::UsdRelationship& Relationship : PxrUsdPrim.GetRelationships())
			{
				FString RelationshipName = UsdToUnreal::ConvertString(Relationship.GetName().GetString());

				std::vector<pxr::SdfPath> Targets;
				if (Relationship.GetTargets(&Targets))
				{
					if (Targets.size() == 1)
					{
						FString UETarget = UsdToUnreal::ConvertPath(Targets[0]);
						const bool bReadOnly = false;
						CreateField(
							EObjectFieldType::Relationship,
							RelationshipName,
							UETarget,
							UsdUtils::EUsdBasicDataTypes::String,
							TEXT(""),
							bReadOnly
						);
					}
					else if (Targets.size() > 1)
					{
						FString CombinedTargets = FString::Printf(TEXT("%d elements: ["), static_cast<int32>(Targets.size()));
						for (const pxr::SdfPath& Target : Targets)
						{
							CombinedTargets += UsdToUnreal::ConvertPath(Target) + TEXT(", ");
						}
						CombinedTargets.RemoveFromEnd(TEXT(", "));

						// STextBlock can get very slow calculating its desired size for very long string so chop it if needed
						const int32 MaxValueLength = 300;
						if (CombinedTargets.Len() > MaxValueLength)
						{
							CombinedTargets.LeftInline(MaxValueLength);
							CombinedTargets.Append(TEXT("..."));
						}

						CombinedTargets += TEXT("]");

						const bool bReadOnly = true;
						CreateField(
							EObjectFieldType::Relationship,
							RelationshipName,
							CombinedTargets,
							UsdUtils::EUsdBasicDataTypes::String,
							TEXT(""),
							bReadOnly
						);
					}
				}
			}

			AddMetadataFields(PxrUsdPrim);
		}
		// It's a property path?
		else if (!ObjectPath.IsEmpty())
		{
			FScopedUsdAllocs Allocs;

			pxr::SdfPath UsdPath = UnrealToUsd::ConvertPath(*ObjectPath).Get();
			if (UsdPath.IsPropertyPath())
			{
				pxr::SdfPath PxrPrimPath = UsdPath.GetPrimPath();
				pxr::TfToken PropertyName = UsdPath.GetNameToken();

				pxr::UsdStageRefPtr PxrUsdStage{UsdStage};
				if (pxr::UsdPrim Prim = PxrUsdStage->GetPrimAtPath(PxrPrimPath))
				{
					if (pxr::UsdProperty Property = Prim.GetProperty(PropertyName))
					{
						AddMetadataFields(Property);
					}
				}
			}
		}
	}

	Sort();
#endif	  // #if USE_USD_SDK
}

void FUsdObjectFieldsViewModel::Sort()
{
	// When sorting by type, we'll sort according to the types first, but still sort alphabetically within each type
	if (CurrentSortColumn == ObjectFieldColumnIds::TypeColumn)
	{
		if (CurrentSortMode == EColumnSortMode::Ascending)
		{
			Fields.Sort(
				[](const TSharedPtr<FUsdObjectFieldViewModel>& A, const TSharedPtr<FUsdObjectFieldViewModel>& B)
				{
					if (A->Type == B->Type)
					{
						return A->Label < B->Label;
					}
					else
					{
						return A->Type < B->Type;
					}
				}
			);
		}
		else
		{
			Fields.Sort(
				[](const TSharedPtr<FUsdObjectFieldViewModel>& A, const TSharedPtr<FUsdObjectFieldViewModel>& B)
				{
					if (A->Type == B->Type)
					{
						return A->Label < B->Label;
					}
					else
					{
						return A->Type > B->Type;
					}
				}
			);
		}
	}
	// When sorting by name we'll sort alphabetically regardless of type
	else if (CurrentSortColumn == ObjectFieldColumnIds::NameColumn)
	{
		if (CurrentSortMode == EColumnSortMode::Ascending)
		{
			Fields.Sort(
				[](const TSharedPtr<FUsdObjectFieldViewModel>& A, const TSharedPtr<FUsdObjectFieldViewModel>& B)
				{
					return A->Label < B->Label;
				}
			);
		}
		else
		{
			Fields.Sort(
				[](const TSharedPtr<FUsdObjectFieldViewModel>& A, const TSharedPtr<FUsdObjectFieldViewModel>& B)
				{
					return A->Label > B->Label;
				}
			);
		}
	}
}

UE::FUsdStageWeak FUsdObjectFieldsViewModel::GetUsdStage() const
{
	return UsdStage;
}

FString FUsdObjectFieldsViewModel::GetObjectPath() const
{
	return ObjectPath;
}

#undef LOCTEXT_NAMESPACE
