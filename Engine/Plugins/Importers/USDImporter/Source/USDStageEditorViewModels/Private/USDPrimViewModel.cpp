// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDPrimViewModel.h"

#include "UnrealUSDWrapper.h"
#include "USDConversionUtils.h"
#include "USDIntegrationUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/sdf/path.h"
#include "pxr/usd/usd/payloads.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usdGeom/xform.h"
#include "pxr/usd/usdSkel/root.h"
#include "pxr/usd/usdSkel/skeleton.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#define LOCTEXT_NAMESPACE "USDPrimViewModel"

FUsdPrimViewModel::FUsdPrimViewModel(FUsdPrimViewModel* InParentItem, const UE::FUsdStageWeak& InUsdStage, const UE::FUsdPrim& InPrim)
	: UsdStage(InUsdStage)
	, UsdPrim(InPrim)
	, ParentItem(InParentItem)
	, RowData(MakeShared<FUsdPrimModel>())
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdPrimViewModel::FUsdPrimViewModel);
	RefreshData(false);

	if (ShouldGenerateChildren())
	{
		FillChildren();
	}
}

TArray<FUsdPrimViewModelRef>& FUsdPrimViewModel::UpdateChildren()
{
	if (!UsdPrim)
	{
		return Children;
	}

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	bool bNeedsRefresh = false;

	pxr::UsdPrimSiblingRange PrimChildren = pxr::UsdPrim(UsdPrim).GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate));

	const int32 NumUsdChildren = (TArray<FUsdPrimViewModelRef>::SizeType)std::distance(PrimChildren.begin(), PrimChildren.end());
	const int32 NumUnrealChildren = [&]()
	{
		int32 ValidPrims = 0;
		for (const FUsdPrimViewModelRef& Child : Children)
		{
			if (!Child->RowData->Name.IsEmpty())
			{
				++ValidPrims;
			}
		}

		return ValidPrims;
	}();

	if (NumUsdChildren != NumUnrealChildren)
	{
		FScopedUnrealAllocs UnrealAllocs;

		Children.Reset();
		bNeedsRefresh = true;
	}
	else
	{
		int32 ChildIndex = 0;

		for (const pxr::UsdPrim& Child : PrimChildren)
		{
			if (!Children.IsValidIndex(ChildIndex)
				|| Children[ChildIndex]->UsdPrim.GetPrimPath().GetString() != UsdToUnreal::ConvertPath(Child.GetPrimPath()))
			{
				FScopedUnrealAllocs UnrealAllocs;

				Children.Reset();
				bNeedsRefresh = true;
				break;
			}

			++ChildIndex;
		}
	}

	if (bNeedsRefresh && ShouldGenerateChildren())
	{
		FillChildren();
	}
#endif	  // #if USE_USD_SDK

	return Children;
}

void FUsdPrimViewModel::SetIsExpanded(bool bNewIsExpanded)
{
	if (bNewIsExpanded == bIsExpanded)
	{
		return;
	}
	bIsExpanded = bNewIsExpanded;

	// We should always have our own immediate children up-to-date, as that is needed to get an expander arrow.
	// If we're collapsed though, we don't have to have grandchildren
	if (bIsExpanded)
	{
		for (FUsdPrimViewModelRef Child : Children)
		{
			Child->FillChildren();
		}
	}
	else
	{
		for (FUsdPrimViewModelRef Child : Children)
		{
			Child->Children.Reset();
		}
	}
}

bool FUsdPrimViewModel::ShouldGenerateChildren() const
{
	// We need to generate children if our parent is expanded, because having child nodes is what makes
	// the treeview give us an expander arrow in the first place
	return (!ParentItem || ParentItem->bIsExpanded);
}

void FUsdPrimViewModel::FillChildren()
{
#if USE_USD_SDK
	if (!UsdPrim)
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;
	pxr::UsdPrimSiblingRange PrimChildren = pxr::UsdPrim(UsdPrim).GetFilteredChildren(pxr::UsdTraverseInstanceProxies(pxr::UsdPrimAllPrimsPredicate));

	FScopedUnrealAllocs UnrealAllocs;
	for (pxr::UsdPrim Child : PrimChildren)
	{
		Children.Add(MakeShared<FUsdPrimViewModel>(this, UsdStage, UE::FUsdPrim(Child)));
	}
#endif	  // #if USE_USD_SDK
}

void FUsdPrimViewModel::RefreshData(bool bRefreshChildren)
{
#if USE_USD_SDK
	// Before we fully abort due to an invalid prim, first check the case that we just need to get a "refreshed prim"
	// for the same path. This is important for example when setting/clearing the instanceable metadata: If this prim used
	// to be an instance proxy and now our parent is not an instance anymore, the prim will become "invalid", but that's
	// just because the instance doesn't exist anymore: The analogous non-instance-proxy prim still may exist on the stage
	if (!UsdPrim && UsdStage)
	{
		UsdPrim = UsdStage.GetPrimAtPath(UsdPrim.GetPrimPath());
	}

	if (!UsdPrim)
	{
		return;
	}

	const bool bIsPseudoRoot = UsdPrim.GetStage().GetPseudoRoot() == UsdPrim;

	RowData->Name = FText::FromName(bIsPseudoRoot ? TEXT("Stage") : UsdPrim.GetName());
	RowData->bHasCompositionArcs = UsdUtils::HasCompositionArcs(UsdPrim);

	RowData->Type = bIsPseudoRoot ? FText::GetEmpty() : FText::FromName(UsdPrim.GetTypeName());
	RowData->bHasPayload = UsdPrim.HasAuthoredPayloads();
	RowData->bIsLoaded = UsdPrim.IsLoaded();

	bool bOldVisibility = RowData->bIsVisible;
	if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(UsdPrim))
	{
		RowData->bIsVisible = (UsdGeomImageable.ComputeVisibility() != pxr::UsdGeomTokens->invisible);
	}

	// If our visibility was enabled, it may be that the visibilities of all of our parents were enabled to accomplish
	// the target change, so we need to refresh them too. This happens when we manually change visibility on
	// a USceneComponent and write that to the USD Stage, for example
	if (bOldVisibility == false && RowData->bIsVisible)
	{
		FUsdPrimViewModel* Item = ParentItem;
		while (Item)
		{
			Item->RefreshData(false);
			Item = Item->ParentItem;
		}
	}

	if (bRefreshChildren)
	{
		for (FUsdPrimViewModelRef& Child : UpdateChildren())
		{
			Child->RefreshData(bRefreshChildren);
		}
	}
#endif	  // #if USE_USD_SDK
}

bool FUsdPrimViewModel::HasVisibilityAttribute() const
{
#if USE_USD_SDK
	if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(UsdPrim))
	{
		return true;
	}
#endif	  // #if USE_USD_SDK
	return false;
}

void FUsdPrimViewModel::ToggleVisibility()
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	if (pxr::UsdGeomImageable UsdGeomImageable = pxr::UsdGeomImageable(UsdPrim))
	{
		// MakeInvisible/MakeVisible internally seem to trigger multiple notices, so group them up to prevent some unnecessary updates
		pxr::SdfChangeBlock SdfChangeBlock;

		if (RowData->IsVisible())
		{
			UsdGeomImageable.MakeInvisible();
		}
		else
		{
			UsdGeomImageable.MakeVisible();
		}

		RefreshData(false);
	}
#endif	  // #if USE_USD_SDK
}

void FUsdPrimViewModel::TogglePayload()
{
	if (UsdPrim && UsdPrim.HasAuthoredPayloads())
	{
		if (UsdPrim.IsLoaded())
		{
			UsdPrim.Unload();
		}
		else
		{
			UsdPrim.Load();
		}

		RefreshData(false);
	}
}

void FUsdPrimViewModel::ApplySchema(FName SchemaName)
{
#if USE_USD_SDK
	UsdUtils::ApplySchema(UsdPrim, UnrealToUsd::ConvertToken(*SchemaName.ToString()).Get());
#endif	  // #if USE_USD_SDK
}

bool FUsdPrimViewModel::CanApplySchema(FName SchemaName) const
{
#if USE_USD_SDK
	if (!UsdPrim || UsdPrim.IsPseudoRoot())
	{
		return false;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim PxrUsdPrim{UsdPrim};
	pxr::TfToken SchemaToken = UnrealToUsd::ConvertToken(*SchemaName.ToString()).Get();

	if (SchemaToken == UnrealIdentifiers::ControlRigAPI && !(PxrUsdPrim.IsA<pxr::UsdSkelRoot>() || PxrUsdPrim.IsA<pxr::UsdSkelSkeleton>()))
	{
		return false;
	}

	return UsdUtils::CanApplySchema(UsdPrim, SchemaToken);
#else
	return false;
#endif	  // #if USE_USD_SDK
}

void FUsdPrimViewModel::RemoveSchema(FName SchemaName)
{
#if USE_USD_SDK
	UsdUtils::RemoveSchema(UsdPrim, UnrealToUsd::ConvertToken(*SchemaName.ToString()).Get());
#endif	  // #if USE_USD_SDK
}

bool FUsdPrimViewModel::CanRemoveSchema(FName SchemaName) const
{
#if USE_USD_SDK
	return UsdUtils::CanRemoveSchema(UsdPrim, UnrealToUsd::ConvertToken(*SchemaName.ToString()).Get());
#else
	return false;
#endif	  // #if USE_USD_SDK
}

bool FUsdPrimViewModel::HasSpecsOnLocalLayer() const
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim PxrUsdPrim{UsdPrim};
	if (PxrUsdPrim)
	{
		if (pxr::UsdStageRefPtr PrimUsdStage = PxrUsdPrim.GetStage())
		{
			for (const pxr::SdfPrimSpecHandle& Spec : PxrUsdPrim.GetPrimStack())
			{
				if (Spec && PrimUsdStage->HasLocalLayer(Spec->GetLayer()))
				{
					return true;
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK

	return false;
}

bool FUsdPrimViewModel::HasReferencesOnLocalLayer() const
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim PxrUsdPrim{UsdPrim};
	if (PxrUsdPrim)
	{
		if (pxr::UsdStageRefPtr PrimUsdStage = PxrUsdPrim.GetStage())
		{
			for (const pxr::SdfPrimSpecHandle& Spec : PxrUsdPrim.GetPrimStack())
			{
				if (Spec && Spec->HasReferences())
				{
					pxr::SdfLayerHandle SpecLayer = Spec->GetLayer();
					if (PrimUsdStage->GetEditTarget().GetLayer() == SpecLayer && PrimUsdStage->HasLocalLayer(SpecLayer))
					{
						// At this point we know this prim has a spec with references on this edit target layer
						//
						// For us to be able to interact with it however, it must be a non-ancestral reference, so we need to check if we
						// have any on this layer and prim path.
						//
						// Ancestral references are reference composition arcs introduced by an ancestor prim (so e.g. a parent prim has a 
						// reference, that contributes reference specs to this prim). We can't interact with those, as they're "encapsulated" 
						// within the reference (see https://openusd.org/release/usdfaq.html#list-edited-composition-arcs)
						pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery::GetDirectReferences(PxrUsdPrim);
						for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
						{
							if (CompositionArc.HasSpecs() && 
								CompositionArc.GetIntroducingLayer() == SpecLayer && 
								CompositionArc.GetIntroducingPrimPath() == PxrUsdPrim.GetPrimPath())
							{
								return true;
							}
						}
					}
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK

	return false;
}
bool FUsdPrimViewModel::HasPayloadsOnLocalLayer() const
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim PxrUsdPrim{UsdPrim};
	if (PxrUsdPrim)
	{
		if (pxr::UsdStageRefPtr PrimUsdStage = PxrUsdPrim.GetStage())
		{
			for (const pxr::SdfPrimSpecHandle& Spec : PxrUsdPrim.GetPrimStack())
			{
				if (Spec && Spec->HasPayloads())
				{
					pxr::SdfLayerHandle SpecLayer = Spec->GetLayer();
					if (PrimUsdStage->GetEditTarget().GetLayer() == SpecLayer && PrimUsdStage->HasLocalLayer(SpecLayer))
					{
						// See analogous comment on HasReferencesOnLocalLayer

						// There's no pxr::UsdPrimCompositionQuery::GetDirectPayloads()...
						pxr::UsdPrimCompositionQuery::Filter Filter;
						Filter.dependencyTypeFilter = pxr::UsdPrimCompositionQuery::DependencyTypeFilter::Direct;
						Filter.arcTypeFilter = pxr::UsdPrimCompositionQuery::ArcTypeFilter::Payload;
						pxr::UsdPrimCompositionQuery PrimCompositionQuery{PxrUsdPrim, Filter};

						for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
						{
							if (CompositionArc.HasSpecs() && 
								CompositionArc.GetIntroducingLayer() == SpecLayer && 
								CompositionArc.GetIntroducingPrimPath() == PxrUsdPrim.GetPrimPath())
							{
								return true;
							}
						}
					}
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK

	return false;
}

void FUsdPrimViewModel::DefinePrim(const TCHAR* PrimName)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	UE::FSdfPath ParentPrimPath;

	if (ParentItem)
	{
		ParentPrimPath = ParentItem->UsdPrim.GetPrimPath();
	}
	else
	{
		ParentPrimPath = UE::FSdfPath::AbsoluteRootPath();
	}

	UE::FSdfPath NewPrimPath = ParentPrimPath.AppendChild(PrimName);

	UsdPrim = pxr::UsdGeomXform::Define(UsdStage, NewPrimPath).GetPrim();
#endif	  // #if USE_USD_SDK
}

void FUsdPrimViewModel::ClearReferences()
{
#if USE_USD_SDK
	if (!UsdPrim)
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim Prim{UsdPrim};

	pxr::UsdReferences References = Prim.GetReferences();
	References.ClearReferences();

	// Set it back to Xform instead of fully typeless so that we at least get an actor / component in the USD Stage Editor
	if (!Prim.HasAuthoredTypeName())
	{
		Prim.SetTypeName(UnrealIdentifiers::Xform);
	}
#endif	  // #if USE_USD_SDK
}

void FUsdPrimViewModel::ClearPayloads()
{
#if USE_USD_SDK
	if (!UsdPrim)
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdPrim Prim{UsdPrim};

	pxr::UsdPayloads Payloads = Prim.GetPayloads();
	Payloads.ClearPayloads();

	// Set it back to Xform instead of fully typeless so that we at least get an actor / component in the USD Stage Editor
	if (!Prim.HasAuthoredTypeName())
	{
		Prim.SetTypeName(UnrealIdentifiers::Xform);
	}
#endif	  // #if USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
