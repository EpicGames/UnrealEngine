// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDReferencesViewModel.h"

#include "USDErrorUtils.h"
#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/pcp/layerStack.h"
#include "pxr/usd/sdf/payload.h"
#include "pxr/usd/sdf/reference.h"
#include "pxr/usd/usd/payloads.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/primCompositionQuery.h"
#include "pxr/usd/usd/references.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

void FUsdReferencesViewModel::UpdateReferences(const UE::FUsdStageWeak& InUsdStage, const TCHAR* InPrimPath)
{
	// We're provided with an empty prim path when we're meant to clear our references and go invisible,
	// so always do that
	References.Reset();

	if (!InUsdStage || !InPrimPath || FString{InPrimPath}.IsEmpty() || UE::FSdfPath{InPrimPath}.IsAbsoluteRootPath())
	{
		return;
	}

	UsdStage = InUsdStage;
	PrimPath = UE::FSdfPath{InPrimPath};

#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	if (pxr::UsdPrim Prim{UsdStage.GetPrimAtPath(PrimPath)})
	{
		pxr::UsdPrimCompositionQuery::Filter Filter;
		Filter.dependencyTypeFilter = pxr::UsdPrimCompositionQuery::DependencyTypeFilter::Direct;

		pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery(Prim, Filter);

		pxr::PcpLayerStackRefPtr RootLayerStack;

		for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
		{
			pxr::PcpNodeRef IntroducingNode = CompositionArc.GetIntroducingNode();

			if (CompositionArc.GetArcType() == pxr::PcpArcTypeRoot)
			{
				RootLayerStack = IntroducingNode.GetLayerStack();
			}
			else if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
			{
				pxr::SdfReferenceEditorProxy ReferenceEditor;
				pxr::SdfReference UsdReference;

				if (CompositionArc.GetIntroducingListEditor(&ReferenceEditor, &UsdReference))
				{
					FUsdReference Reference;
					Reference.AssetPath = UsdToUnreal::ConvertString(UsdReference.GetAssetPath());
					Reference.PrimPath = UsdToUnreal::ConvertPath(UsdReference.GetPrimPath());
					Reference.bIntroducedInLocalLayerStack = IntroducingNode.GetLayerStack() == RootLayerStack;
					Reference.bIsPayload = false;

					References.Add(MakeSharedUnreal<FUsdReference>(MoveTemp(Reference)));
				}
			}
			else if (CompositionArc.GetArcType() == pxr::PcpArcTypePayload)
			{
				pxr::SdfPayloadEditorProxy Editor;
				pxr::SdfPayload UsdPayload;

				if (CompositionArc.GetIntroducingListEditor(&Editor, &UsdPayload))
				{
					FUsdReference Reference;
					Reference.AssetPath = UsdToUnreal::ConvertString(UsdPayload.GetAssetPath());
					Reference.PrimPath = UsdToUnreal::ConvertPath(UsdPayload.GetPrimPath());
					Reference.bIntroducedInLocalLayerStack = IntroducingNode.GetLayerStack() == RootLayerStack;
					Reference.bIsPayload = true;

					References.Add(MakeSharedUnreal<FUsdReference>(MoveTemp(Reference)));
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK
}

void FUsdReferencesViewModel::RemoveReference(const TSharedPtr<FUsdReference>& Reference)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	if (pxr::UsdPrim Prim{UsdStage.GetPrimAtPath(UE::FSdfPath(PrimPath))})
	{
		pxr::UsdPrimCompositionQuery::Filter Filter;
		Filter.dependencyTypeFilter = pxr::UsdPrimCompositionQuery::DependencyTypeFilter::Direct;

		pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery(Prim, Filter);

		// Annoyingly there is no pxr::UsdReferences::GetReferences() (even though there are add/clear/set?).
		// We must iterate through all composition arcs again to find the exact reference.
		//
		// That is a good idea anyway though because since we first constructed our FUsdReferences in this way,
		// we know that the path strings will match exactly, whether they're relative/absolute paths,
		// upper/lower case drive letters, etc.
		//
		// Also, using the EditorProxy objects makes it easy to remove them, because otherwise we'd have
		// to author opinions using pxr::UsdReferences/pxr::UsdPayloads, and have to worry about editing inside
		// of variants and so on
		for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
		{
			if (Reference->bIsPayload && CompositionArc.GetArcType() == pxr::PcpArcTypePayload)
			{
				pxr::SdfPayloadEditorProxy Editor;
				pxr::SdfPayload Payload;

				if (CompositionArc.GetIntroducingListEditor(&Editor, &Payload))
				{
					FString AssetPath = UsdToUnreal::ConvertString(Payload.GetAssetPath());
					if (AssetPath != Reference->AssetPath)
					{
						continue;
					}

					FString TargetPrimPath = UsdToUnreal::ConvertPath(Payload.GetPrimPath());
					if (TargetPrimPath != Reference->PrimPath)
					{
						continue;
					}

					Editor.Remove(Payload);
				}
			}
			else if (!Reference->bIsPayload && CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
			{
				pxr::SdfReferenceEditorProxy Editor;
				pxr::SdfReference SdfReference;

				if (CompositionArc.GetIntroducingListEditor(&Editor, &SdfReference))
				{
					FString AssetPath = UsdToUnreal::ConvertString(SdfReference.GetAssetPath());
					if (AssetPath != Reference->AssetPath)
					{
						continue;
					}

					FString TargetPrimPath = UsdToUnreal::ConvertPath(SdfReference.GetPrimPath());
					if (TargetPrimPath != Reference->PrimPath)
					{
						continue;
					}

					Editor.Remove(SdfReference);
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK
}

void FUsdReferencesViewModel::ReloadReference(const TSharedPtr<FUsdReference>& Reference)
{
#if USE_USD_SDK
	FScopedUsdAllocs UsdAllocs;

	pxr::SdfLayerRefPtr ReloadedLayer;
	pxr::SdfPath SdfReferencedPrimPath;

	if (pxr::UsdPrim Prim{UsdStage.GetPrimAtPath(UE::FSdfPath(PrimPath))})
	{
		pxr::UsdPrimCompositionQuery::Filter Filter;
		Filter.dependencyTypeFilter = pxr::UsdPrimCompositionQuery::DependencyTypeFilter::Direct;

		pxr::UsdPrimCompositionQuery PrimCompositionQuery = pxr::UsdPrimCompositionQuery(Prim, Filter);

		for (const pxr::UsdPrimCompositionQueryArc& CompositionArc : PrimCompositionQuery.GetCompositionArcs())
		{
			if (CompositionArc.GetArcType() == pxr::PcpArcTypeReference)
			{
				pxr::SdfReferenceEditorProxy ReferenceEditor;
				pxr::SdfReference UsdReference;

				if (CompositionArc.GetIntroducingListEditor(&ReferenceEditor, &UsdReference))
				{
					FString AssetPath = UsdToUnreal::ConvertString(UsdReference.GetAssetPath());
					if (AssetPath != Reference->AssetPath)
					{
						continue;
					}

					FString ReferencedPrimPath = UsdToUnreal::ConvertPath(UsdReference.GetPrimPath());
					if (ReferencedPrimPath != Reference->PrimPath)
					{
						continue;
					}

					pxr::PcpNodeRef TargetNode = CompositionArc.GetTargetNode();
					pxr::SdfLayerHandle Layer = CompositionArc.GetTargetLayer();

					SdfReferencedPrimPath = TargetNode.GetPath();
					ReloadedLayer = pxr::SdfLayerRefPtr{Layer};
				}
			}
			else if (CompositionArc.GetArcType() == pxr::PcpArcTypePayload)
			{
				pxr::SdfPayloadEditorProxy Editor;
				pxr::SdfPayload Payload;

				if (CompositionArc.GetIntroducingListEditor(&Editor, &Payload))
				{
					FString AssetPath = UsdToUnreal::ConvertString(Payload.GetAssetPath());
					if (AssetPath != Reference->AssetPath)
					{
						continue;
					}

					FString ReferencedPrimPath = UsdToUnreal::ConvertPath(Payload.GetPrimPath());
					if (ReferencedPrimPath != Reference->PrimPath)
					{
						continue;
					}

					pxr::PcpNodeRef TargetNode = CompositionArc.GetTargetNode();
					pxr::SdfLayerHandle Layer = CompositionArc.GetTargetLayer();

					SdfReferencedPrimPath = TargetNode.GetPath();
					ReloadedLayer = pxr::SdfLayerRefPtr{Layer};
				}
			}
		}
	}

	if (!ReloadedLayer || SdfReferencedPrimPath.IsEmpty())
	{
		return;
	}

	// Retrieving the layer stack from the composition query arc above is not perfect: It will give us
	// all the sublayers of the layer we want to reload, but not any layers referenced by prims in the
	// subtree of our target prim. Instead, here we "just let USD do it" by instead reopening the referenced
	// layer with a population mask for our target prim, an retrieving all UsedLayers from the generated stage.
	// All of those layers are already opened anyway, and with the population mask this should be
	// pretty fast still, and in turn we avoid having to recurse through the composition query arcs ourselves
	// which could have some mistakes and miss complex edge cases
	pxr::UsdStagePopulationMask Mask;
	Mask.Add(SdfReferencedPrimPath);
	const pxr::SdfLayerRefPtr SessionLayer = nullptr;
	const pxr::UsdStage::InitialLoadSet InitialLoadSet = pxr::UsdStage::InitialLoadSet::LoadAll;
	if (pxr::UsdStageRefPtr TempStage = pxr::UsdStage::OpenMasked(ReloadedLayer, SessionLayer, Mask, InitialLoadSet))
	{
		std::vector<pxr::SdfLayerHandle> UsedLayers = TempStage->GetUsedLayers();

		// Reference: SdfLayer::ReloadLayers, which we don't use directly because it takes a std::set<> but just loops over it anyway
		pxr::SdfChangeBlock Block;
		for (const pxr::SdfLayerHandle& Layer : UsedLayers)
		{
			if (Layer)
			{
				USD_LOG_INFO(TEXT("Reloading layer '%s'"), *UsdToUnreal::ConvertString(Layer->GetIdentifier()));
				const bool bForce = true;
				Layer->Reload(bForce);
			}
		}
	}
#endif	  // #if USE_USD_SDK
}
