// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageViewModel.h"

#include "UnrealUSDWrapper.h"
#include "USDAssetCache3.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDLayerUtils.h"
#include "USDMemory.h"
#include "USDStageActor.h"
#include "USDStageImportContext.h"
#include "USDStageImporter.h"
#include "USDStageImporterModule.h"
#include "USDStageImportOptions.h"
#include "USDStageModule.h"
#include "USDTypesConversion.h"
#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdStage.h"

#include "Engine/World.h"
#include "Misc/Paths.h"
#include "ScopedTransaction.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/common.h"
#include "pxr/usd/usd/stage.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

#define LOCTEXT_NAMESPACE "UsdStageViewModel"

#if USE_USD_SDK
namespace UsdViewModelImpl
{
	/**
	 * Saves the UE-state session layer for the given stage.
	 * We use this instead of pxr::UsdStage::SaveSessionLayers because that function
	 * will emit a warning about the main session layer not being saved every time it is used
	 */
	void SaveUEStateLayer(const UE::FUsdStage& UsdStage)
	{
		const bool bCreateIfNeeded = false;
		if (UE::FSdfLayer UEStateLayer = UsdUtils::GetUEPersistentStateSublayer(UsdStage, bCreateIfNeeded))
		{
			UEStateLayer.Export(*UEStateLayer.GetRealPath());
		}
	}
}
#endif	  // #if USE_USD_SDK

void FUsdStageViewModel::NewStage()
{
	FScopedTransaction Transaction(LOCTEXT("NewStageTransaction", "Created new USD stage"));

	FScopedUsdMessageLog ScopedLog;

	if (!UsdStageActor.IsValid())
	{
		IUsdStageModule& UsdStageModule = FModuleManager::GetModuleChecked<IUsdStageModule>(TEXT("USDStage"));
		UsdStageActor = &UsdStageModule.GetUsdStageActor(IUsdClassesModule::GetCurrentWorld());
	}

	if (AUsdStageActor* StageActor = UsdStageActor.Get())
	{
		StageActor->Modify();
		StageActor->NewStage();
	}
}

void FUsdStageViewModel::OpenStage(const TCHAR* FilePath)
{
	FScopedUsdMessageLog ScopedLog;

	if (!UsdStageActor.IsValid())
	{
		IUsdStageModule& UsdStageModule = FModuleManager::GetModuleChecked<IUsdStageModule>(TEXT("USDStage"));
		UsdStageActor = &UsdStageModule.GetUsdStageActor(IUsdClassesModule::GetCurrentWorld());
	}

	if (AUsdStageActor* StageActor = UsdStageActor.Get())
	{
		StageActor->Modify();
		StageActor->SetRootLayer(FilePath);
	}
	else
	{
		USD_LOG_ERROR(TEXT("Failed to find a AUsdStageActor that could open stage '%s'!"), FilePath);
	}
}

void FUsdStageViewModel::SetLoadAllRule()
{
#if USE_USD_SDK
	if (!UsdStageActor.IsValid())
	{
		return;
	}

	// This will set what happens to payloads on any other future stages this stage actor opens
	UsdStageActor->SetInitialLoadSet(EUsdInitialLoadSet::LoadAll);

	// This will toggle all current payloads, and set what happens to any future payloads that
	// are created on the currently opened stage
	if (UE::FUsdStage Stage = UsdStageActor->GetOrOpenUsdStage())
	{
		FScopedUsdAllocs Allocs;

		pxr::SdfChangeBlock Block;

		pxr::UsdStageRefPtr UsdStage = pxr::UsdStageRefPtr(Stage);
		UsdStage->SetLoadRules(pxr::UsdStageLoadRules::LoadAll());
	}
#endif	  // USE_USD_SDK
}

void FUsdStageViewModel::SetLoadNoneRule()
{
#if USE_USD_SDK
	if (!UsdStageActor.IsValid())
	{
		return;
	}

	UsdStageActor->SetInitialLoadSet(EUsdInitialLoadSet::LoadNone);

	// This will toggle all current payloads, and set what happens to any future payloads that
	// are created on the currently opened stage
	if (UE::FUsdStage Stage = UsdStageActor->GetOrOpenUsdStage())
	{
		FScopedUsdAllocs Allocs;

		pxr::SdfChangeBlock Block;

		pxr::UsdStageRefPtr UsdStage = pxr::UsdStageRefPtr(Stage);
		UsdStage->SetLoadRules(pxr::UsdStageLoadRules::LoadNone());
	}
#endif	  // USE_USD_SDK
}

bool FUsdStageViewModel::HasDefaultLoadRule()
{
#if USE_USD_SDK
	if (!UsdStageActor.IsValid())
	{
		return true;
	}

	// This will toggle all current payloads, and set what happens to any future payloads that
	// are created on the currently opened stage
	if (UE::FUsdStage Stage = UsdStageActor->GetOrOpenUsdStage())
	{
		FScopedUsdAllocs Allocs;

		pxr::UsdStageRefPtr UsdStage = pxr::UsdStageRefPtr(Stage);

		pxr::UsdStageLoadRules Rules = UsdStage->GetLoadRules();
		Rules.Minimize();

		return Rules == pxr::UsdStageLoadRules::LoadNone() || Rules == pxr::UsdStageLoadRules::LoadAll();
	}
#endif	  // USE_USD_SDK

	return true;
}

void FUsdStageViewModel::ReloadStage()
{
	if (!UsdStageActor.IsValid())
	{
		return;
	}

#if USE_USD_SDK
	UE::FUsdStage Stage = UsdStageActor->GetOrOpenUsdStage();
	pxr::UsdStageRefPtr UsdStage = pxr::UsdStageRefPtr(Stage);

	// Can't reload from disk something that doesn't exist on disk yet
	// (actually USD will let us do this but it seems to just clear the anonymous layers instead)
	if (Stage.GetRootLayer().IsAnonymous())
	{
		return;
	}

	if (UsdStage)
	{
		FScopedUsdMessageLog ScopedLog;

		{
			FScopedUsdAllocs Allocs;

			TSet<FString> NativeLayerExtensions{UnrealUSDWrapper::GetNativeFileFormats()};

			std::set<pxr::SdfLayerHandle> LayersToRefresh;

			const std::vector<pxr::SdfLayerHandle>& HandleVec = UsdStage->GetUsedLayers();

			// We should only reload the native layers, because things like .mtlx files can show up in this list, if
			// USD plugins allow interpreting them as "layers". We won't show these on our UI though, and not
			// necessarily these are equipped to be modified/reloaded from USD (like how the .mtlx ones aren't).
			for (const pxr::SdfLayerHandle& Handle : HandleVec)
			{
				std::string LayerPath;
				std::map<std::string, std::string> Arguments;
				if (pxr::SdfLayer::SplitIdentifier(Handle->GetIdentifier(), &LayerPath, &Arguments))
				{
					FString UELayerPath = UsdToUnreal::ConvertString(LayerPath);
					FString UELayerExtension = FPaths::GetExtension(UELayerPath);
					if (NativeLayerExtensions.Contains(UELayerExtension))
					{
						LayersToRefresh.insert(Handle);
					}
				}
			}

			const bool bForce = true;
			pxr::SdfLayer::ReloadLayers(LayersToRefresh, bForce);

			// When reloading our UEState layer is closed but there is nothing on the root layer
			// that would automatically pull the UEState session layer and cause it to be reloaded, so we need to try
			// to load it back again
			const bool bCreateIfNeeded = false;
			UsdUtils::GetUEPersistentStateSublayer(Stage, bCreateIfNeeded);
		}

		if (FUsdLogManager::HasAccumulatedErrors())
		{
			return;
		}

		// If we were editing an unsaved layer, when we reload the edit target will be cleared.
		// We need to make sure we're always editing something or else UsdEditContext might trigger some errors
		const pxr::UsdEditTarget& EditTarget = UsdStage->GetEditTarget();
		if (!EditTarget.IsValid() || EditTarget.IsNull())
		{
			UsdStage->SetEditTarget(UsdStage->GetEditTargetForLocalLayer(UsdStage->GetRootLayer()));
		}
	}
#endif	  // #if USE_USD_SDK
}

void FUsdStageViewModel::ResetStage()
{
#if USE_USD_SDK
	if (!UsdStageActor.IsValid())
	{
		return;
	}

	UE::FUsdStage Stage = UsdStageActor->GetOrOpenUsdStage();
	pxr::UsdStageRefPtr UsdStage = pxr::UsdStageRefPtr(Stage);

	if (UsdStage)
	{
		FScopedUsdAllocs Allocs;

		UsdStage->GetSessionLayer()->Clear();

		UsdStage->SetEditTarget(UsdStage->GetEditTargetForLocalLayer(UsdStage->GetRootLayer()));

		UsdStage->MuteAndUnmuteLayers({}, UsdStage->GetMutedLayers());
	}
#endif	  // #if USE_USD_SDK
}

void FUsdStageViewModel::CloseStage()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FUsdStageViewModel::CloseStage);
	if (AUsdStageActor* StageActor = UsdStageActor.Get())
	{
		StageActor->Reset();
	}
}

void FUsdStageViewModel::SaveStage()
{
#if USE_USD_SDK
	if (UsdStageActor.IsValid())
	{
		if (UE::FUsdStage UsdStage = UsdStageActor->GetOrOpenUsdStage())
		{
			FScopedUsdAllocs UsdAllocs;

			FScopedUsdMessageLog ScopedLog;

			// Save layers manually instead of calling UsdStage::Save(). This is roughly the same implementation anyway, except
			// that UsdStage::Save() will ignore a layer if it also happens to be added as a sublayer to a session layer, and
			// we want to ensure we always save dirty layers when we hit SaveStage
			for (const pxr::SdfLayerHandle& Handle : pxr::UsdStageRefPtr { UsdStage } -> GetUsedLayers())
			{
				if (!Handle->IsAnonymous() && Handle->IsDirty())
				{
					Handle->Save();
				}
			}

			UsdViewModelImpl::SaveUEStateLayer(UsdStage);
		}
	}
#endif	  // #if USE_USD_SDK
}

void FUsdStageViewModel::SaveStageAs(const TCHAR* FilePath)
{
#if USE_USD_SDK
	FScopedTransaction Transaction(FText::Format(LOCTEXT("SaveAsTransaction", "Saved USD stage as '{0}'"), FText::FromString(FilePath)));

	AUsdStageActor* StageActorPtr = UsdStageActor.Get();
	if (StageActorPtr)
	{
		if (UE::FUsdStage UsdStage = UsdStageActor->GetOrOpenUsdStage())
		{
			FScopedUsdMessageLog ScopedLog;

			if (UE::FSdfLayer RootLayer = UsdStage.GetRootLayer())
			{
				if (pxr::SdfLayerRefPtr(RootLayer)->Export(TCHAR_TO_UTF8(FilePath)))
				{
					FScopedUnrealAllocs UEAllocs;

					// In the process of opening FilePath below we'll close our previous stage, which is an anonymous
					// layer and marked dirty by default. Even though we just saved (exported) it to disk, we'd end
					// up getting the "do you want to save these dirty USD layers?" dialog by default...
					// Here we'll write out a comment on the layer that we can easily check for in
					// USDStageEditorModule::SaveStageActorLayersForWorld to know to skip showing that dialog. Note
					// how this comment iself doesn't actually get saved to disk though.
					//
					// We also block listening here because we don't want to record to the transactor that we wrote
					// this comment, because we can't record restoring it to what it was either, given that we'll stop
					// listening to the previous stage after we open the next one. If we just recorded writing the
					// comment, undoing/redoing through this operation would have left the temp stage with the comment
					// in it permanently.
					FScopedBlockNoticeListening BlockListening(StageActorPtr);
					FString OldComment = RootLayer.GetComment();
					RootLayer.SetComment(UnrealIdentifiers::LayerSavedComment);

					// Open our stage right here so that we can provide it with a session layer. Note that we'll be using
					// the same session layer as the original stage, so that the user doesn't lose their session
					UE::FSdfLayer NewRootLayer = UE::FSdfLayer::FindOrOpen(FilePath);
					UE::FUsdStage NewStage = UnrealUSDWrapper::OpenStage(NewRootLayer, UsdStage.GetSessionLayer(), StageActorPtr->InitialLoadSet);
					StageActorPtr->SetUsdStage(NewStage);

					RootLayer.SetComment(*OldComment);

					// Annoyingly, if we're in a workflow where NewRootLayer == RootLayer (i.e. delete root layer
					// on disk -> press Save -> pick the same filepath as the original root layer), adding/removing that
					// comment above will cause the layer to be marked as dirty... So here we save that layer alone again
					// to remove the dirtiness state
					if (NewRootLayer && RootLayer == NewRootLayer && NewRootLayer.IsDirty())
					{
						NewRootLayer.Save();
					}

					UsdViewModelImpl::SaveUEStateLayer(UsdStage);
				}
			}
		}
	}
#endif	  // #if USE_USD_SDK
}

void FUsdStageViewModel::ImportStage(const TCHAR* TargetContentFolder, UUsdStageImportOptions* Options)
{
#if USE_USD_SDK
	AUsdStageActor* StageActor = UsdStageActor.Get();
	if (!StageActor)
	{
		return;
	}

	const UE::FUsdStage UsdStage = StageActor->GetOrOpenUsdStage();
	if (!UsdStage)
	{
		return;
	}

	// Import directly from stage
	{
		FUsdStageImportContext ImportContext;

		// Preload some settings according to USDStage options. These will overwrite whatever is loaded from config
		ImportContext.ImportOptions->ExistingAssetCache = StageActor->AssetCache;
		ImportContext.ImportOptions->bUseExistingAssetCache = true;
		ImportContext.ImportOptions->PurposesToImport = StageActor->PurposesToLoad;
		ImportContext.ImportOptions->RenderContextToImport = StageActor->RenderContext;
		ImportContext.ImportOptions->MaterialPurpose = StageActor->MaterialPurpose;
		ImportContext.ImportOptions->StageOptions.MetersPerUnit = UsdUtils::GetUsdStageMetersPerUnit(UsdStage);
		ImportContext.ImportOptions->StageOptions.UpAxis = UsdUtils::GetUsdStageUpAxisAsEnum(UsdStage);
		ImportContext.ImportOptions->ImportTimeCode = StageActor->GetTime();
		ImportContext.ImportOptions->NaniteTriangleThreshold = StageActor->NaniteTriangleThreshold;
		ImportContext.ImportOptions->RootMotionHandling = StageActor->RootMotionHandling;
		ImportContext.ImportOptions->FallbackCollisionType = StageActor->FallbackCollisionType;
		ImportContext.ImportOptions->SubdivisionLevel = StageActor->SubdivisionLevel;
		ImportContext.ImportOptions->MetadataOptions = StageActor->MetadataOptions;
		ImportContext.ImportOptions->KindsToCollapse = StageActor->KindsToCollapse;
		ImportContext.ImportOptions->bUsePrimKindsForCollapsing = StageActor->bUsePrimKindsForCollapsing;
		ImportContext.ImportOptions->bMergeIdenticalMaterialSlots = StageActor->bMergeIdenticalMaterialSlots;
		ImportContext.ImportOptions->bShareAssetsForIdenticalPrims = StageActor->bShareAssetsForIdenticalPrims;

		ImportContext.bReadFromStageCache = true;	 // So that we import whatever the user has open right now, even if the file has changes

		const FString RootPath = UsdStage.GetRootLayer().GetRealPath();
		FString StageName = FPaths::GetBaseFilename(RootPath);

		// Provide a StageName when importing transient stages as this is used for the content folder name and actor label
		if (UsdStage.GetRootLayer().IsAnonymous() && RootPath.IsEmpty())
		{
			StageName = TEXT("TransientStage");
		}

		// Pass the stage directly too in case we're importing a transient stage with no filepath
		ImportContext.Stage = UsdStage;

		const bool bIsAutomated = TargetContentFolder && Options;

		if (ImportContext.Init(StageName, RootPath, TEXT("/Game/"), RF_Public | RF_Transactional | RF_Standalone, bIsAutomated))
		{
			FScopedTransaction Transaction(FText::Format(LOCTEXT("ImportTransaction", "Import USD stage '{0}'"), FText::FromString(StageName)));

			if (bIsAutomated)
			{
				// Apply same conversion that FUsdStageImportContext::Init does on our received path
				ImportContext.PackagePath = FString::Printf(TEXT("%s/%s/"), TargetContentFolder, *StageName);
				ImportContext.ImportOptions = Options;
			}

			// Pick the asset cache that the user potentially changed on the import options dialog. The USDStageImporter will sort itself out
			// if that happens to be nullptr/invalid
			ImportContext.UsdAssetCache = ImportContext.ImportOptions->bUseExistingAssetCache
											  ? Cast<UUsdAssetCache3>(ImportContext.ImportOptions->ExistingAssetCache.TryLoad())
											  : nullptr;
			ImportContext.BBoxCache = StageActor->GetBBoxCache();

			ImportContext.TargetSceneActorAttachParent = StageActor->GetRootComponent()->GetAttachParent();
			ImportContext.TargetSceneActorTargetTransform = StageActor->GetActorTransform();

			UUsdStageImporter* USDImporter = IUsdStageImporterModule::Get().GetImporter();
			USDImporter->ImportFromFile(ImportContext);

			// Note that our ImportContext can't keep strong references to the assets in AssetsCache, and when
			// we CloseStage(), the stage actor will stop referencing them. The only thing keeping them alive at this point is
			// the transaction buffer, but it should be enough at least until this import is complete
			CloseStage();
		}
	}

#endif	  // #if USE_USD_SDK
}

#undef LOCTEXT_NAMESPACE
