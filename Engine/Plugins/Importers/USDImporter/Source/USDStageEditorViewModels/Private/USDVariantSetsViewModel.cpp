// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDVariantSetsViewModel.h"

#include "USDMemory.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdPrim.h"

#include "ScopedTransaction.h"

#if USE_USD_SDK
#include "USDIncludesStart.h"
#include "pxr/usd/usd/prim.h"
#include "pxr/usd/usd/variantSets.h"
#include "USDIncludesEnd.h"
#endif	  // #if USE_USD_SDK

FUsdVariantSetViewModel::FUsdVariantSetViewModel(FUsdVariantSetsViewModel* InOwner)
	: Owner(InOwner)
{
}

void FUsdVariantSetViewModel::SetVariantSelection(const TSharedPtr<FString>& InVariantSelection)
{
#if USE_USD_SDK
	if (!Owner || !Owner->UsdStage)
	{
		return;
	}

	FScopedTransaction Transaction(FText::Format(
		NSLOCTEXT("USDVariantSetsList", "SwitchVariantSetTransaction", "Switch USD Variant Set '{0}' to option '{1}'"),
		FText::FromString(SetName),
		FText::FromString(*VariantSelection)
	));

	VariantSelection = InVariantSelection;

	FScopedUsdAllocs UsdAllocs;

	std::string UsdVariantSetName = UnrealToUsd::ConvertString(*SetName).Get();
	std::string UsdVariantSelection = VariantSelection ? UnrealToUsd::ConvertString(*(*VariantSelection)).Get() : std::string{};

	if (const UE::FUsdPrim& Prim = Owner->UsdStage.GetPrimAtPath(UE::FSdfPath{*Owner->PrimPath}))
	{
		pxr::UsdVariantSets UsdVariantSets = pxr::UsdPrim{Prim}.GetVariantSets();
		UsdVariantSets.SetSelection(UsdVariantSetName, UsdVariantSelection);
	}

#endif	  // #if USE_USD_SDK
}

void FUsdVariantSetsViewModel::UpdateVariantSets(const UE::FUsdStageWeak& InUsdStage, const TCHAR* InPrimPath)
{
#if USE_USD_SDK
	VariantSets.Reset();
	UsdStage = InUsdStage;
	PrimPath = InPrimPath;

	if (!UsdStage || PrimPath.IsEmpty() || UE::FSdfPath{InPrimPath}.IsAbsoluteRootPath())
	{
		return;
	}

	UE::FUsdPrim UsdPrim = UsdStage.GetPrimAtPath(UE::FSdfPath(*PrimPath));
	if (!UsdPrim)
	{
		return;
	}

	FScopedUsdAllocs UsdAllocs;

	pxr::UsdVariantSets UsdVariantSets = pxr::UsdPrim(UsdPrim).GetVariantSets();

	std::vector<std::string> UsdVariantSetsNames;
	UsdVariantSets.GetNames(&UsdVariantSetsNames);

	for (const std::string& UsdVariantSetName : UsdVariantSetsNames)
	{
		FUsdVariantSetViewModel VariantSet(this);
		VariantSet.SetName = UsdToUnreal::ConvertString(UsdVariantSetName.c_str());

		pxr::UsdVariantSet UsdVariantSet = pxr::UsdPrim(UsdPrim).GetVariantSet(UsdVariantSetName.c_str());
		VariantSet.VariantSelection = MakeSharedUnreal<FString>(UsdToUnreal::ConvertString(UsdVariantSet.GetVariantSelection().c_str()));

		std::vector<std::string> VariantNames = UsdVariantSet.GetVariantNames();

		for (const std::string& VariantName : VariantNames)
		{
			VariantSet.Variants.Add(MakeSharedUnreal<FString>(UsdToUnreal::ConvertString(VariantName)));
		}

		VariantSets.Add(MakeSharedUnreal<FUsdVariantSetViewModel>(MoveTemp(VariantSet)));
	}
#endif	  // #if USE_USD_SDK
}
