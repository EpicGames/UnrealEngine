// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdStage.h"

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

class FUsdIntegrationsViewModel
{
public:
	UE_API void UpdateAttributes(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath);

public:
	TArray<TSharedPtr<UE::FUsdAttribute>> Attributes;

	FString PrimPath;
	UE::FUsdStageWeak UsdStage;
};

#undef UE_API
