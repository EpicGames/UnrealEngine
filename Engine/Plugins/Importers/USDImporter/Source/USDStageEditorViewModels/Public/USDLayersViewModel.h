// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#include "UsdWrappers/UsdStage.h"
#include "Widgets/IUSDTreeViewItem.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

class FUsdLayerModel : public TSharedFromThis<FUsdLayerModel>
{
public:
	FString DisplayName;
	bool bIsEditTarget = false;
	bool bIsMuted = false;
	bool bIsDirty = false;
	bool bIsInIsolatedStage = true;
};

class FUsdLayerViewModel : public IUsdTreeViewItem
{
public:
	UE_API explicit FUsdLayerViewModel(
		FUsdLayerViewModel* InParentItem,
		const UE::FUsdStageWeak& InUsdStage,
		const UE::FUsdStageWeak& InIsolatedStage,
		const FString& InLayerIdentifier
	);

	UE_API bool IsValid() const;

	UE_API TArray<TSharedRef<FUsdLayerViewModel>> GetChildren();

	UE_API void FillChildren();

	UE_API void RefreshData();

	UE_API UE::FSdfLayer GetLayer() const;
	UE_API FText GetDisplayName() const;

	UE_API bool IsLayerMuted() const;
	UE_API bool CanMuteLayer() const;
	UE_API void ToggleMuteLayer();

	UE_API bool IsEditTarget() const;
	UE_API bool CanEditLayer() const;
	UE_API bool EditLayer();

	UE_API bool IsInIsolatedStage() const;

	UE_API void AddSubLayer(const TCHAR* SubLayerIdentifier);
	UE_API void NewSubLayer(const TCHAR* SubLayerIdentifier);
	UE_API bool RemoveSubLayer(int32 SubLayerIndex);

	UE_API bool IsLayerDirty() const;

	UE_API bool CanReload() const;
	UE_API void Reload();

public:
	TSharedRef<FUsdLayerModel> LayerModel;
	FUsdLayerViewModel* ParentItem;
	TArray<TSharedRef<FUsdLayerViewModel>> Children;

	UE::FUsdStageWeak UsdStage;
	UE::FUsdStageWeak IsolatedStage;
	FString LayerIdentifier;
};

#undef UE_API
