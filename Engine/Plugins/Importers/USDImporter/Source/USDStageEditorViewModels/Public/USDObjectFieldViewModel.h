// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDValueConversion.h"

#include "UsdWrappers/UsdStage.h"

#include "Templates/SharedPointer.h"
#include "Widgets/Views/SHeaderRow.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

class FUsdObjectFieldsViewModel;

enum class EObjectFieldType : int32
{
	Metadata = 0,
	Attribute = 1,
	Relationship = 2,
	MAX = 3
};

namespace ObjectFieldColumnIds
{
	inline const FName TypeColumn = TEXT("FieldType");
	inline const FName NameColumn = TEXT("FieldName");
	inline const FName ValueColumn = TEXT("FieldValue");
};

class FUsdObjectFieldViewModel : public TSharedFromThis<FUsdObjectFieldViewModel>
{
public:
	UE_API explicit FUsdObjectFieldViewModel(FUsdObjectFieldsViewModel* InOwner);

	// This member function is necessary because the no-RTTI slate module can't query USD for the available token options
	UE_API TArray<TSharedPtr<FString>> GetDropdownOptions() const;
	UE_API void SetAttributeValue(const UsdUtils::FConvertedVtValue& InValue);

public:
	EObjectFieldType Type;
	FString Label;
	UsdUtils::FConvertedVtValue Value;
	FString ValueRole;
	bool bReadOnly = false;

private:
	FUsdObjectFieldsViewModel* Owner;
};

class FUsdObjectFieldsViewModel
{
public:
	template<typename T>
	void CreateField(
		EObjectFieldType Type,
		const FString& FieldName,
		const T& Value,
		UsdUtils::EUsdBasicDataTypes UsdType,
		const FString& ValueRole = FString(),
		bool bReadOnly = false
	);
	UE_API void CreateField(EObjectFieldType Type, const FString& FieldName, const UsdUtils::FConvertedVtValue& Value, bool bReadOnly = false);

	UE_API void SetFieldValue(const FString& FieldName, const UsdUtils::FConvertedVtValue& Value);

	UE_API void Refresh(const UE::FUsdStageWeak& UsdStage, const TCHAR* ObjectPath, float TimeCode);
	UE_API void Sort();

	UE_API UE::FUsdStageWeak GetUsdStage() const;
	UE_API FString GetObjectPath() const;

public:
	EColumnSortMode::Type CurrentSortMode = EColumnSortMode::Ascending;
	FName CurrentSortColumn = ObjectFieldColumnIds::TypeColumn;
	TArray<TSharedPtr<FUsdObjectFieldViewModel>> Fields;

private:
	UE::FUsdStageWeak UsdStage;
	FString ObjectPath;
};

#undef UE_API
