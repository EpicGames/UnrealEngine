// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/IUSDTreeViewItem.h"

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

using FUsdPrimViewModelRef = TSharedRef<class FUsdPrimViewModel>;
using FUsdPrimViewModelPtr = TSharedPtr<class FUsdPrimViewModel>;

class FUsdPrimModel : public TSharedFromThis<FUsdPrimModel>
{
public:
	FText GetName() const
	{
		return Name;
	}
	FText GetType() const
	{
		return Type;
	}
	bool HasPayload() const
	{
		return bHasPayload;
	}
	bool IsLoaded() const
	{
		return bIsLoaded;
	}
	bool HasCompositionArcs() const
	{
		return bHasCompositionArcs;
	}
	bool IsVisible() const
	{
		return bIsVisible;
	}

	FText Name;
	FText Type;
	bool bHasPayload = false;
	bool bIsLoaded = false;
	bool bHasCompositionArcs = false;
	bool bIsVisible = true;
};

class FUsdPrimViewModel : public IUsdTreeViewItem
{
public:
	UE_API FUsdPrimViewModel(FUsdPrimViewModel* InParentItem, const UE::FUsdStageWeak& InUsdStage, const UE::FUsdPrim& InPrim = {});

	UE_API TArray<FUsdPrimViewModelRef>& UpdateChildren();

	UE_API void SetIsExpanded(bool bNewIsExpanded);
	UE_API bool ShouldGenerateChildren() const;
	UE_API void FillChildren();

	UE_API void RefreshData(bool bRefreshChildren);

	UE_API bool HasVisibilityAttribute() const;
	UE_API void ToggleVisibility();
	UE_API void TogglePayload();

	UE_API void ApplySchema(FName SchemaName);
	UE_API bool CanApplySchema(FName SchemaName) const;
	UE_API void RemoveSchema(FName SchemaName);
	UE_API bool CanRemoveSchema(FName SchemaName) const;

	// Returns true if this prim has at least one spec on the stage's local layer stack
	UE_API bool HasSpecsOnLocalLayer() const;
	UE_API bool HasReferencesOnLocalLayer() const;
	UE_API bool HasPayloadsOnLocalLayer() const;

	UE_API void DefinePrim(const TCHAR* PrimName);

	/** Delegate for hooking up an inline editable text block to be notified that a rename is requested. */
	DECLARE_DELEGATE(FOnRenameRequest);

	/** Broadcasts whenever a rename is requested */
	FOnRenameRequest RenameRequestEvent;

public:
	UE_API void ClearReferences();
	UE_API void ClearPayloads();

public:
	UE::FUsdStageWeak UsdStage;
	UE::FUsdPrim UsdPrim;

	FUsdPrimViewModel* ParentItem;
	TArray<FUsdPrimViewModelRef> Children;

	TSharedRef<FUsdPrimModel> RowData;	  // Data model

	bool bIsRenamingExistingPrim = false;

private:
	bool bIsExpanded = false;	 // Set by our parent tree, controls whether we generate grandchildren or not
};

#undef UE_API
