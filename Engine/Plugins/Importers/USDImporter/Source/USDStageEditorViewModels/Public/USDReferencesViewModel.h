// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "UsdWrappers/ForwardDeclarations.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdStage.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

class FUsdReference : public TSharedFromThis<FUsdReference>
{
public:
	FString AssetPath;
	FString PrimPath;
	bool bIntroducedInLocalLayerStack = true;
	bool bIsPayload = false;
};

class FUsdReferencesViewModel
{
public:
	UE_API void UpdateReferences(const UE::FUsdStageWeak& UsdStage, const TCHAR* PrimPath);
	UE_API void RemoveReference(const TSharedPtr<FUsdReference>& Reference);
	UE_API void ReloadReference(const TSharedPtr<FUsdReference>& Reference);

public:
	UE::FUsdStageWeak UsdStage;
	UE::FSdfPath PrimPath;
	TArray<TSharedPtr<FUsdReference>> References;
};

#undef UE_API
