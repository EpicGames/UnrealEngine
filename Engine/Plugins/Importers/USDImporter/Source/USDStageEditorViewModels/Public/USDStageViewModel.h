// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

class AUsdStageActor;
class UPackage;
class UUsdStageImportOptions;

class FUsdStageViewModel
{
public:
	UE_API void NewStage();
	UE_API void OpenStage(const TCHAR* FilePath);
	UE_API void SetLoadAllRule();
	UE_API void SetLoadNoneRule();
	// Returns false if we have a payload in the current stage that has been manually loaded or unloaded. Returns true otherwise.
	// The intent is to get the LoadAll/LoadNone buttons to *both* show as unchecked in case the user manually toggled any,
	// to indicate that picking either LoadAll or LoadNone would trigger some change.
	UE_API bool HasDefaultLoadRule();
	UE_API void ReloadStage();
	UE_API void ResetStage();
	UE_API void CloseStage();
	UE_API void SaveStage();
	/** Temporary until SaveAs feature is properly implemented, may be removed in a future release */
	UE_API void SaveStageAs(const TCHAR* FilePath);
	UE_API void ImportStage(const TCHAR* TargetContentFolder = nullptr, UUsdStageImportOptions* Options = nullptr);

public:
	TWeakObjectPtr<AUsdStageActor> UsdStageActor;
};

#undef UE_API
