// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "UsdWrappers/UsdStage.h"

#define UE_API USDSTAGEEDITORVIEWMODELS_API

class FUsdVariantSetsViewModel;

class FUsdVariantSetViewModel : public TSharedFromThis<FUsdVariantSetViewModel>
{
public:
	UE_API explicit FUsdVariantSetViewModel(FUsdVariantSetsViewModel* InOwner);

	UE_API void SetVariantSelection(const TSharedPtr<FString>& InVariantSelection);

public:
	FString SetName;
	TSharedPtr<FString> VariantSelection;
	TArray<TSharedPtr<FString>> Variants;

private:
	FUsdVariantSetsViewModel* Owner;
};

class FUsdVariantSetsViewModel
{
public:
	UE_API void UpdateVariantSets(const UE::FUsdStageWeak& InUsdStage, const TCHAR* PrimPath);

public:
	UE::FUsdStageWeak UsdStage;
	FString PrimPath;

	TArray<TSharedPtr<FUsdVariantSetViewModel>> VariantSets;
};

#undef UE_API
