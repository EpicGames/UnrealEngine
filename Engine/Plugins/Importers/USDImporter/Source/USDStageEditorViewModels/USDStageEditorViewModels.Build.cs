// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDStageEditorViewModels : ModuleRules
	{
		public USDStageEditorViewModels(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DesktopWidgets",
					"EditorFramework",
					"Engine",
					"InputCore",
					"LevelEditor",
					"Slate",
					"SlateCore",
					"UnrealEd",
					"UnrealUSDWrapper",
					"USDClasses",
					"USDStage",
					"USDStageImporter",
					"USDUtilities",
					"WorkspaceMenuStructure",
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
