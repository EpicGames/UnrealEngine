// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDOptionsWindow.h"

#include "USDStageImportOptions.h"
#include "UsdWrappers/UsdStage.h"
#include "Widgets/SUSDStagePreviewTree.h"

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "USDOptionsWindow"

bool SUsdOptionsWindow::ShowOptions(UObject& OptionsObject, const FText& WindowTitle, const FText& AcceptText, const UE::FUsdStage* Stage)
{
	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);

	const static FString ImporterName = TEXT("USDImporter");
	const FString ClassName = OptionsObject.GetClass()->GetName();

	const FString WidthConfigName = ClassName + TEXT("_DialogWidth");
	int32 Width = 100;
	if (GConfig->GetInt(*ImporterName, *WidthConfigName, Width, GEditorPerProjectIni))
	{
		// Just to make sure we don't load something unusable
		Width = FMath::Min(FMath::Max(Width, 100), DisplayMetrics.PrimaryDisplayWidth);
	}
	else
	{
		// Make it wider if we're going to show two panes
		Width = 500 * (Stage == nullptr ? 1 : 2);
	}

	const FString HeightConfigName = ClassName + TEXT("_DialogHeight");
	int32 Height = 100;
	if (GConfig->GetInt(*ImporterName, *HeightConfigName, Height, GEditorPerProjectIni))
	{
		Height = FMath::Min(FMath::Max(Height, 100), DisplayMetrics.PrimaryDisplayHeight);
	}
	else
	{
		Height = 600;
	}

	// clang-format off
	const FVector2D ClientSize{static_cast<float>(Width), static_cast<float>(Height)};
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.SizingRule(ESizingRule::UserSized)
		.AdjustInitialSizeAndPositionForDPIScale(false)
		.ClientSize(ClientSize);

	TSharedPtr<SUsdOptionsWindow> OptionsWindow;
	Window->SetContent
	(
		SAssignNew(OptionsWindow, SUsdOptionsWindow)
		.OptionsObject(&OptionsObject)
		.AcceptText(AcceptText)
		.WidgetWindow(Window)
		.Stage(Stage)
	);
	// clang-format on

	Window->GetOnWindowClosedEvent().AddLambda(
		[WidthConfigName, HeightConfigName, OptionsWindow](const TSharedRef<SWindow>& Window)
		{
			// We use the geometry here because any sort of size we can extract from the window includes some added margins
			// that are not easy to account for, even with GetWindowSizeFromClientSize. The original provided ClientSize
			// seems to always match the content geometry though, so we're saving and loading the same thing
			FGeometry Geometry = Window->GetContent()->GetCachedGeometry();
			FVector2D ContentSize = Geometry.Size * Geometry.Scale;
			const int32 Width = static_cast<int32>(ContentSize.X);
			const int32 Height = static_cast<int32>(ContentSize.Y);

			GConfig->SetInt(*ImporterName, *WidthConfigName, Width, GEditorPerProjectIni);
			GConfig->SetInt(*ImporterName, *HeightConfigName, Height, GEditorPerProjectIni);

			const bool bRemoveFromCache = false;
			GConfig->Flush(bRemoveFromCache, GEditorPerProjectIni);
		}
	);

	// Preemptively make sure we have a progress dialog created before showing our modal. This because the progress
	// dialog itself is also modal. If it doesn't exist yet, and our options dialog causes a progress dialog
	// to be spawned (e.g. when switching the Level to export via the LevelSequenceUSDExporter), the progress dialog
	// will be pushed to the end of FSlateApplication::ActiveModalWindows (SlateApplication.cpp) and cause our options
	// dialog to pop out of its modal loop (FSlateApplication::AddModalWindow), instantly returning false to our caller
	FScopedSlowTask Progress(1, LOCTEXT("ShowingDialog", "Picking options..."));
	Progress.MakeDialogDelayed(0.25f);

	const bool bSlowTaskWindow = false;
	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, bSlowTaskWindow);

	const bool bAccepted = OptionsWindow->UserAccepted();
	if (bAccepted)
	{
		if (UUsdStageImportOptions* StageImportOptions = Cast<UUsdStageImportOptions>(&OptionsObject))
		{
			StageImportOptions->PrimsToImport = OptionsWindow->GetSelectedFullPrimPaths();
		}
	}

	return bAccepted;
}

bool SUsdOptionsWindow::ShowImportOptions(UObject& OptionsObject, const UE::FUsdStage* StageToImport)
{
	return SUsdOptionsWindow::ShowOptions(
		OptionsObject,
		LOCTEXT("USDImportOptionsTitle", "USD Import Options"),
		LOCTEXT("USDOptionWindow_Import", "Import"),
		StageToImport
	);
}

bool SUsdOptionsWindow::ShowExportOptions(UObject& OptionsObject)
{
	return SUsdOptionsWindow::ShowOptions(
		OptionsObject,
		LOCTEXT("USDExportOptionsTitle", "USD Export Options"),
		LOCTEXT("USDOptionWindow_Export", "Export")
	);
}

void SUsdOptionsWindow::Construct(const FArguments& InArgs)
{
	OptionsObject = InArgs._OptionsObject;
	Window = InArgs._WidgetWindow;
	AcceptText = InArgs._AcceptText;
	const UE::FUsdStage* Stage = InArgs._Stage;
	bAccepted = false;

	// Prepare box that shows the OptionsObject's properties
	TSharedPtr<SWidget> DetailsViewBox;
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(OptionsObject);

		DetailsViewBox = DetailsView.ToSharedRef();
	}

	// clang-format off
	TSharedRef<SSplitter> Splitter = SNew(SSplitter).Orientation(Orient_Horizontal);
	{
		// If we have a stage, show the preview tree on the left
		if (Stage)
		{
			SAssignNew(StagePreviewTree, SUsdStagePreviewTree, *Stage);

			Splitter->AddSlot()
			.Value(0.5f)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
						.Text(LOCTEXT("TreeDescriptionText", "Select prims to import:"))
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.MaxWidth(300.0f)
					.Padding(FMargin(5.0f, 2.0f))
					.VAlign(VAlign_Center)
					[
						SNew(SSearchBox)
						.HintText(LOCTEXT("SearchInitialText", "Filter prims..."))
						.OnTextChanged_Lambda([this](const FText& SearchText)
						{
							if (StagePreviewTree)
							{
								StagePreviewTree->SetFilterText(SearchText);
							}
						})
					]
				]

				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(0.0f)
				[
					StagePreviewTree.ToSharedRef()
				]
			];
		}

		Splitter->AddSlot()
		.Value(0.5f)
		[
			DetailsViewBox.ToSharedRef()
		];
	}

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(10.0f, 0.0f, 10.0f, 0.0f))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f)
			[
				Splitter
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(8)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)

				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(AcceptText)
					.OnClicked(this, &SUsdOptionsWindow::OnAccept)
				]

				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("USDOptionWindow_Cancel", "Cancel"))
					.OnClicked(this, &SUsdOptionsWindow::OnCancel)
				]
			]
		]
	];
	// clang-format on
}

bool SUsdOptionsWindow::SupportsKeyboardFocus() const
{
	return true;
}

FReply SUsdOptionsWindow::OnAccept()
{
	if (OptionsObject)
	{
		OptionsObject->SaveConfig();
	}

	bAccepted = true;
	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SUsdOptionsWindow::OnCancel()
{
	bAccepted = false;
	if (Window.IsValid())
	{
		Window.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SUsdOptionsWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}
	return FReply::Unhandled();
}

bool SUsdOptionsWindow::UserAccepted() const
{
	return bAccepted;
}

TArray<FString> SUsdOptionsWindow::GetSelectedFullPrimPaths() const
{
	if (StagePreviewTree)
	{
		return StagePreviewTree->GetSelectedFullPrimPaths();
	}

	return {};
}

#undef LOCTEXT_NAMESPACE
