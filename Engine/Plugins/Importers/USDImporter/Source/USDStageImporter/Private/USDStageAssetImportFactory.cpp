// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageAssetImportFactory.h"

#include "USDAssetImportData.h"
#include "USDAssetUserData.h"
#include "USDConversionUtils.h"
#include "USDErrorUtils.h"
#include "USDObjectUtils.h"
#include "USDStageImporter.h"
#include "USDStageImporterModule.h"
#include "USDStageImportOptions.h"

#include "AssetImportTask.h"
#include "Editor.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageAssetImportFactory)

#define LOCTEXT_NAMESPACE "USDStageAssetImportFactory"

UUsdStageAssetImportFactory::UUsdStageAssetImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = nullptr;

	// Lower our priority to let Interchange/others try and import/reimport first.
	// We use a custom asset import data anyway, so the order likely doesn't matter.
	ImportPriority -= 10000;

	bEditorImport = true;
	bText = false;

	FModuleManager::Get().LoadModuleChecked(TEXT("UnrealUSDWrapper"));
	UnrealUSDWrapper::AddUsdImportFileFormatDescriptions(Formats);
}

bool UUsdStageAssetImportFactory::DoesSupportClass(UClass* Class)
{
	return (Class == UStaticMesh::StaticClass() || Class == USkeletalMesh::StaticClass());
}

UClass* UUsdStageAssetImportFactory::ResolveSupportedClass()
{
	return UStaticMesh::StaticClass();
}

UObject* UUsdStageAssetImportFactory::FactoryCreateFile(
	UClass* InClass,
	UObject* InParent,
	FName InName,
	EObjectFlags Flags,
	const FString& Filename,
	const TCHAR* Parms,
	FFeedbackContext* Warn,
	bool& bOutOperationCanceled
)
{
	UObject* ImportedObject = nullptr;

	if (AssetImportTask && IsAutomatedImport())
	{
		ImportContext.ImportOptions = Cast<UUsdStageImportOptions>(AssetImportTask->Options);
	}

	// When importing from file we don't want to use any opened stage
	ImportContext.bReadFromStageCache = false;

	// We shouldn't be able to import actors when doing a manual asset import (from the content browser)
	TOptional<UsdUtils::FScopedSuppressActorImport> SuppressActorImport;
	if (!IsAutomatedImport())
	{
		SuppressActorImport.Emplace(ImportContext.ImportOptions);
	}

	const FString InitialPackagePath = InParent ? InParent->GetName() : TEXT("/Game/");
	const bool bIsReimport = false;
	if (ImportContext.Init(InName.ToString(), Filename, InitialPackagePath, Flags, IsAutomatedImport(), bIsReimport))
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Parms);

		FScopedUsdMessageLog ScopedMessageLog;

		UUsdStageImporter* USDImporter = IUsdStageImporterModule::Get().GetImporter();
		USDImporter->ImportFromFile(ImportContext);

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ImportContext.SceneActor);
		GEditor->BroadcastLevelActorListChanged();

		ImportedObject = ImportContext.ImportedAsset ? ToRawPtr(ImportContext.ImportedAsset) : Cast<UObject>(ImportContext.SceneActor);

		ImportContext.ImportedAssets.Remove(ImportedObject);
		AdditionalImportedObjects = ImportContext.ImportedAssets;
	}
	else
	{
		bOutOperationCanceled = true;
	}

	return ImportedObject;
}

bool UUsdStageAssetImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	for (const FString& SupportedExtension : UnrealUSDWrapper::GetAllSupportedFileFormats())
	{
		if (SupportedExtension.Equals(Extension, ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	return false;
}

void UUsdStageAssetImportFactory::CleanUp()
{
	ImportContext.Reset();
	Super::CleanUp();
}

bool UUsdStageAssetImportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	// It seems that every import attempt creates a new factory instance, which is just left behind for GC to collect.
	// Since we may emit a warning from here, that means that if we do 5 imports and then want to emit the warning,
	// all of our 5 factory instances will do it (+ the CDO), and we'll get spammed...
	// Conversely, if we force GC before import, it will clean up those leftover factories and emit the warning only once.
	// We can avoid these issues by only checking for reimport via the Factory's CDO, which is guaranteed to be
	// registered exactly once.
	if (!IsTemplate())
	{
		return false;
	}

	// Try looking for the exact UUsdAssetImportData the USDImporter emits
	bool bFromOtherImporter = false;
	UAssetImportData* ImportData = UsdUnreal::ObjectUtils::GetAssetImportData(Obj);

	// Failing that, check if we have any other asset import data with USD file extensions, but
	// just so that we can emit a warning for some feedback
	if (!ImportData)
	{
		bFromOtherImporter = true;
		ImportData = UsdUnreal::ObjectUtils::GetBaseAssetImportData(Obj);
	}

	if (ImportData)
	{
		const FString FileName = ImportData->GetFirstFilename();
		const FString FileExtension = FPaths::GetExtension(FileName);

		// Reimporting from here means opening FileName as a USD stage and trying to re-read the same prims,
		// so make sure we only claim we can reimport something if that would work. Otherwise we may intercept
		// some other formats like .vdb files and then fail to open them as stages
		for (const FString& Extension : UnrealUSDWrapper::GetNativeFileFormats())
		{
			if (Extension == FileExtension)
			{
				if (bFromOtherImporter)
				{
					// This came from Interchange or some other custom USD format importer, just emit a warning
					// and return. Note that our factory has very low ImportPriority, so if we're here it's very
					// likely nothing else can handle this asset anyway, so it's probably OK to emit a warning
					FScopedUsdMessageLog ScopedMessageLog;

					USD_LOG_USERWARNING(FText::Format(
						LOCTEXT(
							"ReimportErrorWrongImportData",
							"Skipped trying to reimport asset '{0}' with the USD Importer plugin as it doesn't seem to have valid USD import data or user data! Only assets created by the USD Importer plugin can be reimported by the USD Importer plugin."
						),
						FText::FromName(Obj->GetFName())
					));
					return false;
				}
				else
				{
					// We can actually reimport this
					OutFilenames.Add(ImportData->GetFirstFilename());
					return true;
				}
			}
		}
	}

	return false;
}

void UUsdStageAssetImportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	if (NewReimportPaths.Num() != 1)
	{
		return;
	}

	if (UUsdAssetImportData* ImportData = UsdUnreal::ObjectUtils::GetAssetImportData(Obj))
	{
		ImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UUsdStageAssetImportFactory::Reimport(UObject* Obj)
{
	if (!Obj)
	{
		USD_LOG_USERERROR(LOCTEXT("ReimportErrorInvalidAsset", "Failed to reimport asset as it is invalid!"));
		return EReimportResult::Failed;
	}

	FScopedUsdMessageLog ScopedMessageLog;

	FString ReimportFilePath;
	FString OriginalPrimPath;
	UObject* ReimportOptions = nullptr;

	if (IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(Obj))
	{
		if (UUsdAssetUserData* UserData = UserDataInterface->GetAssetUserData<UUsdAssetUserData>())
		{
			if (!UserData->PrimPaths.IsEmpty())
			{
				OriginalPrimPath = UserData->PrimPaths[0];
			}
		}
	}

	if (UUsdAssetImportData* ImportData = UsdUnreal::ObjectUtils::GetAssetImportData(Obj))
	{
		ReimportFilePath = ImportData->GetFirstFilename();
		ReimportOptions = ImportData->ImportOptions;
	}

	if (ReimportFilePath.IsEmpty() || OriginalPrimPath.IsEmpty())
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("ReimportErrorNoImportData", "Failed to reimport asset '{0}' as it doesn't seem to have valid USD import data or user data!"),
			FText::FromName(Obj->GetFName())
		));
		return EReimportResult::Failed;
	}

	if (ReimportOptions)
	{
		// Duplicate this as we may update these options on the Init call below, and if we just imported a scene and
		// all assets are in memory (sharing the same import options object), that update would otherwise affect all the
		// UUsdAssetImportData objects, which is not what we would expect
		ImportContext.ImportOptions = Cast<UUsdStageImportOptions>(DuplicateObject(ReimportOptions, GetTransientPackage()));
	}

	ImportContext.bReadFromStageCache = false;

	const bool bIsAutomated = IsAutomatedImport() || IsAutomatedReimport();

	// We shouldn't be able to import actors when doing a manual asset reimport (from the content browser)
	TOptional<UsdUtils::FScopedSuppressActorImport> SuppressActorImport;
	if (!bIsAutomated)
	{
		SuppressActorImport.Emplace(ImportContext.ImportOptions);
	}

	const bool bIsReimport = true;
	if (!ImportContext.Init(Obj->GetName(), ReimportFilePath, Obj->GetName(), Obj->GetFlags(), bIsAutomated, bIsReimport))
	{
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT("ReimportErrorNoContext", "Failed to initialize re-import context for asset '{0}'!"),
			FText::FromName(Obj->GetFName())
		));
		return EReimportResult::Cancelled;
	}

	ImportContext.PackagePath = Obj->GetOutermost()->GetPathName();

	UUsdStageImporter* USDImporter = IUsdStageImporterModule::Get().GetImporter();
	UObject* ReimportedAsset = nullptr;
	bool bSuccess = USDImporter->ReimportSingleAsset(ImportContext, Obj, OriginalPrimPath, ReimportedAsset);

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(Obj);

	return bSuccess ? EReimportResult::Succeeded : EReimportResult::Failed;
}

int32 UUsdStageAssetImportFactory::GetPriority() const
{
	return ImportPriority;
}

#undef LOCTEXT_NAMESPACE
