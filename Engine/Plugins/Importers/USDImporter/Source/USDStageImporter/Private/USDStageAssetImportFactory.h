// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDStageImportContext.h"

#include "EditorReimportHandler.h"
#include "Factories/Factory.h"

#include "USDStageAssetImportFactory.generated.h"

/** Factory to import USD files that gets called when we hit Import in the Content Browser, as well as during reimport */
UCLASS(hidecategories = Object)
class UUsdStageAssetImportFactory
	: public UFactory
	, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

public:
	/** UFactory interface */
	virtual bool DoesSupportClass(UClass* Class) override;
	virtual UClass* ResolveSupportedClass() override;
	virtual UObject* FactoryCreateFile(
		UClass* InClass,
		UObject* InParent,
		FName InName,
		EObjectFlags Flags,
		const FString& Filename,
		const TCHAR* Parms,
		FFeedbackContext* Warn,
		bool& bOutOperationCanceled
	) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual void CleanUp() override;

	/** FReimportHandler interface */
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames);
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths);
	virtual EReimportResult::Type Reimport(UObject* Obj);
	virtual int32 GetPriority() const override;

private:
	UPROPERTY()
	FUsdStageImportContext ImportContext;
};
