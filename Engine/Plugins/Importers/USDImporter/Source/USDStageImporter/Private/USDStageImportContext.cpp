// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageImportContext.h"

#include "USDOptionsWindow.h"
#include "USDStageImportOptions.h"

#include "Dialogs/DlgPickPath.h"
#include "Editor.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageImportContext)

FUsdStageImportContext::FUsdStageImportContext()
{
	Reset();
}

bool FUsdStageImportContext::Init(
	const FString& InName,
	const FString& InFilePath,
	const FString& InInitialPackagePath,
	EObjectFlags InFlags,
	bool bInIsAutomated,
	bool bIsReimport,
	bool bAllowActorImport
)
{
	ObjectName = InName;
	FilePath = InFilePath;
	bIsAutomated = bInIsAutomated;
	ImportObjectFlags = InFlags;
	World = GEditor->GetEditorWorldContext().World();
	PackagePath = InInitialPackagePath;

	if (!PackagePath.EndsWith(TEXT("/")))
	{
		PackagePath.Append(TEXT("/"));
	}

	FPaths::NormalizeFilename(FilePath);

	// Open the stage if we haven't yet, as we'll need it open to show the preview tree
	if (!Stage)
	{
		const EUsdInitialLoadSet InitialLoadSet = EUsdInitialLoadSet::LoadAll;
		Stage = UnrealUSDWrapper::OpenStage(*FilePath, InitialLoadSet, bReadFromStageCache);
	}

	if (!bIsAutomated)
	{
		// Show dialog for content folder
		if (!bIsReimport)
		{
			// clang-format off
			TSharedRef<SDlgPickPath> PickContentPathDlg =
				SNew(SDlgPickPath)
				.Title(NSLOCTEXT("USDStageImportContext", "ChooseImportRootContentPath", "Choose where to place the imported USD assets"))
				.DefaultPath(FText::FromString(InInitialPackagePath));
			// clang-format on

			if (PickContentPathDlg->ShowModal() == EAppReturnType::Cancel)
			{
				return false;
			}
			// e.g. "/Game/MyFolder/layername/"
			// We inject the package path here because this is what the automated import task upstream code will do. This way the importer
			// can always expect to receive /ContentPath/layername/
			PackagePath = FString::Printf(TEXT("%s/%s/"), *PickContentPathDlg->GetPath().ToString(), *InName);
		}

		// Update the boundaries of the subdivision level slider according to the cvar
#if WITH_EDITOR
		static IConsoleVariable* MaxSubdivCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.Subdiv.MaxSubdivLevel"));
		const int32 MaxSubdivLevel = FMath::Max(MaxSubdivCvar ? MaxSubdivCvar->GetInt() : 6, 1);
		const FString MaxSubdivLevelString = LexToString(MaxSubdivLevel);

		if (FProperty* Property = UUsdStageImportOptions::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, SubdivisionLevel)))
		{
			Property->SetMetaData(TEXT("UIMax"), *MaxSubdivLevelString);
			Property->SetMetaData(TEXT("ClampMax"), *MaxSubdivLevelString);
		}
#endif	  // WITH_EDITOR

		// Show dialog for import options
		bool bProceedWithImport = SUsdOptionsWindow::ShowImportOptions(*ImportOptions, &Stage);
		if (!bProceedWithImport)
		{
			return false;
		}
	}

	return true;
}

void FUsdStageImportContext::Reset()
{
	World = nullptr;
	TargetSceneActorTargetTransform = FTransform::Identity;
	TargetSceneActorAttachParent = nullptr;
	SceneActor = nullptr;
	ObjectName = FString{};
	PackagePath = FString{};
	FilePath = FString{};
	ImportOptions = NewObject<UUsdStageImportOptions>();
	ImportedAsset = nullptr;
	ImportedAssets.Reset();
	LevelSequenceHelper.Clear();
	UsdAssetCache = nullptr;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AssetCache_DEPRECATED = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	BBoxCache = nullptr;
	MaterialToPrimvarToUVIndex.Empty();
	Stage = UE::FUsdStage{};
	ImportObjectFlags = EObjectFlags::RF_NoFlags;
	bIsAutomated = false;

	// Always reading from the stage cache is a good default because while we can have multiple instances of the
	// same stage open, USD will only open a particular layer once. If we try importing without using the stage
	// cache and the stage we want to import uses an existing open layer, we will forcibly reload
	// that layer (check UnrealUSDWrapper::OpenStage), which would erase our previous changes to it and lead to modifications
	// on existing open stages (e.g. we have cube.usda open with a stage actor and with some changes and we click
	// file -> import into level and import cube.usda again)
	bReadFromStageCache = true;

	bStageWasOriginallyOpenInCache = false;
	OriginalMetersPerUnit = 0.01;
	OriginalUpAxis = EUsdUpAxis::ZAxis;

	bNeedsGarbageCollection = false;
}
