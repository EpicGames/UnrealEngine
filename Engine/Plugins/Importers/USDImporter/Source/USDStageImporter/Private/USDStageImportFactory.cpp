// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageImportFactory.h"

#include "USDErrorUtils.h"
#include "USDStageImporter.h"
#include "USDStageImporterModule.h"
#include "USDStageImportOptions.h"

#include "AssetImportTask.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageImportFactory)

#define LOCTEXT_NAMESPACE "USDImportFactory"

UUsdStageImportFactory::UUsdStageImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UWorld::StaticClass();

	ImportPriority += 100;

	bEditorImport = true;
	bText = false;

	FModuleManager::Get().LoadModuleChecked(TEXT("UnrealUSDWrapper"));
	UnrealUSDWrapper::AddUsdImportFileFormatDescriptions(Formats);
}

UObject* UUsdStageImportFactory::FactoryCreateFile(
	UClass* InClass,
	UObject* InParent,
	FName InName,
	EObjectFlags Flags,
	const FString& Filename,
	const TCHAR* Parms,
	FFeedbackContext* Warn,
	bool& bOutOperationCanceled
)
{
	UObject* ImportedObject = nullptr;

	if (AssetImportTask && IsAutomatedImport())
	{
		ImportContext.ImportOptions = Cast<UUsdStageImportOptions>(AssetImportTask->Options);
	}

	// When importing from file we don't want to use any opened stage
	ImportContext.bReadFromStageCache = false;

#if USE_USD_SDK
	const FString InitialPackagePath = InParent ? InParent->GetName() : TEXT("/Game/");
	const bool bIsReimport = false;
	if (ImportContext.Init(InName.ToString(), Filename, InitialPackagePath, Flags, IsAutomatedImport(), bIsReimport))
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Parms);

		FScopedUsdMessageLog ScopedMessageLog;

		UUsdStageImporter* USDImporter = IUsdStageImporterModule::Get().GetImporter();
		USDImporter->ImportFromFile(ImportContext);

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ImportContext.World);
		GEditor->BroadcastLevelActorListChanged();
		GEditor->RedrawLevelEditingViewports();

		ImportedObject = ImportContext.ImportedAsset ? ToRawPtr(ImportContext.ImportedAsset) : Cast<UObject>(ImportContext.SceneActor);

		ImportContext.ImportedAssets.Remove(ImportedObject);
		AdditionalImportedObjects = ImportContext.ImportedAssets;
	}
	else
	{
		bOutOperationCanceled = true;
	}

#endif	  // #if USE_USD_SDK

	return ImportedObject;
}

bool UUsdStageImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	for (const FString& SupportedExtension : UnrealUSDWrapper::GetAllSupportedFileFormats())
	{
		if (SupportedExtension.Equals(Extension, ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	return false;
}

void UUsdStageImportFactory::CleanUp()
{
	ImportContext.Reset();
	Super::CleanUp();
}

#undef LOCTEXT_NAMESPACE
