// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "USDStageImportContext.h"

#include "Factories/SceneImportFactory.h"

#include "USDStageImportFactory.generated.h"

/** Factory to import USD files that gets called when we hit File -> Import into level... */
UCLASS(Transient)
class UUsdStageImportFactory : public USceneImportFactory
{
	GENERATED_UCLASS_BODY()

public:
	// UFactory Interface
	virtual UObject* FactoryCreateFile(
		UClass* InClass,
		UObject* InParent,
		FName InName,
		EObjectFlags Flags,
		const FString& Filename,
		const TCHAR* Parms,
		FFeedbackContext* Warn,
		bool& bOutOperationCanceled
	) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual void CleanUp() override;

private:
	UPROPERTY()
	FUsdStageImportContext ImportContext;
};
