// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageImportOptions.h"

#include "Objects/USDSchemaTranslator.h"
#include "USDSchemasModule.h"

#include "AnalyticsEventAttribute.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDStageImportOptions)

UUsdStageImportOptions::UUsdStageImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bImportActors = true;
	bImportGeometry = true;
	bImportSkeletalAnimations = true;
	bImportLevelSequences = true;
	bImportMaterials = true;
	bImportGroomAssets = true;
	bImportSparseVolumeTextures = true;
	bImportSounds = true;
	bImportOnlyUsedMaterials = false;

	bUseExistingAssetCache = false;
	ExistingAssetCache = nullptr;
	PurposesToImport = (int32)(EUsdPurpose::Default | EUsdPurpose::Proxy | EUsdPurpose::Render | EUsdPurpose::Guide);
	NaniteTriangleThreshold = INT32_MAX;
	IUsdSchemasModule& UsdSchemasModule = FModuleManager::Get().LoadModuleChecked<IUsdSchemasModule>(TEXT("USDSchemas"));
	RenderContextToImport = UnrealIdentifiers::UnrealRenderContext;
	MaterialPurpose = *UnrealIdentifiers::MaterialPreviewPurpose;
	SubdivisionLevel = 0;
	MetadataOptions = FUsdMetadataImportOptions{
		true,  /* bCollectMetadata */
		true,  /* bCollectFromEntireSubtrees */
		false, /* bCollectOnComponents */
		{},	   /* BlockedPrefixFilters */
		false  /* bInvertFilters */
	};
	bOverrideStageOptions = false;
	StageOptions.MetersPerUnit = 0.01;
	StageOptions.UpAxis = EUsdUpAxis::ZAxis;
	bImportAtSpecificTimeCode = false;
	ImportTimeCode = 0.0f;

	ExistingActorPolicy = EReplaceActorPolicy::Replace;
	ExistingAssetPolicy = EReplaceAssetPolicy::Replace;
	bShareAssetsForIdenticalPrims = true;

	bPrimPathFolderStructure = false;
	KindsToCollapse = (int32)(EUsdDefaultKind::Component | EUsdDefaultKind::Subcomponent);
	bUsePrimKindsForCollapsing = true;
	bMergeIdenticalMaterialSlots = true;
	bInterpretLODs = true;
}

// Deprecated
void UUsdStageImportOptions::EnableActorImport(bool bEnable)
{
}

void UUsdStageImportOptions::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		SaveConfig();
	}
}

void UsdUtils::AddAnalyticsAttributes(const UUsdStageImportOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes)
{
	InOutAttributes.Emplace(TEXT("ImportActors"), LexToString(Options.bImportActors));
	InOutAttributes.Emplace(TEXT("ImportGeometry"), LexToString(Options.bImportGeometry));
	InOutAttributes.Emplace(TEXT("ImportSkeletalAnimations"), LexToString(Options.bImportSkeletalAnimations));
	InOutAttributes.Emplace(TEXT("ImportLevelSequences"), LexToString(Options.bImportLevelSequences));
	InOutAttributes.Emplace(TEXT("ImportMaterials"), LexToString(Options.bImportMaterials));
	InOutAttributes.Emplace(TEXT("ImportGroomAssets"), LexToString(Options.bImportGroomAssets));
	InOutAttributes.Emplace(TEXT("ImportSparseVolumeTextures"), LexToString(Options.bImportSparseVolumeTextures));
	InOutAttributes.Emplace(TEXT("ImportSounds"), LexToString(Options.bImportSounds));
	InOutAttributes.Emplace(TEXT("ImportOnlyUsedMaterials"), LexToString(Options.bImportOnlyUsedMaterials));
	if (Options.PrimsToImport != TArray<FString>{TEXT("/")})
	{
		InOutAttributes.Emplace(TEXT("NumPrimsToImport"), LexToString(Options.PrimsToImport.Num()));
	}

	InOutAttributes.Emplace(TEXT("bUseExistingAssetCache"), Options.bUseExistingAssetCache);
	InOutAttributes.Emplace(TEXT("PurposesToImport"), LexToString(Options.PurposesToImport));
	InOutAttributes.Emplace(TEXT("NaniteTriangleThreshold"), LexToString(Options.NaniteTriangleThreshold));
	InOutAttributes.Emplace(TEXT("RenderContextToImport"), Options.RenderContextToImport.ToString());
	InOutAttributes.Emplace(TEXT("MaterialPurpose"), Options.MaterialPurpose.ToString());
	InOutAttributes.Emplace(TEXT("RootMotionHandling"), LexToString((uint8)Options.RootMotionHandling));
	InOutAttributes.Emplace(TEXT("FallbackCollisionType"), (uint8)Options.FallbackCollisionType);
	InOutAttributes.Emplace(TEXT("SubdivisionLevel"), LexToString(Options.SubdivisionLevel));

	UsdUtils::AddAnalyticsAttributes(Options.MetadataOptions, InOutAttributes);

	InOutAttributes.Emplace(TEXT("OverrideStageOptions"), Options.bOverrideStageOptions);
	if (Options.bOverrideStageOptions)
	{
		UsdUtils::AddAnalyticsAttributes(Options.StageOptions, InOutAttributes);
	}
	InOutAttributes.Emplace(TEXT("ImportAtSpecificTimeCode"), Options.bImportAtSpecificTimeCode);
	if (Options.bImportAtSpecificTimeCode)
	{
		InOutAttributes.Emplace(TEXT("ImportTimeCode"), LexToString(Options.ImportTimeCode));
	}
	InOutAttributes.Emplace(TEXT("NumGroomInterpolationSettings"), LexToString(Options.GroomInterpolationSettings.Num()));
	InOutAttributes.Emplace(TEXT("ReplaceActorPolicy"), LexToString((uint8)Options.ExistingActorPolicy));
	InOutAttributes.Emplace(TEXT("ReplaceAssetPolicy"), LexToString((uint8)Options.ExistingAssetPolicy));
	InOutAttributes.Emplace(TEXT("ShareAssetsForIdenticalPrims"), Options.bShareAssetsForIdenticalPrims);
	InOutAttributes.Emplace(TEXT("PrimPathFolderStructure"), LexToString(Options.bPrimPathFolderStructure));
	InOutAttributes.Emplace(TEXT("KindsToCollapse"), LexToString(Options.KindsToCollapse));
	InOutAttributes.Emplace(TEXT("bUsePrimKindsForCollapsing"), Options.bUsePrimKindsForCollapsing);
	InOutAttributes.Emplace(TEXT("MergeIdenticalMaterialSlots"), LexToString(Options.bMergeIdenticalMaterialSlots));
	InOutAttributes.Emplace(TEXT("InterpretLODs"), LexToString(Options.bInterpretLODs));
}

UsdUtils::FScopedSuppressActorImport::FScopedSuppressActorImport(UUsdStageImportOptions* InOptions)
	: Options{InOptions}
{
	for (TFieldIterator<FProperty> PropertyIterator(UUsdStageImportOptions::StaticClass()); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;
		if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, bImportActors))
		{
			Property->SetMetaData(
				TEXT("ToolTip"),
				TEXT("Actor import is disabled when importing via the Content Browser. Use File->\"Import into Level...\" to also import actors.")
			);
			Property->SetPropertyFlags(CPF_EditConst);
			break;
		}
	}

	if (UUsdStageImportOptions* OptionsPtr = Options.Get())
	{
		bOldImportActorsValue = OptionsPtr->bImportActors;
		OptionsPtr->bImportActors = false;
		OptionsPtr->SaveConfig();
	}
}

UsdUtils::FScopedSuppressActorImport::~FScopedSuppressActorImport()
{
	for (TFieldIterator<FProperty> PropertyIterator(UUsdStageImportOptions::StaticClass()); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;
		if (Property && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, bImportActors))
		{
			Property->SetMetaData(TEXT("ToolTip"), TEXT("Whether to spawn imported actors into the current level"));
			Property->ClearPropertyFlags(CPF_EditConst);
			break;
		}
	}

	if (UUsdStageImportOptions* OptionsPtr = Options.Get())
	{
		OptionsPtr->bImportActors = bOldImportActorsValue;
		OptionsPtr->SaveConfig();
	}
}
