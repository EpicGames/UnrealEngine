// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageImportOptionsCustomization.h"

#include "Objects/USDSchemaTranslator.h"
#include "USDMaterialUtils.h"
#include "USDProjectSettings.h"
#include "USDStageImportOptions.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "UObject/Object.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "UsdStageImportOptionsCustomization"

FUsdStageImportOptionsCustomization::FUsdStageImportOptionsCustomization()
{
}

TSharedRef<IDetailCustomization> FUsdStageImportOptionsCustomization::MakeInstance()
{
	return MakeShared<FUsdStageImportOptionsCustomization>();
}

void FUsdStageImportOptionsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	TWeakObjectPtr<UObject> SelectedObject = SelectedObjects[0];
	if (!SelectedObject.IsValid())
	{
		return;
	}

	CurrentOptions = Cast<UUsdStageImportOptions>(SelectedObject.Get());
	if (!CurrentOptions)
	{
		return;
	}

	// Hide this property since we'll show the preview tree for it
	DetailLayoutBuilder.EditCategory(TEXT("Prims to Import"));
	if (TSharedPtr<IPropertyHandle> PrimsToImportProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, PrimsToImport)
		))
	{
		DetailLayoutBuilder.HideProperty(PrimsToImportProperty);
	}

	RenderContextComboBoxItems.Reset();
	TSharedPtr<FString> InitiallySelectedContext;
	for (const FName& Context : UsdUnreal::MaterialUtils::GetRegisteredRenderContexts())
	{
		TSharedPtr<FString> ContextStr;
		if (Context == UnrealIdentifiers::UniversalRenderContext)
		{
			ContextStr = MakeShared<FString>(UnrealIdentifiers::UniversalRenderContextDisplayString);
		}
		else
		{
			ContextStr = MakeShared<FString>(Context.ToString());
		}

		if (Context == CurrentOptions->RenderContextToImport)
		{
			InitiallySelectedContext = ContextStr;
		}

		RenderContextComboBoxItems.Add(ContextStr);
	}

	IDetailCategoryBuilder& CatBuilder = DetailLayoutBuilder.EditCategory(TEXT("USD options"));

	if (TSharedPtr<IPropertyHandle> RenderContextProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, RenderContextToImport)
		))
	{
		DetailLayoutBuilder.HideProperty(RenderContextProperty);

		// clang-format off
		CatBuilder.AddCustomRow(FText::FromString(TEXT("RenderContextCustomization")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Render Context to Import")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(RenderContextProperty->GetToolTipText())
		]
		.ValueContent()
		[
			SAssignNew(RenderContextComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&RenderContextComboBoxItems)
			.InitiallySelectedItem(InitiallySelectedContext)
			.OnSelectionChanged(this, &FUsdStageImportOptionsCustomization::OnComboBoxSelectionChanged)
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> Item)
			{
				return SNew(STextBlock)
					.Text(Item.IsValid() ? FText::FromString(*Item) : FText::GetEmpty())
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
			})
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &FUsdStageImportOptionsCustomization::GetComboBoxSelectedOptionText)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];
		// clang-format on
	}

	if (TSharedPtr<IPropertyHandle> MaterialPurposeProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, MaterialPurpose)
		))
	{
		DetailLayoutBuilder.HideProperty(MaterialPurposeProperty);

		// clang-format off
		CatBuilder.AddCustomRow(FText::FromString(TEXT("MaterialPurposeCustomization")))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Material purpose")))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.ToolTipText(MaterialPurposeProperty->GetToolTipText())
		]
		.ValueContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&MaterialPurposeComboBoxItems)
				.OnComboBoxOpening_Lambda([this]()
				{
					MaterialPurposeComboBoxItems = {
						MakeShared<FString>(UnrealIdentifiers::MaterialAllPurpose),
						MakeShared<FString>(UnrealIdentifiers::MaterialPreviewPurpose),
						MakeShared<FString>(UnrealIdentifiers::MaterialFullPurpose)
					};

					// Add additional purposes from project settings
					if (const UUsdProjectSettings* ProjectSettings = GetDefault<UUsdProjectSettings>())
					{
						MaterialPurposeComboBoxItems.Reserve(MaterialPurposeComboBoxItems.Num() + ProjectSettings->AdditionalMaterialPurposes.Num());

						TSet<FString> ExistingEntries = {
							UnrealIdentifiers::MaterialAllPurpose,
							UnrealIdentifiers::MaterialPreviewPurpose,
							UnrealIdentifiers::MaterialFullPurpose
						};

						for (const FName& AdditionalPurpose : ProjectSettings->AdditionalMaterialPurposes)
						{
							FString AdditionalPurposeStr = AdditionalPurpose.ToString();

							if (!ExistingEntries.Contains(AdditionalPurposeStr))
							{
								ExistingEntries.Add(AdditionalPurposeStr);
								MaterialPurposeComboBoxItems.AddUnique(MakeShared<FString>(AdditionalPurposeStr));
							}
						}
					}
				})
				.OnGenerateWidget_Lambda([&](TSharedPtr<FString> Option)
				{
					TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
					if (Option)
					{
						Widget = SNew(STextBlock)
							.Text(FText::FromString((*Option) == UnrealIdentifiers::MaterialAllPurpose
								? UnrealIdentifiers::MaterialAllPurposeText
								: *Option
							))
							.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));
					}

					return Widget.ToSharedRef();
				})
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> ChosenOption, ESelectInfo::Type SelectInfo)
				{
					if (CurrentOptions && ChosenOption)
					{
						CurrentOptions->MaterialPurpose = **ChosenOption;
					}
				})
				[
					SNew(SEditableTextBox)
					.Text_Lambda([this]() -> FText
					{
						if (CurrentOptions)
						{
							return FText::FromString(CurrentOptions->MaterialPurpose == *UnrealIdentifiers::MaterialAllPurpose
								? UnrealIdentifiers::MaterialAllPurposeText
								: CurrentOptions->MaterialPurpose.ToString()
							);
						}

						return FText::GetEmpty();
					})
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.OnTextCommitted_Lambda([this](const FText& NewText, ETextCommit::Type CommitType)
					{
						if (CommitType != ETextCommit::OnEnter)
						{
							return;
						}

						FString NewPurposeString = NewText.ToString();
						FName NewPurpose = *NewPurposeString;

						bool bIsNew = true;
						for (const TSharedPtr<FString>& Purpose : MaterialPurposeComboBoxItems)
						{
							if (Purpose && *Purpose == NewPurposeString)
							{
								bIsNew = false;
								break;
							}
						}

						if (bIsNew)
						{
							if (UUsdProjectSettings* ProjectSettings = GetMutableDefault<UUsdProjectSettings>())
							{
								ProjectSettings->AdditionalMaterialPurposes.AddUnique(NewPurpose);
								ProjectSettings->SaveConfig();
							}
						}

						if (CurrentOptions)
						{
							CurrentOptions->MaterialPurpose = NewPurpose;
						}
					})
				]
			]
		];
		// clang-format on
	}

	// Add/remove properties so that they retain their usual order
	if (TSharedPtr<IPropertyHandle> OverrideStageOptionsProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, bOverrideStageOptions)
		))
	{
		CatBuilder.AddProperty(OverrideStageOptionsProperty);
	}
	if (TSharedPtr<IPropertyHandle> StageOptionsProperty = DetailLayoutBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UUsdStageImportOptions, StageOptions)
		))
	{
		CatBuilder.AddProperty(StageOptionsProperty);
	}
}

void FUsdStageImportOptionsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder)
{
	CustomizeDetails(*DetailBuilder);
}

void FUsdStageImportOptionsCustomization::OnComboBoxSelectionChanged(TSharedPtr<FString> NewContext, ESelectInfo::Type SelectType)
{
	if (CurrentOptions == nullptr || !NewContext.IsValid())
	{
		return;
	}

	FName NewContextName = (*NewContext) == UnrealIdentifiers::UniversalRenderContextDisplayString ? UnrealIdentifiers::UniversalRenderContext
																								   : FName(**NewContext);

	CurrentOptions->RenderContextToImport = NewContextName;
}

FText FUsdStageImportOptionsCustomization::GetComboBoxSelectedOptionText() const
{
	TSharedPtr<FString> SelectedItem = RenderContextComboBox->GetSelectedItem();
	if (SelectedItem.IsValid())
	{
		return FText::FromString(*SelectedItem);
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
