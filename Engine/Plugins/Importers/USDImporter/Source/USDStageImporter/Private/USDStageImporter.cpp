// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageImporter.h"

#include "Objects/USDSchemaTranslator.h"
#include "USDAssetCache3.h"
#include "USDAssetImportData.h"
#include "USDAssetUserData.h"
#include "USDClassesModule.h"
#include "USDConversionUtils.h"
#include "USDDrawModeComponent.h"
#include "USDErrorUtils.h"
#include "USDGeomMeshConversion.h"
#include "USDInfoCacheObject.h"
#include "USDNaniteAssemblyTranslator.h"
#include "USDObjectUtils.h"
#include "USDPrimLinkCacheObject.h"
#include "USDPrimConversion.h"
#include "USDPrimTwin.h"
#include "USDSchemasModule.h"
#include "USDStageImportContext.h"
#include "USDStageImportOptions.h"
#include "USDTypesConversion.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/UsdGeomBBoxCache.h"
#include "UsdWrappers/UsdStage.h"
#include "UsdWrappers/UsdTyped.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AudioDeviceManager.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "EngineUtils.h"
#include "GeometryCache.h"
#include "GroomAsset.h"
#include "GroomBindingAsset.h"
#include "GroomCache.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "LevelSequence.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Paths.h"
#include "Misc/RedirectCollector.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "PackageTools.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Sections/MovieSceneSubSection.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundWave.h"
#include "SparseVolumeTexture/SparseVolumeTexture.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UObject/Linker.h"
#include "UObject/ReferencerFinder.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "USDStageImporter"

namespace UE::USDStageImporter::Private
{
	void OpenStage(FUsdStageImportContext& ImportContext, bool bNeedsMasking)
	{
		const FString FilePath = !ImportContext.FilePath.IsEmpty()
									 ? IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ImportContext.FilePath)
									 : FString{};

		if (FilePath.IsEmpty() && !ImportContext.Stage)
		{
			// There's nothing we can do here to create a stage
			return;
		}

		FScopedUsdMessageLog ScopedLog;

		UE::FUsdStage Stage;
		if (bNeedsMasking)
		{
			// If we're masking we'll make a unique stage for us.
			// Also, within USD the OpenMasked methods do not consult the stage cache anyway
			ImportContext.bReadFromStageCache = false;
			ImportContext.bStageWasOriginallyOpenInCache = false;

			// Here even though we won't use the stage cache we don't want to force reload the stage's layers either:
			// The whole point is that we're just opening another masked "view" into the same composed (potentially
			// modified) layers
			const bool bForceReloadLayersFromDisk = false;

			// We don't have a file path to reopen, so just reopen the existing layers
			if (FilePath.IsEmpty())
			{
				Stage = UnrealUSDWrapper::OpenMaskedStage(
					ImportContext.Stage.GetRootLayer(),
					ImportContext.Stage.GetSessionLayer(),
					EUsdInitialLoadSet::LoadAll,
					ImportContext.ImportOptions->PrimsToImport,
					bForceReloadLayersFromDisk
				);
			}
			else
			{
				Stage = UnrealUSDWrapper::OpenMaskedStage(
					*FilePath,
					EUsdInitialLoadSet::LoadAll,
					ImportContext.ImportOptions->PrimsToImport,
					bForceReloadLayersFromDisk
				);
			}
		}
		else
		{
			for (const UE::FUsdStage& OpenedStage : UnrealUSDWrapper::GetAllStagesFromCache())
			{
				FString RootPath = OpenedStage.GetRootLayer().GetRealPath();
				FPaths::NormalizeFilename(RootPath);
				if (ImportContext.FilePath == RootPath)
				{
					ImportContext.bStageWasOriginallyOpenInCache = true;
					break;
				}
			}

			const bool bForceReloadLayersFromDisk = !ImportContext.bReadFromStageCache;
			Stage = UnrealUSDWrapper::OpenStage(
				*FilePath,
				EUsdInitialLoadSet::LoadAll,
				ImportContext.bReadFromStageCache,
				bForceReloadLayersFromDisk
			);
		}

		if (FUsdLogManager::HasAccumulatedErrors())
		{
			ImportContext.Stage = UE::FUsdStage();
			USD_LOG_USERERROR(FText::Format(LOCTEXT("CouldNotImportUSDFile", "Could not import USD file {0}"), FText::FromString(FilePath)));
		}
		else
		{
			ImportContext.Stage = Stage;
		}
	}

	FString FindValidPackagePath(const FString& InPackagePath)
	{
		int32 Suffix = 0;
		FString SearchPackagePath = InPackagePath;
		UPackage* ExistingPackage = nullptr;

		do
		{
			// Look for the package in memory
			ExistingPackage = FindPackage(nullptr, *SearchPackagePath);

			// Look for the package on disk
			if (!ExistingPackage && FPackageName::DoesPackageExist(SearchPackagePath))
			{
				ExistingPackage = LoadPackage(nullptr, *SearchPackagePath, LOAD_None);
			}

			SearchPackagePath = InPackagePath + TEXT("_") + LexToString(Suffix++);
		} while (ExistingPackage != nullptr);

		// Undo the last SearchPackagePath update, returning the path that worked (vacant Package path)
		return Suffix == 1 ? InPackagePath : InPackagePath + TEXT("_") + LexToString(Suffix - 1);
	}

	void SetupSceneActor(FUsdStageImportContext& ImportContext)
	{
		if (!ImportContext.ImportOptions->bImportActors)
		{
			return;
		}

		ULevel* Level = ImportContext.World->GetCurrentLevel();
		if (!Level)
		{
			return;
		}

		FActorSpawnParameters SpawnParameters;
		SpawnParameters.ObjectFlags = ImportContext.ImportObjectFlags & ~(RF_Standalone | RF_Public | RF_Transient);
		SpawnParameters.OverrideLevel = Level;

		// We always spawn another scene actor regardless of collision or whether the level already has one,
		// so that we can fully build our hierarchy separately before resolving collisions according to ExistingActorPolicy
		AActor* Actor = ImportContext.World->SpawnActor(AActor::StaticClass(), nullptr, SpawnParameters);
		Actor->SetActorLabel(UsdUnreal::ObjectUtils::SanitizeObjectName(ImportContext.ObjectName));

		USceneComponent* RootComponent = Actor->GetRootComponent();
		if (!RootComponent)
		{
			RootComponent = NewObject<USceneComponent>(Actor, USceneComponent::GetDefaultSceneRootVariableName(), RF_Transactional);
			RootComponent->Mobility = EComponentMobility::Static;
			RootComponent->bVisualizeComponent = false;

			Actor->SetRootComponent(RootComponent);
			Actor->AddInstanceComponent(RootComponent);
		}

		if (RootComponent && !RootComponent->IsRegistered())
		{
			RootComponent->RegisterComponent();
		}

		if (ImportContext.TargetSceneActorAttachParent)
		{
			RootComponent->AttachToComponent(ImportContext.TargetSceneActorAttachParent, FAttachmentTransformRules::KeepRelativeTransform);
		}

		Actor->SetActorTransform(ImportContext.TargetSceneActorTargetTransform);

		ImportContext.SceneActor = Actor;
	}

	AActor* GetExistingSceneActor(FUsdStageImportContext& ImportContext)
	{
		// We always reuse the existing scene actor for a scene, regardless of ReplacePolicy
		FString TargetActorLabel = UsdUnreal::ObjectUtils::SanitizeObjectName(ImportContext.ObjectName);
		for (TActorIterator<AActor> ActorItr(ImportContext.World); ActorItr; ++ActorItr)
		{
			AActor* ThisActor = *ActorItr;

			// Found a top level actor with the same label
			if (!ThisActor->HasAnyFlags(RF_Transient) && ThisActor->GetAttachParentActor() == nullptr
				&& ThisActor->GetActorLabel() == TargetActorLabel && ThisActor != ImportContext.SceneActor)
			{
				return ThisActor;
			}
		}

		return nullptr;
	}

	void SetupStageForImport(FUsdStageImportContext& ImportContext)
	{
#if USE_USD_SDK
		if (ImportContext.ImportOptions->bOverrideStageOptions)
		{
			ImportContext.OriginalMetersPerUnit = UsdUtils::GetUsdStageMetersPerUnit(ImportContext.Stage);
			ImportContext.OriginalUpAxis = UsdUtils::GetUsdStageUpAxisAsEnum(ImportContext.Stage);

			UsdUtils::SetUsdStageMetersPerUnit(ImportContext.Stage, ImportContext.ImportOptions->StageOptions.MetersPerUnit);
			UsdUtils::SetUsdStageUpAxis(ImportContext.Stage, ImportContext.ImportOptions->StageOptions.UpAxis);
		}
#endif	  // #if USE_USD_SDK
	}

	void CreateAssetsForPrims(const TArray<UE::FUsdPrim>& Prims, FUsdSchemaTranslationContext& TranslationContext, const FText& ProgressMessage)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateAssetsForPrims);

		FScopedSlowTask SlowTask(Prims.Num(), ProgressMessage);

		for (const UE::FUsdPrim& Prim : Prims)
		{
			if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(
					TranslationContext.AsShared(),
					UE::FUsdTyped(Prim)
				))
			{
				SchemaTranslator->CreateAssets();
			}

			SlowTask.EnterProgressFrame();
		}

		TranslationContext.CompleteTasks();
	}

	void CreateNaniteAssemblyAssets(const TArray<UE::FUsdPrim>& Prims, FUsdSchemaTranslationContext& TranslationContext)
	{
#if USE_USD_SDK && WITH_EDITOR

		// If pointinstancer collapsing is enabled there won't be individual static mesh prototype assets
		// available to use as assembly references.
		static const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("USD.CollapseTopLevelPointInstancers"));
		if (CVar and CVar->GetBool())
		{
			USD_LOG_WARNING(
				TEXT("CVar 'USD.CollapseTopLevelPointInstancers' is enabled, which is currently incompatible with Static Mesh Nantite Assemblies "
					 "utilizing PointInstancer prims. Please disable this CVar when importing Nanite Assemblies.")
			);
		}

		for (const UE::FUsdPrim& Prim : Prims)
		{
			FUsdNaniteAssemblyTranslator AssemblyTranslator(TranslationContext.AsShared(), UE::FUsdTyped(Prim));
			AssemblyTranslator.CreateAssets();
		}

		TranslationContext.CompleteTasks();
#endif // USE_USD_SDK && WITH_EDITOR
	}

	void CacheCollapsingState(FUsdSchemaTranslationContext& TranslationContext)
	{
		// It's better to always rebuild the info cache because our import options may have changed
		// from the options used when the cache was first built, which could change collapsing states, etc.
		TranslationContext.UsdInfoCache->RebuildCacheForSubtrees({UE::FSdfPath::AbsoluteRootPath()}, TranslationContext);
	}

	void ImportMaterials(FUsdStageImportContext& ImportContext, FUsdSchemaTranslationContext& TranslationContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImportMaterials);

		if (!ImportContext.ImportOptions->bImportMaterials)
		{
			return;
		}

		TArray<UE::FUsdPrim> MaterialPrims = UsdUtils::GetAllPrimsOfType(ImportContext.Stage.GetPseudoRoot(), TEXT("UsdShadeMaterial"));

		CreateAssetsForPrims(MaterialPrims, TranslationContext, LOCTEXT("CreateMaterials", "Creating materials"));
	}

	void ImportMeshes(FUsdStageImportContext& ImportContext, FUsdSchemaTranslationContext& TranslationContext)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImportMeshes);

#if USE_USD_SDK
		if (!ImportContext.ImportOptions->bImportGeometry)
		{
			return;
		}

		auto PruneCollapsedMeshes = [&TranslationContext](const UE::FUsdPrim& UsdPrim) -> bool
		{
			if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get().CreateTranslatorForSchema(
					TranslationContext.AsShared(),
					UE::FUsdTyped(UsdPrim)
				))
			{
				return SchemaTranslator->CollapsesChildren(ECollapsingType::Assets);
			}

			return false;
		};

		TArray<UE::FUsdPrim> MeshPrims = UsdUtils::GetAllPrimsOfType(
			ImportContext.Stage.GetPseudoRoot(),
			TEXT("UsdGeomXformable"),
			PruneCollapsedMeshes
		);
		CreateAssetsForPrims(MeshPrims, TranslationContext, LOCTEXT("CreateMeshes", "Creating meshes"));

		CreateNaniteAssemblyAssets(MeshPrims, TranslationContext);

#endif	  // #if USE_USD_SDK
	}

	void ImportAnimation(
		FUsdStageImportContext& ImportContext,
		UE::FUsdPrim& Prim,
		bool bAnimatedVisibility,
		TOptional<bool> bHasAnimatedBounds,
		USceneComponent* SceneComponent
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImportAnimation);

		if (!ImportContext.ImportOptions->bImportLevelSequences)
		{
			return;
		}

		UUsdPrimTwin* UsdPrimTwin = NewObject<UUsdPrimTwin>();
		UsdPrimTwin->PrimPath = Prim.GetPrimPath().GetString();
		UsdPrimTwin->SceneComponent = SceneComponent;
		UsdPrimTwin->SceneComponent->SetMobility(EComponentMobility::Movable);

		ImportContext.LevelSequenceHelper.AddPrim(*UsdPrimTwin, bAnimatedVisibility, bHasAnimatedBounds);
	}

	void ImportActor(
		FUsdStageImportContext& ImportContext,
		UE::FUsdPrim& Prim,
		bool bForceVisibilityAnimationTracks,
		FUsdSchemaTranslationContext& TranslationContext
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ImportActor);

		FScopedSlowTask SlowTask(3, LOCTEXT("ImportActor", "Importing Actor"));

		bool bExpandChilren = true;
		USceneComponent* Component = nullptr;

		// Spawn components and/or actors for this prim
		UE::FSdfPath PrimPath = Prim.GetPrimPath();
		if (TSharedPtr<FUsdSchemaTranslator> SchemaTranslator = FUsdSchemaTranslatorRegistry::Get()
																	.CreateTranslatorForSchema(TranslationContext.AsShared(), UE::FUsdTyped(Prim)))
		{
			Component = SchemaTranslator->CreateComponents();

			bExpandChilren = !SchemaTranslator->CollapsesChildren(ECollapsingType::Components);
		}

		// Refresh our prim reference because FUsdGeomXformableTranslator::CreateComponents can potentially flip through
		// LODs when setting material overrides
		Prim = ImportContext.Stage.GetPrimAtPath(PrimPath);

		// In USD if a parent prim has animated visibility, it will affect the entire subtree. In UE this doesn't
		// happen by default, so if our visibility is animated (or if we're forced to animate visibility from a parent prim),
		// then we must ensure that we generate visibility tracks for our child prims that don't collapse either, so that
		// they get the chance to bake their "computed visibilities" and e.g. be hidden whenever a parent prim goes invisible
#if USE_USD_SDK
		const bool bAnimatedVisibility = bForceVisibilityAnimationTracks || UsdUtils::HasAnimatedVisibility(Prim);
#else
		const bool bAnimatedVisibility = bForceVisibilityAnimationTracks;
#endif

		SlowTask.EnterProgressFrame();
		// Recurse to children
		if (bExpandChilren)
		{
			USceneComponent* ContextParentComponent = Component ? Component : TranslationContext.ParentComponent;
			TGuardValue<USceneComponent*> ParentComponentGuard(TranslationContext.ParentComponent, ContextParentComponent);

			const bool bTraverseInstanceProxies = true;
			for (UE::FUsdPrim Child : Prim.GetFilteredChildren(bTraverseInstanceProxies))
			{
				ImportActor(ImportContext, Child, bAnimatedVisibility, TranslationContext);
			}
		}
		SlowTask.EnterProgressFrame();
		if (Component)
		{
			// LightComponents specifically need this to setup static lighting
			Component->PostEditChange();

			if (!Component->IsRegistered())
			{
				Component->RegisterComponent();
			}

#if USE_USD_SDK
			if (bAnimatedVisibility || UsdUtils::IsAnimated(Prim) || Prim.IsA(TEXT("SpatialAudio")))
			{
				const TOptional<bool> HasAnimatedBounds = {};
				ImportAnimation(ImportContext, Prim, bAnimatedVisibility, HasAnimatedBounds, Component);
			}
			else if (EUsdDrawMode DrawMode = UsdUtils::GetAppliedDrawMode(Prim); DrawMode != EUsdDrawMode::Default)
			{
				if (UsdUtils::HasAnimatedBounds(
						Prim,
						ImportContext.BBoxCache->GetIncludedPurposes(),
						ImportContext.BBoxCache->GetUseExtentsHint(),
						ImportContext.BBoxCache->GetIgnoreVisibility()
					))
				{
					const TOptional<bool> HasAnimatedBounds = true;
					ImportAnimation(ImportContext, Prim, bAnimatedVisibility, HasAnimatedBounds, Component);
				}
			}
#endif	  // USE_USD_SDK
		}
		SlowTask.EnterProgressFrame();
	}

	void ImportActors(FUsdStageImportContext& ImportContext, FUsdSchemaTranslationContext& TranslationContext)
	{
		if (!ImportContext.ImportOptions->bImportActors)
		{
			return;
		}

		const bool bForceVisibilityAnimationTracks = false;
		UE::FUsdPrim RootPrim = ImportContext.Stage.GetPseudoRoot();
		ImportActor(ImportContext, RootPrim, bForceVisibilityAnimationTracks, TranslationContext);
	}

	FString GetUserFriendlyName(UObject* Asset, TSet<FString>& UniqueAssetNames)
	{
		if (!Asset)
		{
			return {};
		}

		FString AssetName = Asset->GetFName().GetPlainNameString();

		FString PrimPath;
		if (IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(Asset))
		{
			if (UUsdAssetUserData* UserData = UserDataInterface->GetAssetUserData<UUsdAssetUserData>())
			{
				if (!UserData->PrimPaths.IsEmpty())
				{
					PrimPath = UserData->PrimPaths[0];
				}
			}
		}

		if (UStaticMesh* Mesh = Cast<UStaticMesh>(Asset))
		{
			// If we have multiple LODs here we must have parsed the LOD variant set pattern. If our prims were named
			// with the LOD pattern, go from e.g. '/Root/MyMesh/LOD0' to '/Root/MyMesh', or else every single LOD mesh
			// will be named "SM_LOD0_X". We'll actually check though because if the user set a custom name for their
			// prim other than LOD0 then we'll keep that
			if (Mesh->GetNumLODs() > 1)
			{
				FString PrimName = FPaths::GetBaseFilename(PrimPath);
				if (PrimName.RemoveFromStart(TEXT("LOD"), ESearchCase::CaseSensitive))
				{
					if (PrimName.IsNumeric())
					{
						AssetName = FPaths::GetBaseFilename(FPaths::GetPath(PrimPath));
					}
				}
			}
		}

		AssetName = UsdUnreal::ObjectUtils::GetPrefixedAssetName(AssetName, Asset->GetClass());

		// We don't care if our assets overwrite something in the final destination package (that conflict will be
		// handled according to EReplaceAssetPolicy). But we do want these assets to have unique names amongst themselves
		// or else they will overwrite each other when publishing
		AssetName = UsdUnreal::ObjectUtils::GetUniqueName(UsdUnreal::ObjectUtils::SanitizeObjectName(AssetName), UniqueAssetNames);
		UniqueAssetNames.Add(AssetName);

		return AssetName;
	}

	void UpdateAssetImportData(UObject* Asset, const FString& MainFilePath, UUsdStageImportOptions* ImportOptions)
	{
		if (!Asset)
		{
			return;
		}

		UUsdAssetImportData* ImportData = UsdUnreal::ObjectUtils::GetAssetImportData(Asset);
		if (!ImportData)
		{
			ImportData = NewObject<UUsdAssetImportData>(Asset);
			UsdUnreal::ObjectUtils::SetAssetImportData(Asset, ImportData);
		}

		if (ImportData)
		{
			// Don't force update these assets as they will already have this preset to their actual source file path
			if (!Asset->IsA<UTexture>() && !Asset->IsA<USparseVolumeTexture>() && !Asset->IsA<USoundBase>())
			{
				ImportData->UpdateFilenameOnly(MainFilePath);
			}

			ImportData->ImportOptions = ImportOptions;
		}
	}

	void UpdateAssetUserData(const TSet<UObject*>& UsedAssetsAndDependencies, const FString& MainFilePath, UUsdStageImportOptions* ImportOptions)
	{
		for (UObject* Asset : UsedAssetsAndDependencies)
		{
			UpdateAssetImportData(Asset, MainFilePath, ImportOptions);
		}
	}

	// Moves Asset from its folder to the package at DestFullContentPath and sets up its flags.
	// Depending on ReplacePolicy it may replace the existing actor (if it finds one) or just abort
	UObject* PublishAsset(
		FUsdStageImportContext& ImportContext,
		UObject* Asset,
		const FString& DestFullPackagePath,
		TMap<UObject*, UObject*>& ObjectsToRemap,
		TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap,
		TSet<UObject*>& OutAssetsToFinalize
	)
	{
		if (!Asset)
		{
			return nullptr;
		}

		EReplaceAssetPolicy ReplacePolicy = ImportContext.ImportOptions->ExistingAssetPolicy;
		FString TargetPackagePath = UPackageTools::SanitizePackageName(DestFullPackagePath);
		FString TargetAssetName = FPaths::GetBaseFilename(TargetPackagePath);
		UObject* ExistingAsset = nullptr;
		UPackage* ExistingPackage = nullptr;

		if (ReplacePolicy == EReplaceAssetPolicy::Append)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			AssetToolsModule.Get().CreateUniqueAssetName(TargetPackagePath, TEXT(""), TargetPackagePath, TargetAssetName);
		}
		else
		{
			// See if we have an existing asset/package
			ExistingPackage = FindPackage(nullptr, *TargetPackagePath);
			if (!ExistingPackage && FPackageName::DoesPackageExist(TargetPackagePath))
			{
				ExistingPackage = LoadPackage(nullptr, *TargetPackagePath, LOAD_None);
			}
			if (ExistingPackage)
			{
				FSoftObjectPath ObjectPath(TargetPackagePath);
				ExistingAsset = static_cast<UObject*>(FindObjectWithOuter(ExistingPackage, Asset->GetClass()));
				if (!ExistingAsset)
				{
					ExistingAsset = ObjectPath.TryLoad();
				}
			}

			// If we're ignoring assets that conflict, just abort now
			if (ExistingAsset != nullptr && ExistingAsset != Asset && ReplacePolicy == EReplaceAssetPolicy::Ignore)
			{
				// Redirect any users of our new transient asset to the old, existing asset
				ObjectsToRemap.Add(Asset, ExistingAsset);
				SoftObjectsToRemap.Add(Asset, ExistingAsset);
				return nullptr;
			}
		}

		// Close editors opened on existing asset if applicable
		bool bAssetWasOpen = false;
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (ExistingAsset && AssetEditorSubsystem->FindEditorForAsset(ExistingAsset, false) != nullptr && !IsEngineExitRequested())
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(ExistingAsset);
			bAssetWasOpen = true;
		}

		// If the existing asset is an audio file, make sure it's not playing before we stomp it
		if (USoundWave* Sound = Cast<USoundWave>(ExistingAsset))
		{
			if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
			{
				AudioDeviceManager->StopSoundsUsingResource(Sound);
			}
			Sound->FreeResources();
		}

		UPackage* Package = ExistingPackage ? ExistingPackage : CreatePackage(*TargetPackagePath);
		if (!Package)
		{
			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("PublishFailure", "Failed to get destination package at '{0}' for imported asset '{1}'!"),
				FText::FromString(TargetPackagePath),
				FText::FromName(Asset->GetFName())
			));
			return nullptr;
		}
		Package->FullyLoad();

		FSoftObjectPath OldPath = Asset;

		// If the asset cache is targetting the transient package (by being within the transient package and having its
		// AssetDirectory point at it) and the asset is transient, it means the asset was generated for this import only
		// (even if the asset cache is otherwise persistent), so we can just rename it.
		// Note that the asset cache may also store transient MIDs when opening the stage at runtime, but we don't import at
		// runtime anyway, so we're all good!
		//
		// We also duplicate LevelSequences when importing because they are technically owned by the LevelSequenceHelper, and
		// we can't just take them
		const bool bMustDuplicate = (ImportContext.UsdAssetCache->AssetDirectory.Path != GetTransientPackage()->GetPathName())
									|| (Asset->GetOutermost() != GetTransientPackage()) || Asset->IsA<ULevelSequence>();

		// Strategy copied from FDatasmithImporterImpl::PublicizeAsset
		// Replace existing asset (reimport or conflict) with new asset
		UObject* MovedAsset = ExistingAsset;
		if (bMustDuplicate || (ExistingAsset != nullptr && ExistingAsset != Asset && ReplacePolicy == EReplaceAssetPolicy::Replace))
		{
			// We have to rename the existing asset away, because some objects manage subobjects (like
			// UMaterialInterface and its EditorOnlyData), and try to rename them whenever they're duplicated/renamed.
			// That can lead to issues if the target names conflict with the names of ExistingAsset's subobjects
			if (ExistingAsset)
			{
				FName UniqueName = MakeUniqueObjectName(GetTransientPackage(), ExistingAsset->GetClass(), ExistingAsset->GetFName());
				ExistingAsset
					->Rename(*UniqueName.ToString(), GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
			}

			if (bMustDuplicate)
			{
				MovedAsset = DuplicateObject<UObject>(Asset, Package, *TargetAssetName);
			}
			// We used to always duplicate here, but now since we're renaming ExistingAsset away, most of the time we can just
			// rename our asset to its target package too
			else
			{
				ensure(Asset->Rename(*TargetAssetName, Package, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch));
				MovedAsset = Asset;
			}

			OutAssetsToFinalize.Add(MovedAsset);
			if (ExistingAsset)
			{
				OutAssetsToFinalize.Add(ExistingAsset);

				// If we're replacing ExistingAsset, we must update all references we can find from ExistingAsset to the new
				// asset, otherwise they'll be left pointing at transient or GC'd/stomped assets
				SoftObjectsToRemap.Add(ExistingAsset, MovedAsset);
				ObjectsToRemap.Add(ExistingAsset, MovedAsset);
			}
		}
		else
		{
			// We can't dirty the package here. Read the comment around MarkPackageDirty, below
			ensure(Asset->Rename(*TargetAssetName, Package, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch));
			MovedAsset = Asset;
		}

		SoftObjectsToRemap.Add(OldPath, MovedAsset);
		if (MovedAsset != Asset)
		{
			ObjectsToRemap.Add(Asset, MovedAsset);
		}

		// Important as some assets (e.g. material instances) are created with no flags, and LevelSequences are created
		// without RF_Standalone by the USDLevelSequenceHelper
		MovedAsset->SetFlags(ImportContext.ImportObjectFlags | EObjectFlags::RF_Public | EObjectFlags::RF_Standalone);
		MovedAsset->ClearFlags(EObjectFlags::RF_Transient);

		// Some subobjects like UStaticMesh::HiResSourceModel->StaticMeshDescriptionBulkData can't be left transient, or else they won't serialize
		// their data. We probably never want to make them public or standalone if they aren't already though
		TArray<UObject*> Subobjects;
		MovedAsset->GetDefaultSubobjects(Subobjects);
		if (UMaterialInterface* Material = Cast<UMaterialInterface>(MovedAsset))
		{
			// Materials in particular have EditorOnlyData which behaves like a default subobject but kind of isn't flagged as one...
			// Since we just cleared the RF_Transient flag, and RF_Transient belongs to RF_PropagateToSubObjects, let's also clear it
			// from its subobjects
			if (UMaterialInterfaceEditorOnlyData* EditorOnlyData = Material->GetEditorOnlyData())
			{
				Subobjects.Add(EditorOnlyData);
			}
		}
		for (UObject* Subobject : Subobjects)
		{
			Subobject->ClearFlags(EObjectFlags::RF_Transient & RF_PropagateToSubObjects);
		}

		// We need to make sure that "dirtying the final package" is not added to the transaction, because if we undo this transaction
		// the assets should remain on their final destination, so we still want the packages to remain marked as dirty (as they're really not on the
		// disk yet). If we didn't suppress, the package would become transactional by this call. When undoing, the assets would still remain on the
		// final package, but the "dirtying" would be undone, so the engine would think the assets weren't dirty (i.e. were already saved), which is
		// not true
		{
			TGuardValue<ITransaction*> SuppressTransaction{GUndo, nullptr};
			Package->MarkPackageDirty();
		}

		// If we swapped the asset inside of a package we need to invalidate the linker and disconnect
		// the package from the file on disk. Otherwise some code paths that go through it to find the
		// package's asset will find our old asset in Engine/Transient instead of the new reimported asset
		// (Repro for this: Import static mesh and place it into the level -> Save everything -> Reimport
		// static mesh -> Reload level -> Without this code, the reloaded level will point at the old static mesh)
		if (ExistingPackage)
		{
			ResetLoaders(ExistingPackage);
		}

		// Reopen asset editor if we were editing the asset
		if (bAssetWasOpen)
		{
			AssetEditorSubsystem->OpenEditorForAsset(MovedAsset);
		}

		// We want the asset cache to stop tracking anything that has been moved to the import package, but it's
		// important to let it track the original assets, so that it can delete those once the import is complete (if it's set up
		// to do that)
		if (MovedAsset->GetPathName().StartsWith(ImportContext.PackagePath))
		{
			ImportContext.UsdAssetCache->StopTrackingAsset(ImportContext.UsdAssetCache->GetHashForAsset(MovedAsset));
		}

		ImportContext.ImportedAsset = MovedAsset;
		ImportContext.ImportedAssets.Add(MovedAsset);

		return MovedAsset;
	}

	// Our asset could have come from an asset cache, where it could have been reused across different stages. After import, we only
	// want to keep the metadata fields that came from the actually imported stage though, so here we prune the others
	void PrunePrimMetadata(UObject* Asset, const UE::FUsdStage& Stage)
	{
		IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(Asset);
		if (!UserDataInterface)
		{
			return;
		}

		UUsdAssetUserData* UserData = UserDataInterface->GetAssetUserData<UUsdAssetUserData>();
		if (!UserData)
		{
			return;
		}

		FString ImportedStageIdentifier = Stage.GetRootLayer().GetIdentifier();

		for (TMap<FString, FUsdCombinedPrimMetadata>::TIterator Iter = UserData->StageIdentifierToMetadata.CreateIterator(); Iter; ++Iter)
		{
			if (Iter->Key != ImportedStageIdentifier)
			{
				Iter.RemoveCurrent();
			}
		}
	}

	void DiscardEmptyLevelSequences(TArray<UObject*>& InOutLevelSequences)
	{
		// Maps from a visited LevelSequence to whether it has useful data or not
		TMap<UObject*, bool> SequenceHasData;
		SequenceHasData.Reserve(InOutLevelSequences.Num());

		TFunction<bool(UMovieSceneSequence*)> TraverseSequence = nullptr;
		TraverseSequence = [&SequenceHasData, &TraverseSequence](UMovieSceneSequence* Sequence) -> bool
		{
			if (!Sequence)
			{
				return false;
			}

			if (bool* FoundResult = SequenceHasData.Find(Sequence))
			{
				return *FoundResult;
			}

			bool bHasData = false;

			// Preemptively add an entry to the map in order to prevent ourselves from getting stuck traversing
			// loops, if those are possible
			SequenceHasData.Add(Sequence, bHasData);

			if (UMovieScene* MovieScene = Sequence->GetMovieScene())
			{
				// When opening the stage we may mark some non-local-layer-stack sequences as read-only.
				// There's no point to doing this when importing, however. It's easier to strip this here than to 
				// prevent the LevelSequences from being made read-only when first importing
				const bool bReadOnly = false;
				MovieScene->SetReadOnly(bReadOnly);

				bool bHasSubsectionWithData = false;
				if (UMovieSceneSubTrack* SubTrack = MovieScene->FindTrack<UMovieSceneSubTrack>())
				{
					TArray<UMovieSceneSection*> Sections = SubTrack->GetAllSections();
					for (UMovieSceneSection* Section : Sections)
					{
						UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
						if (!SubSection)
						{
							// This should never happen, but if it does we have no idea what this is, so lets not remove it
							bHasSubsectionWithData = true;
							continue;
						}

						UMovieSceneSequence* Dependency = SubSection->GetSequence();
						const bool bDependencyHasData = TraverseSequence(Dependency);
						if (bDependencyHasData)
						{
							bHasSubsectionWithData = true;
							continue;
						}
						else
						{
							SubSection->SetSequence(nullptr);
							SubTrack->RemoveSection(*Section);
						}
					}

					if (SubTrack->GetAllSections().Num() == 0)
					{
						MovieScene->RemoveTrack(*SubTrack);
					}
				}

				// TODO: Remove empty bindings too?
				bool bHasBindingsWithTracks = false;
				TArray<FMovieSceneBinding> Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
				for (FMovieSceneBinding& Binding : Bindings)
				{
					if (Binding.GetTracks().Num() != 0)
					{
						bHasBindingsWithTracks = true;
						break;
					}
				}

				bHasData = bHasSubsectionWithData || bHasBindingsWithTracks;
			}

			SequenceHasData.Add(Sequence, bHasData);
			return bHasData;
		};

		for (UObject* Object : InOutLevelSequences)
		{
			TraverseSequence(Cast<ULevelSequence>(Object));
		}

		InOutLevelSequences.RemoveAll(
			[&SequenceHasData](UObject* Sequence)
			{
				// FindRef returns default bool (false) if Sequence is not in the map
				return !SequenceHasData.FindRef(Sequence);
			}
		);
	}

	// Move imported assets from transient folder to their final package, updating AssetCache to point to the moved assets
	void PublishAssets(
		FUsdStageImportContext& ImportContext,
		const TSet<UObject*>& AssetsToPublish,
		TMap<UObject*, UObject*>& ObjectsToRemap,
		TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap,
		TSet<UObject*>& OutPublishedAssets,
		TSet<UObject*>& OutAssetsToFinalize
	)
	{
		TArray<UObject*> Textures;
		TArray<UObject*> Materials;
		TArray<UObject*> StaticMeshes;
		TArray<UObject*> PhysicsAssets;
		TArray<UObject*> Skeletons;
		TArray<UObject*> SkeletalMeshes;
		TArray<UObject*> AnimSequences;
		TArray<UObject*> AnimBlueprints;
		TArray<UObject*> LevelSequences;
		TArray<UObject*> GeometryCaches;
		TArray<UObject*> Grooms;
		TArray<UObject*> GroomCaches;
		TArray<UObject*> GroomBindings;
		TArray<UObject*> SparseVolumeTextures;
		TArray<UObject*> Sounds;

		TSet<FString> UniqueAssetNames;
		TMap<UObject*, FString> AssetToContentFolder;

		for (UObject* Asset : AssetsToPublish)
		{
			if (!Asset)
			{
				continue;
			}

			const static FString TexturesFolder = TEXT("Textures");
			const static FString MaterialsFolder = TEXT("Materials");
			const static FString StaticMeshesFolder = TEXT("StaticMeshes");
			const static FString SkeletalMeshesFolder = TEXT("SkeletalMeshes");
			const static FString LevelSequencesFolder = TEXT("LevelSequences");
			const static FString GeometryCachesFolder = TEXT("GeometryCaches");
			const static FString GroomsFolder = TEXT("Grooms");
			const static FString SoundsFolder = TEXT("Sounds");

			const FString* AssetTypeFolderPtr = nullptr;
			if (Asset->IsA(UMaterialInterface::StaticClass()))
			{
				AssetTypeFolderPtr = &MaterialsFolder;
				Materials.Add(Asset);
			}
			else if (Asset->IsA(UStaticMesh::StaticClass()))
			{
				AssetTypeFolderPtr = &StaticMeshesFolder;
				StaticMeshes.Add(Asset);
			}
			else if (Asset->IsA(UTexture::StaticClass()))
			{
				AssetTypeFolderPtr = &TexturesFolder;
				Textures.Add(Asset);
			}
			else if (Asset->IsA(USkeletalMesh::StaticClass()))
			{
				AssetTypeFolderPtr = &SkeletalMeshesFolder;
				SkeletalMeshes.Add(Asset);
			}
			else if (Asset->IsA(USkeleton::StaticClass()))
			{
				AssetTypeFolderPtr = &SkeletalMeshesFolder;
				Skeletons.Add(Asset);
			}
			else if (Asset->IsA(UPhysicsAsset::StaticClass()))
			{
				AssetTypeFolderPtr = &SkeletalMeshesFolder;
				PhysicsAssets.Add(Asset);
			}
			else if (Asset->IsA(UAnimSequence::StaticClass()))
			{
				AssetTypeFolderPtr = &SkeletalMeshesFolder;
				AnimSequences.Add(Asset);
			}
			else if (Asset->IsA(UAnimBlueprint::StaticClass()))
			{
				AssetTypeFolderPtr = &SkeletalMeshesFolder;
				AnimBlueprints.Add(Asset);
			}
			else if (Asset->IsA(ULevelSequence::StaticClass()))
			{
				AssetTypeFolderPtr = &LevelSequencesFolder;
				LevelSequences.Add(Asset);
			}
			else if (Asset->IsA(UGeometryCache::StaticClass()))
			{
				AssetTypeFolderPtr = &GeometryCachesFolder;
				GeometryCaches.Add(Asset);
			}
			else if (Asset->IsA(UGroomAsset::StaticClass()))
			{
				AssetTypeFolderPtr = &GroomsFolder;
				Grooms.Add(Asset);
			}
			else if (Asset->IsA(UGroomCache::StaticClass()))
			{
				AssetTypeFolderPtr = &GroomsFolder;
				GroomCaches.Add(Asset);
			}
			else if (UGroomBindingAsset* GroomBinding = Cast<UGroomBindingAsset>(Asset))
			{
				AssetTypeFolderPtr = &GroomsFolder;
				GroomBindings.Add(Asset);
			}
			else if (Asset->IsA(USparseVolumeTexture::StaticClass()))
			{
				AssetTypeFolderPtr = &TexturesFolder;
				SparseVolumeTextures.Add(Asset);
			}
			else if (Asset->IsA(USoundBase::StaticClass()))
			{
				AssetTypeFolderPtr = &SoundsFolder;
				Sounds.Add(Asset);
			}
			else
			{
				// We don't know what to do with this asset
				ensure(false);
				continue;
			}

			if (ImportContext.ImportOptions->bPrimPathFolderStructure)
			{
				if (IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(Asset))
				{
					if (UUsdAssetUserData* UserData = UserDataInterface->GetAssetUserData<UUsdAssetUserData>())
					{
						// For skeletal stuff, the primpaths point to the SkelRoot, so it is useful to place the assets in there,
						// as we'll always have at least the skeletal mesh and the skeleton
						if (AssetTypeFolderPtr == &SkeletalMeshesFolder)
						{
							AssetToContentFolder.Add(Asset, UserData->PrimPaths.IsEmpty() ? FString{} : UserData->PrimPaths[0]);
						}
						else
						{
							AssetToContentFolder.Add(Asset, FPaths::GetPath(UserData->PrimPaths.IsEmpty() ? FString{} : UserData->PrimPaths[0]));
						}
					}
				}
			}
			else if (AssetTypeFolderPtr)
			{
				AssetToContentFolder.Add(Asset, *AssetTypeFolderPtr);
			}
		}

		TFunction<void(const TArray<UObject*>&)> PublishAssetType = [&AssetToContentFolder,
																	 &UniqueAssetNames,
																	 &ImportContext,
																	 &ObjectsToRemap,
																	 &SoftObjectsToRemap,
																	 &OutAssetsToFinalize,
																	 &OutPublishedAssets	//
		](const TArray<UObject*>& Assets)
		{
			for (UObject* Asset : Assets)
			{
				if (FString* AssetTypeFolder = AssetToContentFolder.Find(Asset))
				{
					FString TargetAssetName = GetUserFriendlyName(Asset, UniqueAssetNames);
					FString DestPackagePath = FPaths::Combine(ImportContext.PackagePath, *AssetTypeFolder, TargetAssetName);
					UObject*
						PublishedAsset = PublishAsset(ImportContext, Asset, DestPackagePath, ObjectsToRemap, SoftObjectsToRemap, OutAssetsToFinalize);

					if (PublishedAsset)
					{
						PrunePrimMetadata(PublishedAsset, ImportContext.Stage);
						OutPublishedAssets.Add(PublishedAsset);
					}
				}
			}
		};

		DiscardEmptyLevelSequences(LevelSequences);

		// We should publish them in the order of dependents before dependencies, because publishing will involve
		// taking these assets from the asset cache, which can only happen in that order. If we don't follow this order
		// we may end up unnecessarily duplicating assets
		PublishAssetType(LevelSequences);
		PublishAssetType(AnimBlueprints);
		PublishAssetType(AnimSequences);
		PublishAssetType(GroomBindings);
		PublishAssetType(GroomCaches);
		PublishAssetType(Grooms);
		PublishAssetType(SkeletalMeshes);
		PublishAssetType(Skeletons);
		PublishAssetType(PhysicsAssets);
		PublishAssetType(StaticMeshes);
		PublishAssetType(GeometryCaches);
		PublishAssetType(Materials);
		PublishAssetType(Textures);
		PublishAssetType(SparseVolumeTextures);
		PublishAssetType(Sounds);
	}

	void ResolveComponentConflict(
		USceneComponent* NewRoot,
		USceneComponent* ExistingRoot,
		EReplaceActorPolicy ReplacePolicy,
		TMap<UObject*, UObject*>& ObjectsToRemap,
		TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap
	)
	{
		if (!NewRoot || !ExistingRoot || ReplacePolicy == EReplaceActorPolicy::Append)
		{
			return;
		}

		ObjectsToRemap.Add(ExistingRoot, NewRoot);
		SoftObjectsToRemap.Add(ExistingRoot, NewRoot);

		TArray<USceneComponent*> ExistingComponents = ExistingRoot->GetAttachChildren();
		TArray<USceneComponent*> NewComponents = NewRoot->GetAttachChildren();

		AActor* NewActor = NewRoot->GetOwner();
		AActor* ExistingActor = ExistingRoot->GetOwner();

		const auto CatalogByName = [](AActor* Owner, const TArray<USceneComponent*>& Components, TMap<FString, USceneComponent*>& Map)
		{
			for (USceneComponent* Component : Components)
			{
				if (Component->GetOwner() == Owner)
				{
					Map.Add(Component->GetName(), Component);
				}
			}
		};

		TMap<FString, USceneComponent*> ExistingComponentsByName;
		TMap<FString, USceneComponent*> NewComponentsByName;
		CatalogByName(ExistingActor, ExistingComponents, ExistingComponentsByName);
		CatalogByName(NewActor, NewComponents, NewComponentsByName);

		// Handle conflict between new and existing hierarchies
		for (const TPair<FString, USceneComponent*>& NewPair : NewComponentsByName)
		{
			const FString& Name = NewPair.Key;
			USceneComponent* NewComponent = NewPair.Value;

			if (USceneComponent** FoundExistingComponent = ExistingComponentsByName.Find(Name))
			{
				bool bRecurse = false;

				switch (ReplacePolicy)
				{
					case EReplaceActorPolicy::UpdateTransform:
						(*FoundExistingComponent)->SetRelativeTransform(NewComponent->GetRelativeTransform());
						(*FoundExistingComponent)->AttachToComponent(NewRoot, FAttachmentTransformRules::KeepRelativeTransform);
						bRecurse = true;
						break;
					case EReplaceActorPolicy::Ignore:
						// Note how we're iterating the new hierarchy here, so "ignore" means "keep the existing one"
						NewComponent->DestroyComponent(false);
						(*FoundExistingComponent)->AttachToComponent(NewRoot, FAttachmentTransformRules::KeepRelativeTransform);
						bRecurse = false;
						break;
					case EReplaceActorPolicy::Replace:
					default:
						// Keep NewChild completely, but recurse to replace components and children
						bRecurse = true;
						break;
				}

				if (bRecurse)
				{
					ResolveComponentConflict(NewComponent, *FoundExistingComponent, ReplacePolicy, ObjectsToRemap, SoftObjectsToRemap);
				}
			}
		}

		// Move child components from the existing hierarchy that don't conflict with anything in the new hierarchy,
		// as the new hierarchy is the one that will remain. Do these later so that we don't recurse into them
		for (const TPair<FString, USceneComponent*>& ExistingPair : ExistingComponentsByName)
		{
			const FString& Name = ExistingPair.Key;
			USceneComponent* ExistingComponent = ExistingPair.Value;

			USceneComponent** FoundNewComponent = NewComponentsByName.Find(Name);
			if (!FoundNewComponent)
			{
				ExistingComponent->AttachToComponent(NewRoot, FAttachmentTransformRules::KeepRelativeTransform);
			}
		}
	}

	void RecursiveDestroyActor(AActor* Actor)
	{
		if (!Actor)
		{
			return;
		}

		const bool bResetArray = false;
		TArray<AActor*> Children;
		Actor->GetAttachedActors(Children, bResetArray);

		for (AActor* Child : Children)
		{
			RecursiveDestroyActor(Child);
		}

		Actor->GetWorld()->DestroyActor(Actor);
	}

	void ResolveActorConflict(
		AActor* NewActor,
		AActor* ExistingActor,
		EReplaceActorPolicy ReplacePolicy,
		TMap<UObject*, UObject*>& ObjectsToRemap,
		TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap
	)
	{
		if (!NewActor || !ExistingActor || ReplacePolicy == EReplaceActorPolicy::Append)
		{
			return;
		}

		ObjectsToRemap.Add(ExistingActor, NewActor);
		SoftObjectsToRemap.Add(ExistingActor, NewActor);

		// Collect new and existing actors by label
		const bool bResetArray = false;
		TArray<AActor*> ExistingChildren;
		TArray<AActor*> NewChildren;
		ExistingActor->GetAttachedActors(ExistingChildren, bResetArray);
		NewActor->GetAttachedActors(NewChildren, bResetArray);
		const auto CatalogByLabel = [](const TArray<AActor*>& Actors, TMap<FString, AActor*>& Map)
		{
			for (AActor* Actor : Actors)
			{
				Map.Add(Actor->GetActorLabel(), Actor);
			}
		};
		TMap<FString, AActor*> ExistingChildrenByLabel;
		TMap<FString, AActor*> NewChildrenByLabel;
		CatalogByLabel(ExistingChildren, ExistingChildrenByLabel);
		CatalogByLabel(NewChildren, NewChildrenByLabel);

		// Handle conflicts between new and existing actor hierarchies
		for (const TPair<FString, AActor*>& NewPair : NewChildrenByLabel)
		{
			const FString& Label = NewPair.Key;
			AActor* NewChild = NewPair.Value;

			// There's a conflict
			if (AActor** ExistingChild = ExistingChildrenByLabel.Find(Label))
			{
				bool bRecurse = false;

				switch (ReplacePolicy)
				{
					case EReplaceActorPolicy::UpdateTransform:
						(*ExistingChild)->GetRootComponent()->SetRelativeTransform(NewChild->GetRootComponent()->GetRelativeTransform());
						GEditor->ParentActors(NewActor, *ExistingChild, NAME_None);
						bRecurse = true;
						break;
					case EReplaceActorPolicy::Ignore:
						// Note how we're iterating the new hierarchy here, so "ignore" means "keep the existing one"
						RecursiveDestroyActor(NewChild);
						GEditor->ParentActors(NewActor, *ExistingChild, NAME_None);
						bRecurse = false;
						break;
					case EReplaceActorPolicy::Replace:
					default:
						// Keep NewChild, but recurse to replace components and children
						bRecurse = true;
						break;
				}

				if (bRecurse)
				{
					ResolveActorConflict(NewChild, *ExistingChild, ReplacePolicy, ObjectsToRemap, SoftObjectsToRemap);
				}
			}
		}

		// Handle component hierarchy collisions
		USceneComponent* ExistingRoot = ExistingActor->GetRootComponent();
		USceneComponent* NewRoot = NewActor->GetRootComponent();
		ResolveComponentConflict(NewRoot, ExistingRoot, ReplacePolicy, ObjectsToRemap, SoftObjectsToRemap);

		// Move child actors over from existing hierarchy that don't conflict with anything in new hierarchy
		// Do these later so that we don't recurse into them
		for (const TPair<FString, AActor*>& ExistingPair : ExistingChildrenByLabel)
		{
			const FString& Label = ExistingPair.Key;
			AActor* ExistingChild = ExistingPair.Value;

			AActor** NewChild = NewChildrenByLabel.Find(Label);
			if (NewChild == nullptr)
			{
				GEditor->ParentActors(NewActor, ExistingChild, NAME_None);
			}
		}
	}

	void ResolveActorConflicts(
		FUsdStageImportContext& ImportContext,
		AActor* ExistingSceneActor,
		TMap<UObject*, UObject*>& ObjectsToRemap,
		TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap
	)
	{
		if (!ImportContext.ImportOptions->bImportActors)
		{
			return;
		}

		if (!ImportContext.SceneActor)
		{
			USD_LOG_USERERROR(LOCTEXT("NoSceneActor", "Failed to publish actors as there was no scene actor available!"));
			return;
		}

		EReplaceActorPolicy ReplacePolicy = ImportContext.ImportOptions->ExistingActorPolicy;

		// No conflicts, nothing to replace or redirect (even with Append replace mode we don't want to redirect references to the existing items)
		if (!ExistingSceneActor || ReplacePolicy == EReplaceActorPolicy::Append)
		{
			return;
		}

		ResolveActorConflict(ImportContext.SceneActor, ExistingSceneActor, ReplacePolicy, ObjectsToRemap, SoftObjectsToRemap);
	}

	// If we just reimported a static mesh, we use this to remap the material references to the existing materials, as any
	// materials we just reimported will be discarded
	void CopyOriginalMaterialAssignment(FUsdStageImportContext& ImportContext, UObject* ExistingAsset, UObject* NewAsset)
	{
		UStaticMesh* ExistingMesh = Cast<UStaticMesh>(ExistingAsset);
		UStaticMesh* NewMesh = Cast<UStaticMesh>(NewAsset);

		if (ExistingMesh && NewMesh)
		{
			TArray<FStaticMaterial>& NewMaterials = NewMesh->GetStaticMaterials();
			TArray<FStaticMaterial>& ExistingMaterials = ExistingMesh->GetStaticMaterials();
			int32 NumExistingMaterials = ExistingMaterials.Num();
			int32 NumNewMaterials = NewMaterials.Num();

			for (int32 NewMaterialIndex = 0; NewMaterialIndex < NumNewMaterials && NewMaterialIndex < NumExistingMaterials; ++NewMaterialIndex)
			{
				UMaterialInterface* ExistingMaterial = ExistingMesh->GetMaterial(NewMaterialIndex);

				// Can't use SetMaterial as it starts a scoped transaction that would hold on to our transient assets...
				NewMaterials[NewMaterialIndex].MaterialInterface = ExistingMaterial;
			}

			// Clear out any other assignments we may have
			for (int32 Index = NumExistingMaterials; Index < NumNewMaterials; ++Index)
			{
				NewMaterials[Index].MaterialInterface = nullptr;
			}

			return;
		}

		USkeletalMesh* ExistingSkeletalMesh = Cast<USkeletalMesh>(ExistingAsset);
		USkeletalMesh* NewSkeletalMesh = Cast<USkeletalMesh>(NewAsset);
		if (ExistingSkeletalMesh && NewSkeletalMesh)
		{
			NewSkeletalMesh->SetMaterials(ExistingSkeletalMesh->GetMaterials());
			return;
		}
	}

	void CopySkeletonAssignment(FUsdStageImportContext& ImportContext, UObject* ExistingAsset, UObject* NewAsset)
	{
		USkeletalMesh* ExistingSkeletalMesh = Cast<USkeletalMesh>(ExistingAsset);
		USkeletalMesh* NewSkeletalMesh = Cast<USkeletalMesh>(NewAsset);
		if (ExistingSkeletalMesh && NewSkeletalMesh)
		{
			// Never assign a transient skeleton
			if (ExistingSkeletalMesh->GetSkeleton() && ExistingSkeletalMesh->GetSkeleton()->GetOutermost() == GetTransientPackage())
			{
				return;
			}

			// Assign even if ExistingSkeletalMesh has nullptr skeleton because we must be able to cleanup the
			// abandoned Skeleton in the transient package
			NewSkeletalMesh->SetSkeleton(ExistingSkeletalMesh->GetSkeleton());
			NewSkeletalMesh->SetPhysicsAsset(ExistingSkeletalMesh->GetPhysicsAsset());
		}

		UAnimSequence* ExistingAnimSequence = Cast<UAnimSequence>(ExistingAsset);
		UAnimSequence* NewAnimSequence = Cast<UAnimSequence>(NewAsset);
		if (ExistingAnimSequence && NewAnimSequence)
		{
			// Never assign a transient skeleton
			USkeleton* ExistingSkeleton = ExistingAnimSequence->GetSkeleton();
			if (ExistingSkeleton && ExistingSkeleton->GetOutermost() == GetTransientPackage())
			{
				return;
			}

			NewAnimSequence->SetSkeleton(ExistingSkeleton);
		}
	}

	// Adapted from FDatasmithImporterImpl::FixReferencesForObject
	void RemapReferences(
		FUsdStageImportContext& ImportContext,
		const TSet<UObject*>& PublishedObjects,
		const TMap<UObject*, UObject*>& ObjectsToRemap
	)
	{
		if (ObjectsToRemap.Num() == 0)
		{
			return;
		}

		TMap<UObject*, UObject*> ProcessedObjectsToRemap;
		ProcessedObjectsToRemap.Reserve(ObjectsToRemap.Num());
		for (const TPair<UObject*, UObject*>& SourceTarget : ObjectsToRemap)
		{
			// We never want to remap from all instances of nullptr to something else
			if (SourceTarget.Key == nullptr)
			{
				continue;
			}

			ProcessedObjectsToRemap.Add(SourceTarget);
		}

		TSet<UObject*> Referencers = PublishedObjects;

		// Get all in-memory referencers.
		//
		// We need this as we need to remap even references internal to other loaded assets, like a blueprint component's reference
		// or a spawnable reference, etc.
		//
		// We don't need to search inside unloaded assets because those would presumably already find the proper remapped
		// asset via path whenever they are loaded. By the same logic we don't need to do the same for soft object path references,
		// and only really need to update live UObject references
		{
			TSet<UObject*> OldAssets;
			OldAssets.Reserve(ObjectsToRemap.Num());
			for (const TPair<UObject*, UObject*>& Pair : ObjectsToRemap)
			{
				UObject* OldAsset = Pair.Key;
				if (!OldAsset)
				{
					continue;
				}
				OldAssets.Add(OldAsset);
			}

			TArray<UObject*> FoundReferencerObjects = FReferencerFinder::GetAllReferencers(
				OldAssets,
				nullptr,
				EReferencerFinderFlags::SkipInnerReferences
			);

			Referencers.Append(FoundReferencerObjects);
		}

		// Remap references held by assets that were moved directly to the destination package, and won't be in ProcessedObjectsToRemap
		if (GEditor)
		{
			// Remap references to spawned assets/actors from all worlds.
			// Checking EditorPreview worlds also lets us remap references from components inside Blueprint viewports
			const TIndirectArray<FWorldContext> WorldContexts = GEngine->GetWorldContexts();
			for (const FWorldContext& Context : WorldContexts)
			{
				if (Context.WorldType == EWorldType::Editor || Context.WorldType == EWorldType::EditorPreview)
				{
					if (UWorld* World = Context.World())
					{
						Referencers.Add(World);
					}
				}
			}
		}
		for (const TPair<UObject*, UObject*>& Pair : ProcessedObjectsToRemap)
		{
			// Remap internal references between the remapped objects
			Referencers.Add(Pair.Value);
		}

		// Actually fixup all the referencers that we've collected.
		// Note we don't care if transient assets reference each other, as we'll delete them all at once anyway
		for (UObject* Referencer : Referencers)
		{
			// Allow remapping references from transient worlds in particular because that's where the EditorPreview worlds live
			if (!Referencer || (!Referencer->IsA<UWorld>() && Referencer->GetOutermost() == GetTransientPackage()))
			{
				continue;
			}

			constexpr EArchiveReplaceObjectFlags
				ReplaceFlags = (EArchiveReplaceObjectFlags::IgnoreOuterRef | EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
			FArchiveReplaceObjectRef<UObject> ArchiveReplaceObjectRefInner(Referencer, ProcessedObjectsToRemap, ReplaceFlags);
		}
	}

	void PostRemapAssets(const TSet<UObject*>& PublishedObjects)
	{
		for (UObject* PublishedObject : PublishedObjects)
		{
			if (UMaterialInstance* Material = Cast<UMaterialInstance>(PublishedObject))
			{
				// After we remapped all references to the persistent assets we need to call PostEditChange on
				// all materials. This because material instance proxies hold direct references to the UTextures that
				// are used by the material, and those references will be left pointing at the transient assets in
				// case we had to use DuplicateObject to "publish" them to their final locations. Calling
				// PostEditChange rebuilds those proxies from the UMaterialInstance's parameters, which *have* been
				// remapped by our RemapReferences and RemapSoftReferences functions.
				Material->PostEditChange();
			}
			else if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(PublishedObject))
			{
				// Refresh AnimSequences: They contain internal transient ControlRigs that won't be remapped properly via RemapReferences or
				// RemapSoftReferences and may be left trying to point at the original skeletons.
				//
				// Realistically we should probably do this with *every* asset type, as they may all have complex transient references to
				// other assets that don't get picked up by the simple FProperty remapping mentioned above... The properties should be the
				// only references that get serialized however, so if we remap the properties and then PostLoad() we should hopefully be able
				// to achieve a valid state every time, as it should cause these assets to rebuild their transient info from their deserialized
				// (and remapped) properties. It seems that only UAnimSequences are complex enough to require this at this time though
				AnimSequence->PostLoad();
			}
		}
	}

	void Cleanup(AActor* NewSceneActor, AActor* ExistingSceneActor, EReplaceActorPolicy ReplacePolicy, const TSet<UObject*>& AssetsToFinalize)
	{
		if (!NewSceneActor)
		{
			return;
		}

		// By this point all of our actors and components are moved to the new hierarchy, and all references
		// are remapped. So let's clear the replaced existing actors and components
		if (ExistingSceneActor && ExistingSceneActor != NewSceneActor && ReplacePolicy == EReplaceActorPolicy::Replace)
		{
			RecursiveDestroyActor(ExistingSceneActor);
		}

		for (TObjectIterator<UMeshComponent> It; It; ++It)
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(*It))
			{
				if (AssetsToFinalize.Contains(StaticMeshComponent->GetStaticMesh()))
				{
					StaticMeshComponent->FixupOverrideColorsIfNecessary(true);
					StaticMeshComponent->InvalidateLightingCache();
				}
			}
			else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(*It))
			{
				if (AssetsToFinalize.Contains(SkeletalMeshComponent->GetSkeletalMeshAsset()))
				{
					SkeletalMeshComponent->ClearAnimScriptInstance();
				}
			}
		}
	}

	void CloseStageIfNeeded(FUsdStageImportContext& ImportContext)
	{
#if USE_USD_SDK
		// Remove our imported stage from the stage cache if it wasn't in there to begin with
		if (!ImportContext.bStageWasOriginallyOpenInCache && ImportContext.bReadFromStageCache)
		{
			UnrealUSDWrapper::EraseStageFromCache(ImportContext.Stage);
		}

		if (ImportContext.ImportOptions->bOverrideStageOptions)
		{
			UsdUtils::SetUsdStageMetersPerUnit(ImportContext.Stage, ImportContext.OriginalMetersPerUnit);
			UsdUtils::SetUsdStageUpAxis(ImportContext.Stage, ImportContext.OriginalUpAxis);
		}

		// Always discard the context's reference to the stage because it may be a persistent import context (like
		// the non-static data member of UUsdStageImportFactory
		ImportContext.Stage = UE::FUsdStage();
		ImportContext.LevelSequenceHelper.Init(UE::FUsdStage());
#endif	  // #if USE_USD_SDK
	}

	/**
	 * UUsdAssetCache3 can track which assets are requested/added to itself during translation, but it may miss some dependencies
	 * that are only retrieved/added themselves when the original asset is first parsed. This function recursively collects all of those.
	 * Example: An UMaterialInstance is already in the cache, so when translating we just retrieve the existing asset --> The textures that it's using
	 * won't be retrieved or marked as "Used" Example: An USkeletalMesh is already in the cache, so in the same way we would miss its USkeleton,
	 * materials and textures of those materials
	 */
	void CollectUsedAssetDependencies(FUsdStageImportContext& ImportContext, TSet<UObject*>& OutAssetsAndDependencies)
	{
		const int32 ReserveSize = OutAssetsAndDependencies.Num()
								  + (ImportContext.UsdAssetCache ? ImportContext.UsdAssetCache->GetActiveAssets().Num() : 0);

		// We will only emit the level sequences if we have data in the main one.
		// Keep subsequences even if they have no data as the main sequence/other sequences may reference them
		if (ImportContext.ImportOptions->bImportLevelSequences && ImportContext.LevelSequenceHelper.HasData())
		{
			TArray<ULevelSequence*> SubSequences = ImportContext.LevelSequenceHelper.GetSubSequences();
			ULevelSequence* MainSequence = ImportContext.LevelSequenceHelper.GetMainLevelSequence();

			OutAssetsAndDependencies.Reserve(ReserveSize + SubSequences.Num() + 1);
			OutAssetsAndDependencies.Add(MainSequence);
			for (ULevelSequence* SubSequence : SubSequences)
			{
				OutAssetsAndDependencies.Add(SubSequence);
			}
		}
		else
		{
			OutAssetsAndDependencies.Reserve(ReserveSize);
		}

		if (ImportContext.UsdAssetCache)
		{
			const TSet<FSoftObjectPath>& InPrimaryAssets = ImportContext.UsdAssetCache->GetActiveAssets();
			TArray<FSoftObjectPath> AssetQueue = InPrimaryAssets.Array();

			for (int32 AssetIndex = 0; AssetIndex < AssetQueue.Num(); ++AssetIndex)
			{
				UObject* Asset = AssetQueue[AssetIndex].TryLoad();

				// Only add it as a dependency if it's tracked by the asset cache, but still traverse it because
				// we may be in some strange situation where the material shouldn't be in this list, but one of its used textures should
				if (Asset && ImportContext.UsdAssetCache->IsAssetTrackedByCache(Asset->GetPathName()))
				{
					OutAssetsAndDependencies.Add(Asset);
				}

				TSet<UObject*> Dependencies = IUsdClassesModule::GetAssetDependencies(Asset);
				AssetQueue.Append(Dependencies.Array());
			}
		}
	}

	/**
	 * Remaps asset's soft object pointers to point to the post-publish paths of their target assets.
	 * It's important to run this *after* RemapReferences, as we will sometimes rely on those references to find our target assets.
	 */
	void RemapSoftReferences(
		const FUsdStageImportContext& ImportContext,
		const TSet<UObject*>& UsedAssetsAndDependencies,
		const TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap
	)
	{
		TSet<UPackage*> Packages;
		for (UObject* Object : UsedAssetsAndDependencies)
		{
			if (Object)
			{
				Packages.Add(Object->GetOutermost());
			}
		}

		if (AActor* SceneActor = ImportContext.SceneActor)
		{
			Packages.Add(ImportContext.SceneActor->GetWorld()->GetOutermost());
		}

		// In case one our used assets was left on the transient package.
		// We don't care about anything that was left on the transient package, and doing this may actually cause some reference counting issues
		// if we try deleting those assets afterwards
		Packages.Remove(GetTransientPackage());

		// We never want to remap all invalid reference to something. That particularly seems to break LevelSequence bindings somehow
		TMap<FSoftObjectPath, FSoftObjectPath> ProcessedPaths;
		ProcessedPaths.Reserve(SoftObjectsToRemap.Num());
		for (const TPair<FSoftObjectPath, FSoftObjectPath>& SourceTarget : SoftObjectsToRemap)
		{
			if (SourceTarget.Key.IsValid())
			{
				ProcessedPaths.Add(SourceTarget);
			}
		}

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.RenameReferencingSoftObjectPaths(Packages.Array(), ProcessedPaths);

		// Unexpectedly, the RenameReferencingSoftObjectPaths call above will also *globally* register a redirector from
		// source to target paths, that will affect *all other properties in all other objects*! This means that if at any point
		// any other UObject had a FSoftObjectPath property pointing at any of our OriginalPaths, the property would be
		// secretly rewritten when the UObject was serialized!
		//
		// This will cause major havoc on the AssetCache (e.g. two sequential Actions->Import), so we need to remove this global
		// remapping ASAP
		TArray<FSoftObjectPath> OriginalPaths;
		SoftObjectsToRemap.GetKeys(OriginalPaths);
		for (const FSoftObjectPath& OriginalPath : OriginalPaths)
		{
			GRedirectCollector.RemoveAssetPathRedirection(OriginalPath);
		}
	}

	/** After we remapped everything, notify the AssetRegistry that we created some new assets */
	void BroadcastImportEvents(const TSet<UObject*>& UsedAssetsAndDependencies, bool bIsReimport = false)
	{
		for (UObject* Object : UsedAssetsAndDependencies)
		{
			// If it's still on the transient package it means we abandoned this one (maybe we had asset replace policy ignore and hit a conflict)
			if (Object && Object->GetOutermost() != GetTransientPackage())
			{
				FAssetRegistryModule::AssetCreated(Object);

				if (bIsReimport)
				{
					GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetReimport(Object);
				}
				else
				{
					UFactory* Factory = nullptr;
					GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(Factory, Object);
				}
			}
		}
	}

	void SendAnalytics(
		FUsdStageImportContext& ImportContext,
		UObject* Asset,
		const FString& Operation,
		const TSet<UObject*>& ImportedAssets,
		double ElapsedSeconds
	)
	{
#if USE_USD_SDK
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;

			FString EventName = Operation;
			if (Asset)
			{
				const FString ClassName = IUsdClassesModule::GetClassNameForAnalytics(Asset);

				// e.g. "Reimport.StaticMesh"
				EventName = FString::Printf(TEXT("%s.%s"), *EventName, *ClassName);
				EventAttributes.Emplace(TEXT("AssetType"), ClassName);
			}

			if (ImportContext.ImportOptions)
			{
				UsdUtils::AddAnalyticsAttributes(*ImportContext.ImportOptions, EventAttributes);
			}

			IUsdClassesModule::AddAssetCountAttributes(ImportedAssets, EventAttributes);

			FString RootLayerIdentifier = ImportContext.FilePath;
			if (ImportContext.Stage)
			{
				if (RootLayerIdentifier.IsEmpty())
				{
					RootLayerIdentifier = ImportContext.Stage.GetRootLayer().GetIdentifier();
				}
			}

			IUsdClassesModule::SendAnalytics(
				MoveTemp(EventAttributes),
				EventName,
				ImportContext.bIsAutomated,
				ElapsedSeconds,
				UsdUtils::GetUsdStageNumFrames(ImportContext.Stage),
				FPaths::GetExtension(RootLayerIdentifier)
			);
		}
#endif	  // USE_USD_SDK
	}

	// Removes from AssetsToImport assets that are unwanted according to our import options, and adds entries to
	// ObjectsToRemap and SoftObjectsToRemap that remaps them to nullptr.
	// This function is needed because it's not enough to e.g. just prevent new meshes from being imported from
	// UE::USDStageImporter::Private::ImportMeshes, because we may want to reuse meshes we already got from the asset cache.
	// Additionally, we'll want to remap even our components away from pointing to these assets
	void PruneUnwantedAssets(
		FUsdStageImportContext& ImportContext,
		TSet<UObject*>& AssetsToImport,
		TMap<UObject*, UObject*>& ObjectsToRemap,
		TMap<FSoftObjectPath, FSoftObjectPath>& SoftObjectsToRemap
	)
	{
		const bool bImportSkeletalAnimations = ImportContext.ImportOptions->bImportGeometry && ImportContext.ImportOptions->bImportSkeletalAnimations;

		for (TSet<UObject*>::TIterator It(AssetsToImport); It; ++It)
		{
			UObject* Asset = *It;

			if (!Asset)
			{
				It.RemoveCurrent();
				continue;
			}

			// clang-format off
			if ((!ImportContext.ImportOptions->bImportGeometry && (Asset->IsA<UStaticMesh>() || Asset->IsA<USkeletalMesh>() || Asset->IsA<USkeleton>() || Asset->IsA<UPhysicsAsset>() || Asset->IsA<UGeometryCache>())) ||
				(!bImportSkeletalAnimations && (Asset->IsA<UAnimSequence>())) ||
				(!ImportContext.ImportOptions->bImportLevelSequences && (Asset->IsA<ULevelSequence>())) ||
				(!ImportContext.ImportOptions->bImportMaterials && (Asset->IsA<UMaterialInterface>() || Asset->IsA<UTexture>())) ||
				(!ImportContext.ImportOptions->bImportGroomAssets && (Asset->IsA<UGroomAsset>() || Asset->IsA<UGroomCache>() || Asset->IsA<UGroomBindingAsset>())) ||
				(!ImportContext.ImportOptions->bImportSparseVolumeTextures && (Asset->IsA<USparseVolumeTexture>())) ||
				(!ImportContext.ImportOptions->bImportSounds && (Asset->IsA<USoundBase>()))
			)
			{
				ObjectsToRemap.Add(Asset, nullptr);
				SoftObjectsToRemap.Add(Asset, nullptr);
				It.RemoveCurrent();
			}
			// clang-format on
		}
	}

	// We need to recreate the render state for some mesh component types in case we changed the materials that are assigned to them.
	// Also, skeletal mesh components need to be manually ticked, or else they may be showing an animated state of an animation that
	// we chose not to import, and wouldn't update otherwise until manually ticked by the user (or after save/reload), which may look
	// like a bug
	void RefreshComponents(AActor* RootSceneActor, bool bImportAtSpecificTimeCode)
	{
		if (!RootSceneActor)
		{
			return;
		}

		TArray<USceneComponent*> Components;
		const bool bIncludeAllDescendants = true;
		RootSceneActor->GetRootComponent()->GetChildrenComponents(bIncludeAllDescendants, Components);

		for (USceneComponent* Component : Components)
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component))
			{
				if (bImportAtSpecificTimeCode)
				{
					// The asset we return from the import factories may lead to
					// USkeletalMesh::PostEditChangeProperty being called. The FMultiComponentReregisterContext in
					// there will call USkeletalMeshComponent::InitAnim on all components that use the mesh, which will
					// wipe out the Position value on SkeletalMeshComponents. This value describes the current state of
					// animating the AnimSequence according to the import TimeCode, and we need to keep it.
					// If we store the position via this function however, it will store it inside the position within
					// the AnimationData member, so that when USkeletalMeshComponent::InitAnim calls
					// USkeletalMeshComponent::InitializeAnimScriptInstance, it will use that AnimationData to
					// initialize the new AnimScriptInstance and retain our desired Position value.
					// This is also nice because it will serialize this animation position and restore it whenever the
					// component updates, which should provide some persistent to this animation state (which we assume
					// was desirable since the user imported at a particular time)
					const bool bIsLooping = false;
					const bool bIsPlaying = false;
					const float Position = SkeletalMeshComponent->GetPosition();
					SkeletalMeshComponent->OverrideAnimationData(SkeletalMeshComponent->AnimationData.AnimToPlay, bIsLooping, bIsPlaying, Position);

					const bool bForceReinit = true;
					SkeletalMeshComponent->InitAnim(bForceReinit);
				}
				else
				{
					const bool bForceReinit = true;
					SkeletalMeshComponent->InitAnim(bForceReinit);
					SkeletalMeshComponent->TickAnimation(0.f, false);
					SkeletalMeshComponent->RefreshBoneTransforms();
					SkeletalMeshComponent->RefreshFollowerComponents();
					SkeletalMeshComponent->UpdateComponentToWorld();
					SkeletalMeshComponent->FinalizeBoneTransform();
					SkeletalMeshComponent->MarkRenderTransformDirty();
					SkeletalMeshComponent->MarkRenderDynamicDataDirty();
				}

				// It does need us to manually set this to dirty regardless or else it won't update in case we changed material
				// assignments
				SkeletalMeshComponent->MarkRenderStateDirty();
			}
		}
	}
}	 // namespace UE::USDStageImporter::Private

void UUsdStageImporter::ImportFromFile(FUsdStageImportContext& ImportContext)
{
#if USE_USD_SDK
	if (!ImportContext.World)
	{
		USD_LOG_USERERROR(LOCTEXT("NoWorldError", "Failed to import USD Stage because the target UWorld is invalid!"));
		return;
	}

	double StartTime = FPlatformTime::Cycles64();

	// Load some default options in case we don't have an options object, so that we don't have to check for this every time.
	// We could also GetMutableDefault<UUsdStageImportOptions>() instead, but I think in this case it's expected that these
	// should be the default value and not the config
	if (!ImportContext.ImportOptions)
	{
		ImportContext.ImportOptions = NewObject<UUsdStageImportOptions>();
	}

	if (ImportContext.ImportOptions->PrimsToImport.Num() == 0)
	{
		return;
	}

	const bool bNeedsMasking = ImportContext.ImportOptions->PrimsToImport != TArray<FString>{UE::FSdfPath::AbsoluteRootPath().GetString()};

	if (!ImportContext.Stage || bNeedsMasking)
	{
		UE::USDStageImporter::Private::OpenStage(ImportContext, bNeedsMasking);
	}

	if (!ImportContext.Stage)
	{
		USD_LOG_USERERROR(LOCTEXT("NoStageError", "Failed to open the USD Stage!"));
		return;
	}

	UE::USDStageImporter::Private::SetupSceneActor(ImportContext);
	if (!ImportContext.SceneActor && ImportContext.ImportOptions->bImportActors)
	{
		return;
	}

	// Reset the editor selection because the import may destroy actors/components or stomp over existing assets.
	// It's probably too aggressive to do this, but it should be the fastest and safest at least.
	GEditor->ResetAllSelectionSets();

	// Reset the transaction buffer so we can't undo the creation of the assets.
	// We'll also temporarily disable creating new transactions, to prevent code downstream from accidentally creating new ones.
	// This happens for example on USkeleton::AccumulateCurveMetaData, and can lead to thousands of transactions showing up on
	// the editor and a huge performance cost due to serialization spam.
	// See also UEditorEngine::CanTransact.
	GEditor->ResetTransaction(LOCTEXT("ResetBeforeUSDImport", "USD import from file"));
	TStrongObjectPtr<UTransactor> TransactorPin;
	TOptional<TGuardValue<TObjectPtr<class UTransactor>>> TransactionSuppressor;
	if (ensure(!GIsTransacting))
	{
		// Some stuff like the DuplicateObject or FBlueprintCompilationManager::CompileSynchronously inside SkelSkeletonTranslator
		// will trigger GC, which we cannot prevent from here. If that happened when our transactor wasn't being referenced via
		// GEditor->Trans it would actually get collected, so here we prevent that from happening
		TransactorPin.Reset(GEditor->Trans.Get());
		TransactionSuppressor.Emplace(GEditor->Trans, nullptr);
	}

	FUsdDelegates::OnPreUsdImport.Broadcast(ImportContext.FilePath);

	AActor* ExistingSceneActor = UE::USDStageImporter::Private::GetExistingSceneActor(ImportContext);

	UE::USDStageImporter::Private::SetupStageForImport(ImportContext);

	TMap<FSoftObjectPath, FSoftObjectPath> SoftObjectsToRemap;
	TMap<UObject*, UObject*> ObjectsToRemap;
	TSet<UObject*> UsedAssetsAndDependencies;
	TSet<UObject*> PublishedAssets;
	TSet<UObject*> AssetsToFinalize;
	UsdUtils::FBlendShapeMap BlendShapesByPath;

	// Ensure a valid asset cache
	TStrongObjectPtr<UUsdAssetCache3> AssetCachePin;
	if (!ImportContext.UsdAssetCache && ImportContext.ImportOptions->bUseExistingAssetCache)
	{
		// Try using a provided ExistingAssetCache if that wasn't put on our import context already
		ImportContext.UsdAssetCache = Cast<UUsdAssetCache3>(ImportContext.ImportOptions->ExistingAssetCache.TryLoad());
	}
	if (!ImportContext.UsdAssetCache)
	{
		// Get a transient AssetCache.
		// By being in the transient package this cache will also dump its assets in the transient package.
		// We will then take these assets and publish them to the final import folder.
		ImportContext.UsdAssetCache = NewObject<UUsdAssetCache3>();
		AssetCachePin.Reset(ImportContext.UsdAssetCache);
	}
	if (!ensure(ImportContext.UsdAssetCache))
	{
		return;
	}
	// Even though we might be using an external asset cache, temporarily make sure that the asset directory
	// is the transient package, so that any new assets that we create when importing don't end up in the
	// content browser and can instead just be renamed by the stageimporter directly into the import location
	TGuardValue<FString> AssetCacheTransientGuard{ImportContext.UsdAssetCache->AssetDirectory.Path, GetTransientPackage()->GetPathName()};

	UObject* Outer = GetTransientPackage();
	const FName Name = NAME_None;
	const EObjectFlags Flags = EObjectFlags::RF_Transient;
	TStrongObjectPtr<UUsdPrimLinkCache> PrimLinkCache{NewObject<UUsdPrimLinkCache>(Outer, Name, Flags)};
	TStrongObjectPtr<UUsdInfoCache> UsdInfoCache{NewObject<UUsdInfoCache>(Outer, Name, Flags)};

	ImportContext.UsdAssetCache->MarkAssetsAsStale();
	ImportContext.LevelSequenceHelper.SetPrimLinkCache(PrimLinkCache.Get());
	ImportContext.LevelSequenceHelper.Init(ImportContext.Stage);	// Must happen after the context gets an InfoCache!
	ImportContext.LevelSequenceHelper.SetRootMotionHandling(ImportContext.ImportOptions->RootMotionHandling);

	EUsdPurpose PurposesToImport = static_cast<EUsdPurpose>(ImportContext.ImportOptions->PurposesToImport);

	float ImportTime = ImportContext.ImportOptions->bImportAtSpecificTimeCode ? ImportContext.ImportOptions->ImportTimeCode
																			  : static_cast<float>(UsdUtils::GetDefaultTimeCode());

	if (!ImportContext.BBoxCache)
	{
		const bool bUseExtentsHint = true;
		const bool bIgnoreVisibility = false;
		ImportContext.BBoxCache = MakeShared<UE::FUsdGeomBBoxCache>(ImportTime, PurposesToImport, bUseExtentsHint, bIgnoreVisibility);
	}

	TOptional<EUsdPurpose> IncludedPurposesToRevertBBoxCacheTo;
	if (ImportContext.BBoxCache->GetIncludedPurposes() != PurposesToImport)
	{
		IncludedPurposesToRevertBBoxCacheTo = ImportContext.BBoxCache->GetIncludedPurposes();
		ImportContext.BBoxCache->SetIncludedPurposes(PurposesToImport);
	}
	TOptional<float> TimeToRevertBBoxCacheTo;
	if (ImportContext.BBoxCache->GetTime() != ImportTime)
	{
		TimeToRevertBBoxCacheTo = ImportContext.BBoxCache->GetTime();
		ImportContext.BBoxCache->SetTime(ImportTime);
	}
	ImportContext.LevelSequenceHelper.SetBBoxCache(ImportContext.BBoxCache);

	// Shotgun approach to recreate all render states because we may want to reimport/delete/reassign a material/static/skeletalmesh while it is
	// currently being drawn
	FGlobalComponentRecreateRenderStateContext RecreateRenderStateContext;

	TSharedRef<FUsdSchemaTranslationContext> TranslationContext = MakeShared<FUsdSchemaTranslationContext>(
		ImportContext.Stage,
		*ImportContext.UsdAssetCache
	);
	TranslationContext->bIsImporting = true;
	TranslationContext->Level = ImportContext.World->GetCurrentLevel();
	TranslationContext->ObjectFlags = ImportContext.ImportObjectFlags;
	TranslationContext->Time = ImportTime;
	TranslationContext->PurposesToLoad = PurposesToImport;
	TranslationContext->NaniteTriangleThreshold = ImportContext.ImportOptions->NaniteTriangleThreshold;
	TranslationContext->RenderContext = ImportContext.ImportOptions->RenderContextToImport;
	TranslationContext->MaterialPurpose = ImportContext.ImportOptions->MaterialPurpose;
	TranslationContext->RootMotionHandling = ImportContext.ImportOptions->RootMotionHandling;
	TranslationContext->FallbackCollisionType = ImportContext.ImportOptions->FallbackCollisionType;
	TranslationContext->SubdivisionLevel = ImportContext.ImportOptions->SubdivisionLevel;
	TranslationContext->MetadataOptions = ImportContext.ImportOptions->MetadataOptions;
	TranslationContext->ParentComponent = ImportContext.SceneActor ? ImportContext.SceneActor->GetRootComponent() : nullptr;
	TranslationContext->KindsToCollapse = (EUsdDefaultKind)ImportContext.ImportOptions->KindsToCollapse;
	TranslationContext->bUsePrimKindsForCollapsing = ImportContext.ImportOptions->bUsePrimKindsForCollapsing;
	TranslationContext->bMergeIdenticalMaterialSlots = ImportContext.ImportOptions->bMergeIdenticalMaterialSlots;
	TranslationContext->bShareAssetsForIdenticalPrims = ImportContext.ImportOptions->bShareAssetsForIdenticalPrims;
	TranslationContext->bAllowInterpretingLODs = ImportContext.ImportOptions->bInterpretLODs;
	TranslationContext->bAllowParsingSkeletalAnimations = ImportContext.ImportOptions->bImportGeometry
														  && ImportContext.ImportOptions->bImportSkeletalAnimations;
	TranslationContext->bAllowParsingGroomAssets = ImportContext.ImportOptions->bImportGroomAssets;
	TranslationContext->bAllowParsingSparseVolumeTextures = ImportContext.ImportOptions->bImportSparseVolumeTextures;
	TranslationContext->bAllowParsingSounds = ImportContext.ImportOptions->bImportSounds;
	TranslationContext->bTranslateOnlyUsedMaterials = ImportContext.ImportOptions->bImportOnlyUsedMaterials;
	TranslationContext->UsdInfoCache = &UsdInfoCache->GetInner();
	TranslationContext->PrimLinkCache = &PrimLinkCache->GetInner();
	TranslationContext->BBoxCache = ImportContext.BBoxCache;
	TranslationContext->BlendShapesByPath = &BlendShapesByPath;
	TranslationContext->GroomInterpolationSettings = ImportContext.ImportOptions->GroomInterpolationSettings;
	{
		UE::USDStageImporter::Private::CacheCollapsingState(TranslationContext.Get());
		UE::USDStageImporter::Private::ImportMaterials(ImportContext, TranslationContext.Get());
		UE::USDStageImporter::Private::ImportMeshes(ImportContext, TranslationContext.Get());
		UE::USDStageImporter::Private::ImportActors(ImportContext, TranslationContext.Get());
	}
	TranslationContext->CompleteTasks();

	UE::USDStageImporter::Private::CollectUsedAssetDependencies(ImportContext, UsedAssetsAndDependencies);
	UE::USDStageImporter::Private::PruneUnwantedAssets(ImportContext, UsedAssetsAndDependencies, ObjectsToRemap, SoftObjectsToRemap);
	UE::USDStageImporter::Private::UpdateAssetUserData(UsedAssetsAndDependencies, ImportContext.FilePath, ImportContext.ImportOptions);
	UE::USDStageImporter::Private::ResolveActorConflicts(ImportContext, ExistingSceneActor, ObjectsToRemap, SoftObjectsToRemap);
	UE::USDStageImporter::Private::PublishAssets(
		ImportContext,
		UsedAssetsAndDependencies,
		ObjectsToRemap,
		SoftObjectsToRemap,
		PublishedAssets,
		AssetsToFinalize
	);
	UE::USDStageImporter::Private::RemapReferences(ImportContext, PublishedAssets, ObjectsToRemap);
	UE::USDStageImporter::Private::RemapSoftReferences(ImportContext, PublishedAssets, SoftObjectsToRemap);
	UE::USDStageImporter::Private::Cleanup(
		ImportContext.SceneActor,
		ExistingSceneActor,
		ImportContext.ImportOptions->ExistingActorPolicy,
		AssetsToFinalize
	);
	UE::USDStageImporter::Private::PostRemapAssets(PublishedAssets);
	UE::USDStageImporter::Private::BroadcastImportEvents(PublishedAssets);
	UE::USDStageImporter::Private::RefreshComponents(ImportContext.SceneActor, ImportContext.ImportOptions->bImportAtSpecificTimeCode);

	if (IncludedPurposesToRevertBBoxCacheTo.IsSet())
	{
		ImportContext.BBoxCache->SetIncludedPurposes(IncludedPurposesToRevertBBoxCacheTo.GetValue());
	}
	if (TimeToRevertBBoxCacheTo.IsSet())
	{
		ImportContext.BBoxCache->SetTime(TimeToRevertBBoxCacheTo.GetValue());
	}

	FUsdDelegates::OnPostUsdImport.Broadcast(ImportContext.FilePath);

	// Analytics
	{
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		UE::USDStageImporter::Private::SendAnalytics(ImportContext, nullptr, TEXT("Import"), PublishedAssets, ElapsedSeconds);
		USD_LOG_INFO(TEXT("Imported '%s' in %.3f seconds."), *ImportContext.FilePath, ElapsedSeconds);

		UsdUnreal::Analytics::CollectSchemaAnalytics(ImportContext.Stage, TEXT("Import"));
	}

	UE::USDStageImporter::Private::CloseStageIfNeeded(ImportContext);
#endif	  // #if USE_USD_SDK
}

bool UUsdStageImporter::ReimportSingleAsset(
	FUsdStageImportContext& ImportContext,
	UObject* OriginalAsset,
	const FString& OriginalPrimPath,
	UObject*& OutReimportedAsset
)
{
	OutReimportedAsset = nullptr;
	bool bSuccess = false;

#if USE_USD_SDK
	double StartTime = FPlatformTime::Cycles64();

	if (!OriginalAsset)
	{
		return false;
	}

	if (!ImportContext.ImportOptions)
	{
		ImportContext.ImportOptions = NewObject<UUsdStageImportOptions>();
	}

	// TODO: Maybe change this whole reimporting approach to just taking advantage of a population mask instead?
	const bool bNeedsMasking = ImportContext.ImportOptions->PrimsToImport != TArray<FString>{UE::FSdfPath::AbsoluteRootPath().GetString()};

	if (!ImportContext.Stage || bNeedsMasking)
	{
		UE::USDStageImporter::Private::OpenStage(ImportContext, bNeedsMasking);
	}

	if (!ImportContext.Stage)
	{
		USD_LOG_USERERROR(LOCTEXT("NoStageError", "Failed to open the USD Stage!"));
		return bSuccess;
	}

	// Reset the editor selection because the import may destroy actors/components or stomp over existing assets.
	// It's probably too aggressive to do this, but it should be the fastest and safest at least.
	GEditor->ResetAllSelectionSets();

	// Reset the transaction buffer so we can't undo the creation of the assets.
	// We'll also temporarily disable creating new transactions, to prevent code downstream from accidentally creating new ones.
	// This happens for example on USkeleton::AccumulateCurveMetaData, and can lead to thousands of transactions showing up on
	// the editor and a huge performance cost due to serialization spam.
	// See also UEditorEngine::CanTransact.
	GEditor->ResetTransaction(LOCTEXT("ResetBeforeUSDReimport", "USD reimport from file"));
	TStrongObjectPtr<UTransactor> TransactorPin;
	TOptional<TGuardValue<TObjectPtr<class UTransactor>>> TransactionSuppressor;
	if (ensure(!GIsTransacting))
	{
		// Some stuff like the DuplicateObject or FBlueprintCompilationManager::CompileSynchronously inside SkelSkeletonTranslator
		// will trigger GC, which we cannot prevent from here. If that happened when our transactor wasn't being referenced via
		// GEditor->Trans it would actually get collected, so here we prevent that from happening
		TransactorPin.Reset(GEditor->Trans.Get());
		TransactionSuppressor.Emplace(GEditor->Trans, nullptr);
	}

	FUsdDelegates::OnPreUsdImport.Broadcast(ImportContext.FilePath);

	// We still need the scene actor to remap all other users of the mesh to the new reimported one. It's not critical if we fail though,
	// the goal is to just reimport the asset
	UE::USDStageImporter::Private::SetupSceneActor(ImportContext);

	UE::USDStageImporter::Private::SetupStageForImport(ImportContext);

	TMap<FSoftObjectPath, FSoftObjectPath> SoftObjectsToRemap;
	TMap<UObject*, UObject*> ObjectsToRemap;
	TSet<UObject*> AssetsToFinalize;
	UsdUtils::FBlendShapeMap BlendShapesByPath;

	// Ensure a valid asset cache
	TStrongObjectPtr<UUsdAssetCache3> AssetCachePin;
	if (!ImportContext.UsdAssetCache && ImportContext.ImportOptions->bUseExistingAssetCache)
	{
		// Try using a provided ExistingAssetCache if that wasn't put on our import context already
		ImportContext.UsdAssetCache = Cast<UUsdAssetCache3>(ImportContext.ImportOptions->ExistingAssetCache.TryLoad());
	}
	if (!ImportContext.UsdAssetCache)
	{
		// Get a transient AssetCache.
		// By being in the transient package this cache will also dump its assets in the transient package.
		// We will then take these assets and publish them to the final import folder.
		ImportContext.UsdAssetCache = NewObject<UUsdAssetCache3>();
		AssetCachePin.Reset(ImportContext.UsdAssetCache);
	}
	if (!ensure(ImportContext.UsdAssetCache))
	{
		return bSuccess;
	}
	// Even though we might be using an external asset cache, temporarily make sure that the asset directory
	// is the transient package, so that any new assets that we create when importing don't end up in the
	// content browser and can instead just be renamed by the stageimporter directly into the import location
	TGuardValue<FString> AssetCacheTransientGuard{ImportContext.UsdAssetCache->AssetDirectory.Path, GetTransientPackage()->GetPathName()};

	UObject* Outer = GetTransientPackage();
	const FName Name = NAME_None;
	const EObjectFlags Flags = EObjectFlags::RF_Transient;
	TStrongObjectPtr<UUsdPrimLinkCache> PrimLinkCache{NewObject<UUsdPrimLinkCache>(Outer, Name, Flags)};
	TStrongObjectPtr<UUsdInfoCache> UsdInfoCache{NewObject<UUsdInfoCache>(Outer, Name, Flags)};

	ImportContext.UsdAssetCache->MarkAssetsAsStale();
	ImportContext.LevelSequenceHelper.SetPrimLinkCache(PrimLinkCache.Get());
	ImportContext.LevelSequenceHelper.Init(ImportContext.Stage);	// Must happen after the context gets an InfoCache!
	ImportContext.LevelSequenceHelper.SetRootMotionHandling(ImportContext.ImportOptions->RootMotionHandling);

	EUsdPurpose PurposesToImport = static_cast<EUsdPurpose>(ImportContext.ImportOptions->PurposesToImport);

	float ImportTime = ImportContext.ImportOptions->bImportAtSpecificTimeCode ? ImportContext.ImportOptions->ImportTimeCode
																			  : static_cast<float>(UsdUtils::GetDefaultTimeCode());

	if (!ImportContext.BBoxCache)
	{
		const bool bUseExtentsHint = true;
		const bool bIgnoreVisibility = false;
		ImportContext.BBoxCache = MakeShared<UE::FUsdGeomBBoxCache>(ImportTime, PurposesToImport, bUseExtentsHint, bIgnoreVisibility);
	}
	TOptional<EUsdPurpose> IncludedPurposesToRevertTo;
	if (ImportContext.BBoxCache->GetIncludedPurposes() != PurposesToImport)
	{
		IncludedPurposesToRevertTo = ImportContext.BBoxCache->GetIncludedPurposes();
		ImportContext.BBoxCache->SetIncludedPurposes(PurposesToImport);
	}
	TOptional<float> TimeToRevertBBoxCacheTo;
	if (ImportContext.BBoxCache->GetTime() != ImportTime)
	{
		TimeToRevertBBoxCacheTo = ImportContext.BBoxCache->GetTime();
		ImportContext.BBoxCache->SetTime(ImportTime);
	}
	ImportContext.LevelSequenceHelper.SetBBoxCache(ImportContext.BBoxCache);

	// Shotgun approach to recreate all render states because we may want to reimport/delete/reassign a material/static/skeletalmesh while it is
	// currently being drawn
	FGlobalComponentRecreateRenderStateContext RecreateRenderStateContext;

	TSharedRef<FUsdSchemaTranslationContext> TranslationContext = MakeShared<FUsdSchemaTranslationContext>(
		ImportContext.Stage,
		*ImportContext.UsdAssetCache
	);
	TranslationContext->bIsImporting = true;
	TranslationContext->Level = ImportContext.World->GetCurrentLevel();
	TranslationContext->ObjectFlags = ImportContext.ImportObjectFlags;
	TranslationContext->Time = ImportTime;
	TranslationContext->PurposesToLoad = PurposesToImport;
	TranslationContext->NaniteTriangleThreshold = ImportContext.ImportOptions->NaniteTriangleThreshold;
	TranslationContext->RenderContext = ImportContext.ImportOptions->RenderContextToImport;
	TranslationContext->MaterialPurpose = ImportContext.ImportOptions->MaterialPurpose;
	TranslationContext->RootMotionHandling = ImportContext.ImportOptions->RootMotionHandling;
	TranslationContext->FallbackCollisionType = ImportContext.ImportOptions->FallbackCollisionType;
	TranslationContext->SubdivisionLevel = ImportContext.ImportOptions->SubdivisionLevel;
	TranslationContext->MetadataOptions = ImportContext.ImportOptions->MetadataOptions;
	TranslationContext->KindsToCollapse = (EUsdDefaultKind)ImportContext.ImportOptions->KindsToCollapse;
	TranslationContext->bUsePrimKindsForCollapsing = ImportContext.ImportOptions->bUsePrimKindsForCollapsing;
	TranslationContext->bMergeIdenticalMaterialSlots = ImportContext.ImportOptions->bMergeIdenticalMaterialSlots;
	TranslationContext->bShareAssetsForIdenticalPrims = ImportContext.ImportOptions->bShareAssetsForIdenticalPrims;
	TranslationContext->bAllowInterpretingLODs = ImportContext.ImportOptions->bInterpretLODs;
	TranslationContext->bAllowParsingSkeletalAnimations = ImportContext.ImportOptions->bImportGeometry
														  && ImportContext.ImportOptions->bImportSkeletalAnimations;
	TranslationContext->bAllowParsingGroomAssets = ImportContext.ImportOptions->bImportGroomAssets;
	TranslationContext->bAllowParsingSparseVolumeTextures = ImportContext.ImportOptions->bImportSparseVolumeTextures;
	TranslationContext->bAllowParsingSounds = ImportContext.ImportOptions->bImportSounds;
	TranslationContext->bTranslateOnlyUsedMaterials = ImportContext.ImportOptions->bImportOnlyUsedMaterials;
	TranslationContext->UsdInfoCache = &UsdInfoCache->GetInner();
	TranslationContext->PrimLinkCache = &PrimLinkCache->GetInner();
	TranslationContext->BBoxCache = ImportContext.BBoxCache;
	TranslationContext->BlendShapesByPath = &BlendShapesByPath;
	TranslationContext->GroomInterpolationSettings = ImportContext.ImportOptions->GroomInterpolationSettings;

	bool bFoundReimportPrim = false;
	{
		UE::USDStageImporter::Private::CacheCollapsingState(TranslationContext.Get());

		UE::FUsdPrim TargetPrim = ImportContext.Stage.GetPrimAtPath(UE::FSdfPath(*OriginalPrimPath));
		if (TargetPrim)
		{
			bFoundReimportPrim = true;
			UE::USDStageImporter::Private::CreateAssetsForPrims({TargetPrim}, TranslationContext.Get(), LOCTEXT("CreateAssets", "Creating assets"));
		}
		else
		{
			USD_LOG_USERERROR(FText::Format(
				LOCTEXT("ReimportMissingPrim", "Failed to find prim '{0}' in stage '{1}' to use for the reimport of asset '{2}'!"),
				FText::FromString(OriginalPrimPath),
				FText::FromString(ImportContext.FilePath),
				FText::FromString(OriginalAsset->GetPathName())
			));
		}
	}
	TranslationContext->CompleteTasks();

	TSet<FSoftObjectPath> ActiveAssetPaths = ImportContext.UsdAssetCache->GetActiveAssets();
	TSet<UObject*> ActiveAssets;
	ActiveAssets.Reserve(ActiveAssetPaths.Num());
	for (const FSoftObjectPath& ActivePath : ActiveAssetPaths)
	{
		ActiveAssets.Add(ActivePath.TryLoad());
	}
	ActiveAssets.Remove(nullptr);

	// Look for our reimported asset in the assets cache as we may have multiple assets with the same prim path
	UObject* ReimportedObject = nullptr;
	for (UObject* Asset : ActiveAssets)
	{
		UUsdAssetUserData* UserData = nullptr;
		if (IInterface_AssetUserData* UserDataInterface = Cast<IInterface_AssetUserData>(Asset))
		{
			UserData = UserDataInterface->GetAssetUserData<UUsdAssetUserData>();
		}

		if (Asset && UserData && Asset->GetClass() == OriginalAsset->GetClass() && UserData->PrimPaths.Contains(OriginalPrimPath))
		{
			ReimportedObject = Asset;
			break;
		}
	}

	if (ReimportedObject)
	{
		UE::USDStageImporter::Private::UpdateAssetImportData(ReimportedObject, ImportContext.FilePath, ImportContext.ImportOptions);

		// Assign things from the original assets before we publish the reimported asset, overwriting it
		UE::USDStageImporter::Private::CopyOriginalMaterialAssignment(ImportContext, OriginalAsset, ReimportedObject);
		UE::USDStageImporter::Private::CopySkeletonAssignment(ImportContext, OriginalAsset, ReimportedObject);

		// Just publish the one asset we wanted to reimport. Note that we may have other assets here too, but we'll ignore those e.g. a displayColor
		// material or a skeleton
		OutReimportedAsset = UE::USDStageImporter::Private::PublishAsset(
			ImportContext,
			ReimportedObject,
			OriginalAsset->GetOutermost()->GetPathName(),
			ObjectsToRemap,
			SoftObjectsToRemap,
			AssetsToFinalize
		);

		TSet<UObject*> PublishedAssetsAndDependencies{OutReimportedAsset};
		UE::USDStageImporter::Private::RemapReferences(ImportContext, PublishedAssetsAndDependencies, ObjectsToRemap);
		UE::USDStageImporter::Private::RemapSoftReferences(ImportContext, PublishedAssetsAndDependencies, SoftObjectsToRemap);
		UE::USDStageImporter::Private::PostRemapAssets(PublishedAssetsAndDependencies);

		bSuccess = OutReimportedAsset != nullptr && ActiveAssets.Contains(ReimportedObject);
	}
	else if (bFoundReimportPrim)
	{
		// If we're in here we did find the prim we wanted on the stage, but for some reason it didn't produce the
		// expected asset type
		USD_LOG_USERERROR(FText::Format(
			LOCTEXT(
				"ReimportMissingCompatibleAsset",
				"Failed to produce a '{0}' asset from prim '{1}' in stage '{2}' that could be used for the reimport of '{3}'!"
			),
			FText::FromString(OriginalAsset->GetClass()->GetName()),
			FText::FromString(OriginalPrimPath),
			FText::FromString(ImportContext.FilePath),
			FText::FromString(OriginalAsset->GetPathName())
		));
	}

	const bool bIsReimport = true;
	UE::USDStageImporter::Private::Cleanup(ImportContext.SceneActor, nullptr, ImportContext.ImportOptions->ExistingActorPolicy, AssetsToFinalize);
	UE::USDStageImporter::Private::BroadcastImportEvents({ReimportedObject}, bIsReimport);
	UE::USDStageImporter::Private::RefreshComponents(ImportContext.SceneActor, ImportContext.ImportOptions->bImportAtSpecificTimeCode);

	if (IncludedPurposesToRevertTo.IsSet())
	{
		ImportContext.BBoxCache->SetIncludedPurposes(IncludedPurposesToRevertTo.GetValue());
	}
	if (TimeToRevertBBoxCacheTo.IsSet())
	{
		ImportContext.BBoxCache->SetTime(TimeToRevertBBoxCacheTo.GetValue());
	}

	FUsdDelegates::OnPostUsdImport.Broadcast(ImportContext.FilePath);

	// Analytics
	{
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);
		UE::USDStageImporter::Private::SendAnalytics(ImportContext, ReimportedObject, TEXT("Reimport"), {ReimportedObject}, ElapsedSeconds);
		USD_LOG_INFO(TEXT("Re-imported '%s' in %.3f seconds."), *ImportContext.FilePath, ElapsedSeconds);

		UsdUnreal::Analytics::CollectSchemaAnalytics(ImportContext.Stage, TEXT("Reimport"));
	}

	UE::USDStageImporter::Private::CloseStageIfNeeded(ImportContext);

#endif	  // #if USE_USD_SDK
	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
