// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDStageImporterModule.h"

#include "UnrealUSDWrapper.h"
#include "USDMemory.h"
#include "USDStageImporter.h"
#include "USDStageImportOptionsCustomization.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Templates/UniquePtr.h"

#define LOCTEXT_NAMESPACE "UsdStageImporterModule"

class FUSDStageImporterModule : public IUsdStageImporterModule
{
public:
	virtual void StartupModule() override
	{
#if USE_USD_SDK
		LLM_SCOPE_BYTAG(Usd);

		FModuleManager::Get().LoadModuleChecked<IUnrealUSDWrapperModule>(TEXT("UnrealUSDWrapper"));

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.RegisterCustomClassLayout(
			TEXT("UsdStageImportOptions"),
			FOnGetDetailCustomizationInstance::CreateStatic(&FUsdStageImportOptionsCustomization::MakeInstance)
		);

		USDStageImporter = MakeUnique<UUsdStageImporter>();
#endif	  // #if USE_USD_SDK
	}

	virtual void ShutdownModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomClassLayout(TEXT("UsdStageImportOptions"));

		USDStageImporter.Reset();
	}

	class UUsdStageImporter* GetImporter() override
	{
		return USDStageImporter.Get();
	}

private:
	TUniquePtr<UUsdStageImporter> USDStageImporter;
};

IMPLEMENT_MODULE_USD(FUSDStageImporterModule, USDStageImporter);

#undef LOCTEXT_NAMESPACE
