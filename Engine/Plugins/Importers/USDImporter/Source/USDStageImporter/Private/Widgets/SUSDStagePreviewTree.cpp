// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SUSDStagePreviewTree.h"

#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "UsdStagePreviewTree"

namespace UE::UsdStagePreviewTree::Private
{
	const float CheckBoxColumnWidth = 24.0f;

	bool AreAllViewsCheckedRecursive(const SUsdStagePreviewTree& Tree, const TArray<FUsdPrimPreviewModelViewRef>& Views)
	{
		for (const FUsdPrimPreviewModelViewRef& View : Views)
		{
			if (!Tree.GetModel(View->ModelIndex).bShouldImport)
			{
				return false;
			}

			AreAllViewsCheckedRecursive(Tree, View->Children);
		}

		return true;
	};
}

class FUsdPreviewTreeImportColumn : public FUsdTreeViewColumn
{
public:
	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		const FUsdPrimPreviewModelViewWeak WeakTreeItem = StaticCastSharedPtr<FUsdPrimPreviewModelView>(InTreeItem);
		TSharedPtr<SUsdStagePreviewTreeRow> Row = StaticCastSharedPtr<SUsdStagePreviewTreeRow>(TableRow);
		TWeakPtr<SUsdStagePreviewTree> WeakTree = Row ? Row->GetOwnerTree() : nullptr;

		// clang-format off
		return SNew(SBox)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.WidthOverride(UE::UsdStagePreviewTree::Private::CheckBoxColumnWidth)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([WeakTreeItem, WeakTree]() -> ECheckBoxState
				{
					FUsdPrimPreviewModelViewPtr PinnedTreeItem = WeakTreeItem.Pin();
					TSharedPtr<SUsdStagePreviewTree> PinnedTree = WeakTree.Pin();
					if(PinnedTreeItem && PinnedTree)
					{
						FUsdPrimPreviewModel& Model = PinnedTree->GetModel(PinnedTreeItem->ModelIndex);
						return Model.bShouldImport ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}

					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([WeakTreeItem, WeakTree](ECheckBoxState NewCheckedState)
				{
					FUsdPrimPreviewModelViewPtr PinnedTreeItem = WeakTreeItem.Pin();
					TSharedPtr<SUsdStagePreviewTree> PinnedTree = WeakTree.Pin();
					if (PinnedTreeItem && PinnedTree)
					{
						if (NewCheckedState == ECheckBoxState::Checked)
						{
							// Check recursively when shift clicking (tree views support recursive expansion when
							// shift-clicking the expander arrows, so lets do that too on the checkboxes!)
							if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
							{
								const bool bChecked = true;
								PinnedTree->CheckItemsRecursively({PinnedTreeItem.ToSharedRef()}, bChecked);
							}
							else
							{
								FUsdPrimPreviewModel& Model = PinnedTree->GetModel(PinnedTreeItem->ModelIndex);
								Model.bShouldImport = true;

								// If we want to import some prim, we must import all ancestors too
								int32 ParentIndex = Model.ParentIndex;
								while (ParentIndex != INDEX_NONE)
								{
									FUsdPrimPreviewModel& ParentModel = PinnedTree->GetModel(ParentIndex);
									ParentModel.bShouldImport = true;
									ParentIndex = ParentModel.ParentIndex;
								}
							}
						}
						else
						{
							const bool bChecked = false;
							PinnedTree->CheckItemsRecursively({PinnedTreeItem.ToSharedRef()}, bChecked);
						}
					}
				})
			];
		// clang-format on
	}
};

class FUsdPreviewTreeNameColumn : public FUsdTreeViewColumn
{
public:
	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		FUsdPrimPreviewModelViewPtr TreeItem = StaticCastSharedPtr<FUsdPrimPreviewModelView>(InTreeItem);
		TSharedPtr<SUsdStagePreviewTreeRow> Row = StaticCastSharedPtr<SUsdStagePreviewTreeRow>(TableRow);
		TSharedPtr<SUsdStagePreviewTree> Tree = Row ? Row->GetOwnerTree() : nullptr;

		if (Tree && TreeItem)
		{
			FUsdPrimPreviewModel& Model = Tree->GetModel(TreeItem->ModelIndex);

			// clang-format off
			return SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.HighlightText(Tree->GetFilterText())
					.Text(Model.Name)
				];
			// clang-format on
		}

		return SNullWidget::NullWidget;
	}
};

class FUsdPreviewTreeTypeColumn : public FUsdTreeViewColumn
{
public:
	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> InTreeItem, const TSharedPtr<ITableRow> TableRow) override
	{
		FUsdPrimPreviewModelViewPtr TreeItem = StaticCastSharedPtr<FUsdPrimPreviewModelView>(InTreeItem);
		TSharedPtr<SUsdStagePreviewTreeRow> Row = StaticCastSharedPtr<SUsdStagePreviewTreeRow>(TableRow);
		TSharedPtr<SUsdStagePreviewTree> Tree = Row ? Row->GetOwnerTree() : nullptr;

		if (Tree && TreeItem)
		{
			FUsdPrimPreviewModel& Model = Tree->GetModel(TreeItem->ModelIndex);

			// clang-format off
			return SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.HighlightText(Tree->GetFilterText())
					.Text(Model.TypeName)
				];
			// clang-format on
		}

		return SNullWidget::NullWidget;
	}
};

void SUsdStagePreviewTree::Construct(const FArguments& InArgs, const UE::FUsdStage& InStage)
{
	SUsdTreeView::Construct(SUsdTreeView::FArguments());

	OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SUsdStagePreviewTree::ConstructPrimContextMenu);
	OnExpansionChanged = FOnExpansionChanged::CreateLambda(
		[this](FUsdPrimPreviewModelViewRef Item, bool bIsExpanded)
		{
			ItemModels[Item->ModelIndex].bIsExpanded = bIsExpanded;
		}
	);
	OnSetExpansionRecursive = FOnSetExpansionRecursive::CreateLambda(
		[this](FUsdPrimPreviewModelViewRef Item, bool bShouldExpand)
		{
			ExpandItemsRecursively({Item}, bShouldExpand);
		}
	);

	if (InStage)
	{
		if (UE::FUsdPrim RootPrim = InStage.GetPseudoRoot())
		{
			TFunction<int32(const UE::FUsdPrim&)> RecursivelyAddModels;
			RecursivelyAddModels = [this, &RecursivelyAddModels](const UE::FUsdPrim& Prim) -> int32
			{
				int32 NewIndex = ItemModels.Num();
				FUsdPrimPreviewModel& NewPrimModel = ItemModels.Emplace_GetRef();
				NewPrimModel.Name = FText::FromName(Prim.GetName());
				NewPrimModel.TypeName = FText::FromName(Prim.GetTypeName());

				TArray<UE::FUsdPrim> ChildPrims = Prim.GetChildren();
				NewPrimModel.ChildIndices.Reset(ChildPrims.Num());
				ItemModels.Reserve(ItemModels.Num() + ChildPrims.Num());
				for (const UE::FUsdPrim& ChildPrim : ChildPrims)
				{
					// WARNING: This will likely invalidate the NewPrimModel reference! Don't use it past this line!
					int32 NewChildIndex = RecursivelyAddModels(ChildPrim);

					ItemModels[NewChildIndex].ParentIndex = NewIndex;
					ItemModels[NewIndex].ChildIndices.Add(NewChildIndex);
				}

				return NewIndex;
			};

			TArray<UE::FUsdPrim> TopLevelPrims = RootPrim.GetChildren();
			ItemModels.Reset(TopLevelPrims.Num());	  // Not the best guess but better than nothing
			for (const UE::FUsdPrim& TopLevelPrim : TopLevelPrims)
			{
				RecursivelyAddModels(TopLevelPrim);
			}

			RebuildModelViews();
		}
	}
}

TArray<FString> SUsdStagePreviewTree::GetSelectedFullPrimPaths() const
{
	// This will eventually end up in a USD population mask, so we'll follow its rules, which essentially are:
	// - Any prims in this list will have *all* of their children imported;
	// - This list should not contain both a child and an ancestor.
	// This basically amounts to the rule that if a prim P has all of its descendants selected, we put P on the list
	// and no other descendant. Otherwise, we list all of the individual descendants that are enabled

	const static FString Divider = TEXT("/");

	// Since this is the default case, it may be worth it to try a quick scan to see if we're just trying to import
	// everything. If that's the case all we need to do is return the root prim path
	bool bShouldImportAllPrims = true;
	for (const FUsdPrimPreviewModel& Model : ItemModels)
	{
		if (!Model.bShouldImport)
		{
			bShouldImportAllPrims = false;
			break;
		}
	}
	if (bShouldImportAllPrims)
	{
		return {Divider};
	}

	TFunction<void(const TArray<int32>&, FString, TArray<FString>&, bool&)> RecursivelyCollectPaths;
	RecursivelyCollectPaths =
		[this,
		 &RecursivelyCollectPaths](const TArray<int32>& Indices, FString PathSoFar, TArray<FString>& OutChildPaths, bool& bOutAllDescendantsImported)
	{
		for (int32 Index : Indices)
		{
			const FUsdPrimPreviewModel& Model = ItemModels[Index];
			if (!Model.bShouldImport)
			{
				bOutAllDescendantsImported = false;
				continue;
			}

			FString PrimFullPath = PathSoFar + Model.Name.ToString();

			TArray<FString> ChildPaths;
			bool bAllDescendantsImported = true;
			RecursivelyCollectPaths(Model.ChildIndices, PrimFullPath + Divider, ChildPaths, bAllDescendantsImported);

			// If all of the Prim's descendents were imported, fully discard ChildPaths and just add Prim directly
			// to the list.
			// Alternatively, if we imported *nothing else* but we made it this far (i.e. we want to import this prim)
			// then we actually want to import this intermediate prim directly, so also add it directly to the list.
			if (bAllDescendantsImported || ChildPaths.Num() == 0)
			{
				OutChildPaths.Add(MoveTemp(PrimFullPath));
			}
			else
			{
				OutChildPaths.Append(MoveTemp(ChildPaths));
			}

			bOutAllDescendantsImported &= bAllDescendantsImported;
		}
	};

	TArray<int32> RootItemModels;
	RootItemModels.Reserve(RootItems.Num());
	for (const FUsdPrimPreviewModelViewRef& RootItem : RootItems)
	{
		RootItemModels.Add(RootItem->ModelIndex);
	}

	TArray<FString> Result;
	bool bUnusedDummy = false;	  // At this point we know we won't import *everything* anyway, don't bother checking
	RecursivelyCollectPaths(RootItemModels, Divider, Result, bUnusedDummy);
	return Result;
}

void SUsdStagePreviewTree::SetFilterText(const FText& NewText)
{
	CurrentFilterText = NewText;
	RebuildModelViews();
}

TSharedRef<ITableRow> SUsdStagePreviewTree::OnGenerateRow(FUsdPrimPreviewModelViewRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUsdStagePreviewTreeRow, InDisplayNode, OwnerTable, SharedData);
}

void SUsdStagePreviewTree::OnGetChildren(FUsdPrimPreviewModelViewRef InParent, TArray<FUsdPrimPreviewModelViewRef>& OutChildren) const
{
	OutChildren = InParent->Children;
}

void SUsdStagePreviewTree::SetupColumns()
{
	HeaderRowWidget->ClearColumns();

	// Import checkbox column with a checkbox on the header itself (so we can't just use AddColumn)
	{
		const static FName ImportColumnKey = TEXT("ImportColumn");

		SharedData->Columns.Add(ImportColumnKey, MakeShared<FUsdPreviewTreeImportColumn>());

		// clang-format off
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(ImportColumnKey)
			.FixedWidth(UE::UsdStagePreviewTree::Private::CheckBoxColumnWidth)
			.HeaderContentPadding(FMargin(0))
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.WidthOverride(UE::UsdStagePreviewTree::Private::CheckBoxColumnWidth)
				[
					SNew(SCheckBox)
					.ToolTipText(LOCTEXT("HeaderCheckboxToolTip", "Toggle all currently displayed items"))
					.IsChecked_Lambda([this]() -> ECheckBoxState
					{
						return UE::UsdStagePreviewTree::Private::AreAllViewsCheckedRecursive(*this, RootItems)
							? ECheckBoxState::Checked
							: ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						const bool bChecked = NewState == ECheckBoxState::Checked;
						CheckItemsRecursively(RootItems, bChecked);
					})
				]
			]
		);
		// clang-format on
	}

	// Prim name column
	{
		TSharedRef<FUsdPreviewTreeNameColumn> PrimNameColumn = MakeShared<FUsdPreviewTreeNameColumn>();
		PrimNameColumn->bIsMainColumn = true;

		SHeaderRow::FColumn::FArguments Arguments;
		Arguments.FillWidth(70.f);

		AddColumn(TEXT("Prim"), LOCTEXT("Prim", "Prim"), PrimNameColumn, Arguments);
	}

	// Prim type column
	{
		SHeaderRow::FColumn::FArguments Arguments;
		Arguments.FillWidth(20.f);
		AddColumn(TEXT("Type"), LOCTEXT("Type", "Type"), MakeShared<FUsdPreviewTreeTypeColumn>(), Arguments);
	}
}

TArray<FUsdPrimPreviewModelViewRef> SUsdStagePreviewTree::GetAncestorSelectedViews()
{
	TArray<FUsdPrimPreviewModelViewRef> SelectedViews = GetSelectedItems();

	TSet<int32> SelectedIndices;
	SelectedIndices.Reserve(SelectedViews.Num());
	for (const FUsdPrimPreviewModelViewRef& SelectedItem : SelectedViews)
	{
		SelectedIndices.Add(SelectedItem->ModelIndex);
	}

	TArray<FUsdPrimPreviewModelViewRef> AncestorSelectedViews;
	AncestorSelectedViews.Reserve(SelectedViews.Num());

	for (const FUsdPrimPreviewModelViewRef& SelectedView : SelectedViews)
	{
		bool bIsAncestor = true;

		int32 ParentIndex = ItemModels[SelectedView->ModelIndex].ParentIndex;
		while (ParentIndex != INDEX_NONE)
		{
			if (SelectedIndices.Contains(ParentIndex))
			{
				bIsAncestor = false;
				break;
			}

			ParentIndex = ItemModels[ParentIndex].ParentIndex;
		}

		if (bIsAncestor)
		{
			AncestorSelectedViews.Add(SelectedView);
		}
	}

	return AncestorSelectedViews;
}

TSharedPtr<SWidget> SUsdStagePreviewTree::ConstructPrimContextMenu()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	const TSharedPtr<const FUICommandList> CommandList = nullptr;
	FMenuBuilder MenuBuilder{bInShouldCloseWindowAfterMenuSelection, CommandList};

	MenuBuilder.BeginSection("Import", LOCTEXT("ImportText", "Import"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CheckDescendants_Text", "Check Recursively"),
			LOCTEXT("CheckDescendants_ToolTip", "Check this prim and all displayed descendants recursively"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						const bool bShouldSelect = true;
						CheckItemsRecursively(GetAncestorSelectedViews(), bShouldSelect);
					}
				),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("UncheckDescendants_Text", "Uncheck Recursively"),
			LOCTEXT("UncheckDescendants_ToolTip", "Uncheck this prim and all displayed descendants recursively"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						const bool bShouldSelect = false;
						CheckItemsRecursively(GetAncestorSelectedViews(), bShouldSelect);
					}
				),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Expansion", LOCTEXT("ExpansionText", "Expansion"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ExpandDescendants_Text", "Expand Recursively"),
			LOCTEXT("ExpandDescendants_ToolTip", "Expand this prim and all displayed descendants recursively"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						const bool bShouldExpand = true;
						ExpandItemsRecursively(GetAncestorSelectedViews(), bShouldExpand);
					}
				),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CollapseDescendants_Text", "Collapse Recursively"),
			LOCTEXT("CollapseDescendants_ToolTip", "Collapse this prim and all displayed descendants recursively"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(
					[this]()
					{
						const bool bShouldExpand = false;
						ExpandItemsRecursively(GetAncestorSelectedViews(), bShouldExpand);
					}
				),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SUsdStagePreviewTree::RebuildModelViews()
{
	const FString CurrentFilterString = CurrentFilterText.ToString().ToLower();

	// Update our models caching whether they pass the filter or not
	for (FUsdPrimPreviewModel& Model : ItemModels)
	{
		Model.bPassesFilter = CurrentFilterString.IsEmpty() || Model.Name.ToString().ToLower().Contains(CurrentFilterString)
							  || Model.TypeName.ToString().ToLower().Contains(CurrentFilterString);

		// If this model passes the filter, we need to ensure all of its ancestors are marked as passing too
		if (Model.bPassesFilter)
		{
			int32 ParentIndex = Model.ParentIndex;
			while (ParentIndex != INDEX_NONE)
			{
				FUsdPrimPreviewModel& ParentModel = ItemModels[ParentIndex];
				if (ParentModel.bPassesFilter)
				{
					break;
				}
				ParentModel.bPassesFilter = true;
				ParentIndex = ParentModel.ParentIndex;
			}
		}
	}

	// Rebuild and keep track of this mapping as its very cheap to do right now
	ItemModelsToViews.Reset();
	ItemModelsToViews.Reserve(ItemModels.Num());

	// Create a view for each model *that passes the filter*, having nullptr instead if they don't
	TArray<FUsdPrimPreviewModelViewPtr> LinearViews;
	LinearViews.SetNum(ItemModels.Num());
	for (int32 Index = 0; Index < ItemModels.Num(); ++Index)
	{
		const FUsdPrimPreviewModel& Model = ItemModels[Index];

		if (Model.bPassesFilter)
		{
			FUsdPrimPreviewModelViewRef View = MakeShared<FUsdPrimPreviewModelView>();
			View->ModelIndex = Index;

			ItemModelsToViews.Add(Index, View);
			LinearViews[Index] = MoveTemp(View);
		}
	}

	// Now that we have views created for each model we want, connect parent/child based on model indices
	RootItems.Reset();
	SparseItemInfos.Reset();
	for (int32 Index = 0; Index < ItemModels.Num(); ++Index)
	{
		const FUsdPrimPreviewModel& Model = ItemModels[Index];
		FUsdPrimPreviewModelViewPtr& View = LinearViews[Index];
		if (!View)
		{
			// Model didn't pass the filter
			continue;
		}

		// Set the view's parent
		if (Model.ParentIndex == INDEX_NONE)
		{
			RootItems.Add(View.ToSharedRef());
		}
		else
		{
			View->Parent = LinearViews[Model.ParentIndex];
		}

		// Add child models it they pass the filter, also already tracking if they're expanded or not
		bool bHasExpandedChildren = false;
		View->Children.Reserve(Model.ChildIndices.Num());
		for (int32 ChildIndex : Model.ChildIndices)
		{
			const FUsdPrimPreviewModel& ChildModel = ItemModels[ChildIndex];
			if (ChildModel.bPassesFilter)
			{
				View->Children.Add(LinearViews[ChildIndex].ToSharedRef());
				bHasExpandedChildren |= ChildModel.bIsExpanded;
			}
		}

		SparseItemInfos.Add(View.ToSharedRef(), FSparseItemInfo(Model.bIsExpanded, bHasExpandedChildren));
	}

	RequestTreeRefresh();
}

void SUsdStagePreviewTree::CheckItemsRecursively(const TArray<FUsdPrimPreviewModelViewRef>& Items, bool bCheck)
{
	TFunction<void(const TArray<FUsdPrimPreviewModelViewRef>&)> CheckViewsRecursively;
	CheckViewsRecursively = [this, &CheckViewsRecursively](const TArray<FUsdPrimPreviewModelViewRef>& Views)
	{
		for (const FUsdPrimPreviewModelViewRef& View : Views)
		{
			ItemModels[View->ModelIndex].bShouldImport = true;
			CheckViewsRecursively(View->Children);
		}
	};

	TArray<FString> UncheckedInvisiblePrims;
	TFunction<void(const TArray<int32>&)> UncheckModelsRecursively;
	UncheckModelsRecursively = [this, &UncheckModelsRecursively, &UncheckedInvisiblePrims](const TArray<int32>& ModelIndices)
	{
		for (int32 ModelIndex : ModelIndices)
		{
			FUsdPrimPreviewModel& Model = ItemModels[ModelIndex];

			if (Model.bShouldImport && !ItemModelsToViews.Contains(ModelIndex))
			{
				UncheckedInvisiblePrims.Add(Model.Name.ToString());
			}

			Model.bShouldImport = false;
			UncheckModelsRecursively(Model.ChildIndices);
		}
	};

	// Attention here: For consistency, if we are unchecking we must guarantee that no descendant models are ever left
	// checked, whether we have views for them or not. On the other hand if we are just checking something, it is fine
	// if not all of the model descendants are checked in the process, so let's only check what is shown on the screen
	if (bCheck)
	{
		CheckViewsRecursively(Items);

		// If we want to check a view, all of its ancestor *models* must also be checked, regardless of what is shown
		for (const FUsdPrimPreviewModelViewRef& View : Items)
		{
			int32 ParentIndex = ItemModels[View->ModelIndex].ParentIndex;
			while (ParentIndex != INDEX_NONE)
			{
				FUsdPrimPreviewModel& ParentModel = ItemModels[ParentIndex];
				ParentModel.bShouldImport = true;
				ParentIndex = ParentModel.ParentIndex;
			}
		}
	}
	else
	{
		TArray<int32> ModelIndices;
		ModelIndices.Reserve(Items.Num());
		for (const FUsdPrimPreviewModelViewRef& Item : Items)
		{
			ModelIndices.Add(Item->ModelIndex);
		}
		UncheckModelsRecursively(ModelIndices);

		// Notify user in case we unchecked some prims that currently are not being shown
		if (UncheckedInvisiblePrims.Num() > 0)
		{
			const int32 MaxNumPrimsToList = 15;
			const static FString Delimiter = TEXT(", ");
			FString PrimListString;
			for (int32 Index = 0; Index < FMath::Min(UncheckedInvisiblePrims.Num(), MaxNumPrimsToList); ++Index)
			{
				PrimListString += UncheckedInvisiblePrims[Index] + Delimiter;
			}
			if (UncheckedInvisiblePrims.Num() > MaxNumPrimsToList)
			{
				PrimListString += TEXT("...");
			}
			else
			{
				PrimListString.RemoveFromEnd(Delimiter);
			}

			FNotificationInfo Toast(LOCTEXT("UncheckedInvisibleTitle", "USD Prims to import"));
			Toast.SubText = FText::Format(
				UncheckedInvisiblePrims.Num() == 1
					? LOCTEXT("UncheckedInvisibleSingular", "This prim is not currently shown, but was also unchecked:\n\n{0}")
					: LOCTEXT("UncheckedInvisiblePlural", "These prims are not currently shown, but were also unchecked:\n\n{0}"),
				FText::FromString(PrimListString)
			);
			Toast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
			Toast.bUseLargeFont = false;
			Toast.bFireAndForget = true;
			Toast.FadeOutDuration = 0.5f;
			Toast.ExpireDuration = 4.5f;
			Toast.bUseThrobber = true;
			Toast.bUseSuccessFailIcons = false;
			FSlateNotificationManager::Get().AddNotification(Toast);
		}
	}
}

void SUsdStagePreviewTree::ExpandItemsRecursively(const TArray<FUsdPrimPreviewModelViewRef>& Items, bool bExpand)
{
	TFunction<void(const TArray<FUsdPrimPreviewModelViewRef>&)> ExpandRecursively;
	ExpandRecursively = [this, &ExpandRecursively, bExpand](const TArray<FUsdPrimPreviewModelViewRef>& Views)
	{
		for (const FUsdPrimPreviewModelViewRef& View : Views)
		{
			FUsdPrimPreviewModel& Model = ItemModels[View->ModelIndex];
			Model.bIsExpanded = bExpand;

			// We're going to expand recursively anyway, so we know it will have expanded children
			const bool bHasExpandedChildren = bExpand && View->Children.Num() > 0;
			const bool bIsExpanded = bExpand;
			SparseItemInfos.Add(View, FSparseItemInfo(bIsExpanded, bHasExpandedChildren));

			ExpandRecursively(View->Children);
		}
	};
	ExpandRecursively(Items);

	RequestTreeRefresh();
}

void SUsdStagePreviewTreeRow::Construct(
	const FArguments& InArgs,
	FUsdPrimPreviewModelViewRef InTreeItem,
	const TSharedRef<STableViewBase>& OwnerTable,
	TSharedPtr<FSharedUsdTreeData> InSharedData
)
{
	SUsdTreeRow<FUsdPrimPreviewModelViewRef>::Construct(
		typename SUsdTreeRow<FUsdPrimPreviewModelViewRef>::FArguments(),
		InTreeItem,
		OwnerTable,
		InSharedData
	);
}

TSharedPtr<SUsdStagePreviewTree> SUsdStagePreviewTreeRow::GetOwnerTree() const
{
	return StaticCastSharedPtr<SUsdStagePreviewTree>(OwnerTablePtr.Pin());
}

#undef LOCTEXT_NAMESPACE
