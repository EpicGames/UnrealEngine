// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

#include "UsdWrappers/ForwardDeclarations.h"

#define UE_API USDSTAGEIMPORTER_API

/**
 * Slate window used to show import/export options for the USDImporter plugin
 */
class SUsdOptionsWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SUsdOptionsWindow)
		: _OptionsObject(nullptr)
	{
	}
	SLATE_ARGUMENT(UObject*, OptionsObject)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(FText, AcceptText)
	SLATE_ARGUMENT(const UE::FUsdStage*, Stage)
	SLATE_END_ARGS()

public:
	// Show the options window with the given text
	static UE_API bool ShowOptions(UObject& OptionsObject, const FText& WindowTitle, const FText& AcceptText, const UE::FUsdStage* Stage = nullptr);

	// Shortcut functions that show the standard import/export text
	static UE_API bool ShowImportOptions(UObject& OptionsObject, const UE::FUsdStage* StageToImport);
	static UE_API bool ShowExportOptions(UObject& OptionsObject);

	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual bool SupportsKeyboardFocus() const override;

	UE_API FReply OnAccept();
	UE_API FReply OnCancel();

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API bool UserAccepted() const;
	UE_API TArray<FString> GetSelectedFullPrimPaths() const;

private:
	UObject* OptionsObject;

	TWeakPtr<SWindow> Window;
	TSharedPtr<class SUsdStagePreviewTree> StagePreviewTree;

	FText AcceptText;
	bool bAccepted;
};

#undef UE_API
