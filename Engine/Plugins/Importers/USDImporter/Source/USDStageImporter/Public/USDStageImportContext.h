// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Objects/USDInfoCache.h"
#include "USDLevelSequenceHelper.h"
#include "USDStageOptions.h"

#include "UsdWrappers/UsdStage.h"

#include "CoreMinimal.h"

#include "USDStageImportContext.generated.h"

#define UE_API USDSTAGEIMPORTER_API

class UUsdAssetCache2;
class UUsdAssetCache3;
class UUsdStageImportOptions;
class FTokenizedMessage;

USTRUCT()
struct FUsdStageImportContext
{
	GENERATED_BODY()

	UWorld* World;

	/**
	 * Whenever we spawn the scene actor, it should have this local transform and be attached to this parent.
	 * We use this so that Actions->Import can spawn the scene actor exactly where the original stage actor was
	 */
	FTransform TargetSceneActorTargetTransform;
	USceneComponent* TargetSceneActorAttachParent;

	/** Spawned actor that contains the imported scene as a child hierarchy */
	UPROPERTY()
	TObjectPtr<AActor> SceneActor;

	/** Name to use when importing a single mesh */
	UPROPERTY()
	FString ObjectName;

	UPROPERTY()
	FString PackagePath;

	/** Path of the main usd file to import */
	UPROPERTY()
	FString FilePath;

	UPROPERTY()
	TObjectPtr<UUsdStageImportOptions> ImportOptions;

	/** Keep track of the last imported object so that we have something valid to return to upstream code that calls the import factories */
	UPROPERTY()
	TObjectPtr<UObject> ImportedAsset;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> ImportedAssets;

	/** Level sequence that will contain the animation data during the import process */
	FUsdLevelSequenceHelper LevelSequenceHelper;

	UPROPERTY()
	TObjectPtr<UUsdAssetCache3> UsdAssetCache;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "Use the 'UsdAssetCache' member instead, which is of the new UUsdAssetCache3 type")
	UPROPERTY()
	TObjectPtr<UUsdAssetCache2> AssetCache_DEPRECATED;

	UE_DEPRECATED(all, "The import process now always builds its own InfoCache, so this member is no longer used")
	TSharedPtr<FUsdInfoCache> InfoCache_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Bounding box cache used for the USD stage in case we have to spawn bounds components */
	TSharedPtr<UE::FUsdGeomBBoxCache> BBoxCache;

	/**
	 * When parsing materials, we keep track of which primvar we mapped to which UV channel.
	 * When parsing meshes later, we use this data to place the correct primvar values in each UV channel.
	 */
	TMap<FString, TMap<FString, int32>> MaterialToPrimvarToUVIndex;

	/** USD Stage to import */
	UE::FUsdStage Stage;

	/** Object flags to apply to newly imported objects */
	EObjectFlags ImportObjectFlags;

	/** If true, options dialog won't be shown */
	bool bIsAutomated;

	/**
	 * If true, this will try loading the stage from the static stage cache before re-reading the file. If false,
	 * the USD file at FilePath is reopened (but the stage is left untouched).
	 **/
	bool bReadFromStageCache;

	/** If we're reading from the stage cache and the stage was originally open, it will be left open when the import is completed */
	bool bStageWasOriginallyOpenInCache;

	/** We modify the stage with our meters per unit import option on import. If the stage was already open, we use this to undo the changes after
	 * import */
	double OriginalMetersPerUnit;
	EUsdUpAxis OriginalUpAxis;

	/** If we need to run GC after the import is complete */
	bool bNeedsGarbageCollection;

public:
	UE_API FUsdStageImportContext();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FUsdStageImportContext() = default;
	FUsdStageImportContext(const FUsdStageImportContext& Other) = default;
	FUsdStageImportContext& operator=(const FUsdStageImportContext& Other) = default;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool Init(
		const FString& InName,
		const FString& InFilePath,
		const FString& InInitialPackagePath,
		EObjectFlags InFlags,
		bool bInIsAutomated,
		bool bIsReimport = false,
		bool bAllowActorImport = true
	);
	UE_API void Reset();

private:
	/** Error messages **/
	TArray<TSharedRef<FTokenizedMessage>> TokenizedErrorMessages;
};

#undef UE_API
