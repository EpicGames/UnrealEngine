// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UnrealUSDWrapper.h"
#include "USDMetadataImportOptions.h"
#include "USDStageOptions.h"

#include "CoreMinimal.h"
#include "GroomAssetInterpolation.h"

#include "USDStageImportOptions.generated.h"

#define UE_API USDSTAGEIMPORTER_API

enum class EUsdCollisionType : uint8;
struct FAnalyticsEventAttribute;

UENUM(BlueprintType)
enum class EReplaceActorPolicy : uint8
{
	/** Spawn new actors and components alongside the existing ones */
	Append,

	/** Replaces existing actors and components with new ones */
	Replace,

	/** Update transforms on existing actors but do not replace actors or components */
	UpdateTransform,

	/** Ignore any conflicting new assets and components, keeping the old ones */
	Ignore,
};

UENUM(BlueprintType)
enum class EReplaceAssetPolicy : uint8
{
	/** Create new assets with numbered suffixes */
	Append,

	/** Replaces existing asset with new asset */
	Replace,

	/** Ignores the new asset and keeps the existing asset */
	Ignore,
};

UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UUsdStageImportOptions : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "Actors"))
	bool bImportActors;

	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "Geometry"))
	bool bImportGeometry;

	/** Whether to try importing UAnimSequence skeletal animation assets for each encountered UsdSkelAnimQuery */
	UPROPERTY(
		BlueprintReadWrite,
		config,
		EditAnywhere,
		Category = "DataToImport",
		meta = (EditCondition = bImportGeometry, DisplayName = "Skeletal Animations")
	)
	bool bImportSkeletalAnimations;

	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "LevelSequences"))
	bool bImportLevelSequences;

	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "Materials & Textures"))
	bool bImportMaterials;

	/** Whether to import GroomAssets, GroomCaches and GroomBindings */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "Groom Assets"))
	bool bImportGroomAssets;

	/** Whether to import OpenVDB volumes as Sparse Volume Textures */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "Sparse Volume Textures"))
	bool bImportSparseVolumeTextures;

	/** Whether to import audio files referenced by UsdMediaSpatialAudio schemas as Unreal sound assets */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", meta = (DisplayName = "Audio"))
	bool bImportSounds;

	/**
	 * If this is checked, only materials actively used by the stage and import settings will be parsed.
	 * If this is unchecked, all materials present on the stage will be parsed.
	 */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "DataToImport", AdvancedDisplay, meta = (EditCondition = bImportMaterials))
	bool bImportOnlyUsedMaterials;

	/**
	 * List of paths of prims to import (e.g. ["/Root/MyBox", "/Root/OtherPrim"]).
	 * Importing a prim will import its entire subtree.
	 * If this list contains the root prim path the entire stage will be imported (default value).
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Prims to Import")
	TArray<FString> PrimsToImport = TArray<FString>{TEXT("/")};

	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options", meta = (InlineEditConditionToggle))
	bool bUseExistingAssetCache;

	/** Copy assets from an existing UsdAsset cache instead of generating them from scratch, if possible */
	UPROPERTY(
		BlueprintReadWrite,
		config,
		EditAnywhere,
		Category = "USD options",
		meta = (AllowedClasses = "/Script/USDClasses.UsdAssetCache3", EditCondition = "bUseExistingAssetCache")
	)
	FSoftObjectPath ExistingAssetCache = nullptr;

	/** Only import prims with these specific purposes from the USD file */
	UPROPERTY(
		BlueprintReadWrite,
		config,
		EditAnywhere,
		Category = "USD options",
		meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdPurpose")
	)
	int32 PurposesToImport;

	/** Try enabling Nanite for static meshes that are generated with at least this many triangles */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options", meta = (NoSpinbox = "true", UIMin = "0", ClampMin = "0"))
	int32 NaniteTriangleThreshold;

	/** Specifies which set of shaders to use when parsing USD materials, in addition to the universal render context. */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	FName RenderContextToImport;

	/** Specifies which material purpose to use when parsing USD material bindings, in addition to the "allPurpose" fallback */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	FName MaterialPurpose;

	// Describes what to add to the root bone animation within generated AnimSequences, if anything
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options", meta = (EditCondition = bImportSkeletalAnimations))
	EUsdRootMotionHandling RootMotionHandling = EUsdRootMotionHandling::NoAdditionalRootMotion;

	// What type of collision to use for static meshes generated from Prims that don't have physics schemas applied
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	EUsdCollisionType FallbackCollisionType;

	/** Subdivision level to use for all subdivision meshes on the opened stage. 0 means "don't subdivide" */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options", meta = (ClampMin = "0", ClampMax = "6", UIMin = "0", UIMax = "6"))
	int32 SubdivisionLevel;

	/* Describes if/how we should collect metadata from USD prims onto the assets and components we generate when importing */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	FUsdMetadataImportOptions MetadataOptions;

	/** Whether to use the specified StageOptions instead of the stage's own settings */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	bool bOverrideStageOptions;

	/** Custom StageOptions to use for the stage */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "USD options", meta = (EditCondition = bOverrideStageOptions))
	FUsdStageOptions StageOptions;

	/**
	 * When true the stage will be evaluated at ImportTimeCode for the import.
	 * When false, the stage will be evaluated at the default (non-animated) timecode
	 */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	bool bImportAtSpecificTimeCode;

	/** TimeCode to evaluate the stage for import, in case bImportAtSpecificTimeCode is enabled */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options", meta = (EditCondition = bImportAtSpecificTimeCode))
	float ImportTimeCode;

	/**
	 * If true, whenever two prims would have generated identical UAssets (like identical StaticMeshes or materials) then only one instance of
	 * that asset is generated, and the asset is shared by the components generated for both prims.
	 * If false, we will always generate a dedicated asset for each prim.
	 */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "USD options")
	bool bShareAssetsForIdenticalPrims;

	/** Groom group interpolation settings */
	UPROPERTY(EditAnywhere, config, BlueprintReadWrite, Category = "Groom")
	TArray<FHairGroupsInterpolation> GroomInterpolationSettings;

	/** What should happen when imported actors and components try to overwrite existing actors and components */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "Conflicts with Existing Objects", meta = (EditCondition = bImportActors))
	EReplaceActorPolicy ExistingActorPolicy;

	/** What should happen when imported assets try to overwrite existing assets */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "Conflicts with Existing Objects")
	EReplaceAssetPolicy ExistingAssetPolicy;

	UE_DEPRECATED(all, "Use 'Share Assets for Identical Prims' instead")
	UPROPERTY()
	bool bReuseIdenticalAssets_DEPRECATED;

	/**
	 * When enabled, assets will be imported into a content folder structure according to their prim path. When disabled,
	 * assets are imported into content folders according to asset type (e.g. 'Materials', 'StaticMeshes', etc).
	 */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "Processing")
	bool bPrimPathFolderStructure;

	/**
	 * Use KindsToCollapse to determine when to collapse prim subtrees or not (defaults to enabled).
	 * Disable this if you want to prevent collapsing, or to control it manually by right-clicking on individual prims.
	 */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "Processing")
	bool bUsePrimKindsForCollapsing;

	/**
	 * Whether to try to combine individual assets and components of the same type on a kind-per-kind basis,
	 * like multiple Mesh prims into a single Static Mesh
	 */
	UPROPERTY(
		BlueprintReadWrite,
		config,
		EditAnywhere,
		Category = "Processing",
		meta = (Bitmask, BitmaskEnum = "/Script/UnrealUSDWrapper.EUsdDefaultKind", EditCondition = bUsePrimKindsForCollapsing)
	)
	int32 KindsToCollapse;

	/**
	 * Identical material slots will be combined into a single slot if this is enabled. This is only performed in the context
	 * of mesh collapsing, or when parsing LOD variant sets (see bInterpretLODs)
	 */
	UPROPERTY(BlueprintReadWrite, config, EditAnywhere, Category = "Processing")
	bool bMergeIdenticalMaterialSlots;

	UE_DEPRECATED(all, "This option is now controlled via the cvar 'USD.CollapseTopLevelPointInstancers'")
	UPROPERTY()
	bool bCollapseTopLevelPointInstancers_DEPRECATED;

	/** When true, if a prim has a "LOD" variant set with variants named "LOD0", "LOD1", etc. where each contains a UsdGeomMesh, the importer will
	 * attempt to parse the meshes as separate LODs of a single UStaticMesh. When false, only the selected variant will be parsed as LOD0 of the
	 * UStaticMesh.  */
	UPROPERTY(
		BlueprintReadWrite,
		config,
		EditAnywhere,
		Category = "Processing",
		meta = (DisplayName = "Interpret LOD variant sets", ScriptName="InterpretLods;InterpretLODs", EditCondition = bImportGeometry)
	)
	bool bInterpretLODs;

public:
	UE_DEPRECATED(5.6, "Use FScopedSuppressActorImport instead.")
	UE_API void EnableActorImport(bool bEnable);

	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
};

namespace UsdUtils
{
	struct FScopedSuppressActorImport
	{
		FScopedSuppressActorImport(UUsdStageImportOptions* InOptions);
		~FScopedSuppressActorImport();

		FScopedSuppressActorImport(const FScopedSuppressActorImport& Other) = delete;
		FScopedSuppressActorImport(FScopedSuppressActorImport&& Other) = delete;
		FScopedSuppressActorImport& operator=(const FScopedSuppressActorImport& Other) = delete;
		FScopedSuppressActorImport& operator=(FScopedSuppressActorImport&& Other) = delete;

	private:
		bool bOldImportActorsValue = true;
		TWeakObjectPtr<UUsdStageImportOptions> Options;
	};

	USDSTAGEIMPORTER_API void AddAnalyticsAttributes(const UUsdStageImportOptions& Options, TArray<FAnalyticsEventAttribute>& InOutAttributes);
}

#undef UE_API
