// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API USDSTAGEIMPORTER_API

struct FUsdStageImportContext;

class UUsdStageImporter
{
public:
	UE_API void ImportFromFile(FUsdStageImportContext& ImportContext);

	UE_API bool ReimportSingleAsset(
		FUsdStageImportContext& ImportContext,
		UObject* OriginalAsset,
		const FString& OriginalPrimPath,
		UObject*& OutReimportedAsset
	);
};

#undef UE_API
