// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UUsdStageImporter;

class IUsdStageImporterModule : public IModuleInterface
{
public:
	static inline IUsdStageImporterModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IUsdStageImporterModule>("UsdStageImporter");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("UsdStageImporter");
	}

	virtual class UUsdStageImporter* GetImporter() = 0;
};
