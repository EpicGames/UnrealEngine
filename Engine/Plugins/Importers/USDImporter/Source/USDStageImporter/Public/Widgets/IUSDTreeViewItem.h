// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IUsdTreeViewItem : public TSharedFromThis<IUsdTreeViewItem>
{
public:
	virtual ~IUsdTreeViewItem() = default;
};
