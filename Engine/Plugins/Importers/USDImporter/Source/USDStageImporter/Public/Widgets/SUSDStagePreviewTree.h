// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UsdWrappers/ForwardDeclarations.h"
#include "Widgets/IUSDTreeViewItem.h"
#include "Widgets/SUSDTreeView.h"

#include "Templates/SharedPointer.h"

#define UE_API USDSTAGEIMPORTER_API

using FUsdPrimPreviewModelViewRef = TSharedRef<struct FUsdPrimPreviewModelView>;
using FUsdPrimPreviewModelViewPtr = TSharedPtr<struct FUsdPrimPreviewModelView>;
using FUsdPrimPreviewModelViewWeak = TWeakPtr<struct FUsdPrimPreviewModelView>;

struct FUsdPrimPreviewModel
{
	// Prim's name
	FText Name;

	// Prim's type name (like "Mesh" or "Camera")
	FText TypeName;

	// Whether this prim should be imported or not
	bool bShouldImport = true;

	// Only used to make filtering items with SUsdStagePreviewTree::CurrentFilterText a bit faster: This is not
	// actually used to display anything
	bool bPassesFilter = true;

	// Our source of truth for expansion state. Whenever expansion changes on the views we write to this,
	// and whenever we rebuild our views we write this to the tree's sparse infos about its items
	bool bIsExpanded = true;

	int32 ParentIndex = INDEX_NONE;
	TArray<int32> ChildIndices;
};

struct FUsdPrimPreviewModelView : public IUsdTreeViewItem
{
	int32 ModelIndex = INDEX_NONE;

	FUsdPrimPreviewModelViewWeak Parent;
	TArray<FUsdPrimPreviewModelViewRef> Children;
};

class SUsdStagePreviewTree : public SUsdTreeView<FUsdPrimPreviewModelViewRef>
{
public:
	SLATE_BEGIN_ARGS(SUsdStagePreviewTree)
	{
	}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const UE::FUsdStage& InStage);

	UE_API TArray<FString> GetSelectedFullPrimPaths() const;

	UE_API void SetFilterText(const FText& NewText);
	const FText& GetFilterText() const
	{
		return CurrentFilterText;
	}

	FUsdPrimPreviewModel& GetModel(int32 ModelIndex)
	{
		return ItemModels[ModelIndex];
	}
	const FUsdPrimPreviewModel& GetModel(int32 ModelIndex) const
	{
		return ItemModels[ModelIndex];
	}

	UE_API void CheckItemsRecursively(const TArray<FUsdPrimPreviewModelViewRef>& Items, bool bCheck);
	UE_API void ExpandItemsRecursively(const TArray<FUsdPrimPreviewModelViewRef>& Items, bool bExpand);

private:
	// Begin SUsdTreeView interface
	UE_API virtual TSharedRef<ITableRow> OnGenerateRow(FUsdPrimPreviewModelViewRef InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable) override;

	UE_API virtual void OnGetChildren(FUsdPrimPreviewModelViewRef InParent, TArray<FUsdPrimPreviewModelViewRef>& OutChildren) const override;

	UE_API virtual void SetupColumns() override;
	// End SUsdTreeView interface

	// Returns all selected model views, excluding child views of any other selected view.
	// i.e. if both a child and an ancestor view are selected, the returned array will contain only the ancestor.
	UE_API TArray<FUsdPrimPreviewModelViewRef> GetAncestorSelectedViews();

	UE_API TSharedPtr<SWidget> ConstructPrimContextMenu();

	// Rebuilds our RootItems array of FUsdPrimPreviewModelViewRefs based on CurrentFilterText and the ItemModels that
	// pass the filter
	UE_API void RebuildModelViews();

private:
	FText CurrentFilterText;

	TArray<FUsdPrimPreviewModel> ItemModels;
	TMap<int32, FUsdPrimPreviewModelViewRef> ItemModelsToViews;
};

// Custom row so that we can fetch the owning SUsdStagePreviewTree from the columns
class SUsdStagePreviewTreeRow : public SUsdTreeRow<FUsdPrimPreviewModelViewRef>
{
public:
	SLATE_BEGIN_ARGS(SUsdStagePreviewTreeRow)
	{
	}
	SLATE_END_ARGS()

	UE_API void Construct(
		const FArguments& InArgs,
		FUsdPrimPreviewModelViewRef InTreeItem,
		const TSharedRef<STableViewBase>& OwnerTable,
		TSharedPtr<FSharedUsdTreeData> InSharedData
	);

	UE_API TSharedPtr<SUsdStagePreviewTree> GetOwnerTree() const;
};

#undef UE_API
