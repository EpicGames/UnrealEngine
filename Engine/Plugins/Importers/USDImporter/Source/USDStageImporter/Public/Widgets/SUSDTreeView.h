// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

class IUsdTreeViewItem;

class FUsdTreeViewColumn
{
public:
	virtual ~FUsdTreeViewColumn() = default;

	virtual TSharedRef<SWidget> GenerateWidget(const TSharedPtr<IUsdTreeViewItem> TreeItem, const TSharedPtr<ITableRow> TableRow) = 0;

public:
	bool bIsMainColumn = false;
};

struct FSharedUsdTreeData
{
	TMap<FName, TSharedRef<FUsdTreeViewColumn>> Columns;
};

template<typename ItemType>
class SUsdTreeView : public STreeView<ItemType>
{
public:
	SLATE_BEGIN_ARGS(SUsdTreeView)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void AddColumn(
		FName ColumnKey,
		FText ColumnLabel,
		TSharedRef<FUsdTreeViewColumn> Column,
		SHeaderRow::FColumn::FArguments ColumnArguments = SHeaderRow::FColumn::FArguments()
	);
	virtual void SetupColumns() = 0;

	virtual TSharedRef<ITableRow> OnGenerateRow(ItemType InDisplayNode, const TSharedRef<STableViewBase>& OwnerTable) = 0;
	virtual void OnGetChildren(ItemType InParent, TArray<ItemType>& OutChildren) const = 0;

	virtual void OnTreeItemScrolledIntoView(ItemType TreeItem, const TSharedPtr<ITableRow>& Widget)
	{
	}

protected:
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TArray<ItemType> RootItems;

	TSharedPtr<FSharedUsdTreeData> SharedData;
};

template<typename ItemType>
inline void SUsdTreeView<ItemType>::AddColumn(
	FName ColumnKey,
	FText ColumnLabel,
	TSharedRef<FUsdTreeViewColumn> Column,
	SHeaderRow::FColumn::FArguments ColumnArguments
)
{
	ColumnArguments.ColumnId(ColumnKey);
	ColumnArguments.DefaultLabel(ColumnLabel);

	HeaderRowWidget->AddColumn(ColumnArguments);

	SharedData->Columns.Add(ColumnKey, MoveTemp(Column));
}

template<typename ItemType>
inline void SUsdTreeView<ItemType>::Construct(const FArguments& InArgs)
{
	SharedData = MakeShared<FSharedUsdTreeData>();

	HeaderRowWidget = SNew(SHeaderRow).Visibility(EVisibility::Visible);

	SetupColumns();

	STreeView<ItemType>::Construct(typename STreeView<ItemType>::FArguments()
									   .TreeItemsSource(&RootItems)
									   .OnGenerateRow(this, &SUsdTreeView<ItemType>::OnGenerateRow)
									   .OnGetChildren(this, &SUsdTreeView<ItemType>::OnGetChildren)
									   .OnItemScrolledIntoView(this, &SUsdTreeView<ItemType>::OnTreeItemScrolledIntoView)
									   .HeaderRow(HeaderRowWidget));
}

template<typename ItemType>
class SUsdTreeRow : public SMultiColumnTableRow<ItemType>
{
public:
	typedef typename STableRow<ItemType>::FOnCanAcceptDrop FUsdOnCanAcceptDrop;
	typedef typename STableRow<ItemType>::FOnAcceptDrop FUsdOnAcceptDrop;

	SLATE_BEGIN_ARGS(SUsdTreeRow)
	{
	}
	SLATE_EVENT(FUsdOnCanAcceptDrop, OnCanAcceptDrop)
	SLATE_EVENT(FUsdOnAcceptDrop, OnAcceptDrop)
	SLATE_EVENT(FOnDragDetected, OnDragDetected)
	SLATE_EVENT(FOnTableRowDragEnter, OnDragEnter)
	SLATE_EVENT(FOnTableRowDragLeave, OnDragLeave)
	SLATE_EVENT(FOnTableRowDrop, OnDrop)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		ItemType InTreeItem,
		const TSharedRef<STableViewBase>& OwnerTable,
		TSharedPtr<FSharedUsdTreeData> InSharedData
	)
	{
		TreeItem = InTreeItem;
		SharedData = InSharedData;

		SMultiColumnTableRow<ItemType>::Construct(
			typename SMultiColumnTableRow<ItemType>::FArguments()
				.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
				.OnAcceptDrop(InArgs._OnAcceptDrop)
				.OnDragDetected(InArgs._OnDragDetected)
				.OnDragEnter(InArgs._OnDragEnter)
				.OnDragLeave(InArgs._OnDragLeave)
				.OnDrop(InArgs._OnDrop),
			OwnerTable
		);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	TSharedPtr<typename ItemType::ElementType> TreeItem;
	TSharedPtr<FSharedUsdTreeData> SharedData;
};

template<typename ItemType>
inline TSharedRef<SWidget> SUsdTreeRow<ItemType>::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SWidget> ColumnWidget = SNullWidget::NullWidget;

	if (SharedData && SharedData->Columns.Contains(ColumnName))
	{
		ColumnWidget = SharedData->Columns[ColumnName]->GenerateWidget(
			StaticCastSharedPtr<IUsdTreeViewItem>(TSharedPtr<typename ItemType::ElementType>(TreeItem)),
			this->SharedThis(this)
		);
	}

	// clang-format off
	if (SharedData->Columns[ColumnName]->bIsMainColumn)
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			[
				SNew(SExpanderArrow, this->SharedThis(this))
				.IndentAmount(12)
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				ColumnWidget
			];
	}
	else
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				ColumnWidget
			];
	}
	// clang-format on
}
