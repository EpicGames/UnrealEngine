// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDStageImporter : ModuleRules
	{
		public USDStageImporter(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"Core",
					"CoreUObject",
					"EditorFramework",
					"EditorStyle", // For the font style on the stage actor customization
					"Engine",
					"GeometryCache",
					"InputCore",
					"JsonUtilities",
					"LevelSequence",
					"MainFrame",
					"MessageLog",
					"MovieScene",
					"PropertyEditor", // For the import options's details customization
					"RenderCore", // So that we can release resources of reimported meshes
					"Slate",
					"SlateCore",
					"UnrealEd",
					"UnrealUSDWrapper",
					"USDClasses",
					"USDSchemas",
					"USDStage",
					"USDUtilities",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"HairStrandsCore",
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
