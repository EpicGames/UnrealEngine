// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDTestsBlueprintLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDTestsBlueprintLibrary)

bool USDTestsBlueprintLibrary::RecompileBlueprintStageActor(AUsdStageActor* BlueprintDerivedStageActor)
{
	return false;
}

void USDTestsBlueprintLibrary::DirtyStageActorBlueprint(AUsdStageActor* BlueprintDerivedStageActor)
{
}

int64 USDTestsBlueprintLibrary::GetSubtreeVertexCount(AUsdStageActor* StageActor, const FString& PrimPath)
{
	return -1;
}

int64 USDTestsBlueprintLibrary::GetSubtreeMaterialSlotCount(AUsdStageActor* StageActor, const FString& PrimPath)
{
	return -1;
}

void USDTestsBlueprintLibrary::SetUsdStageCpp(AUsdStageActor* StageActor, const FString& NewStageRootLayer)
{
}

void USDTestsBlueprintLibrary::ClearTransactionHistory()
{
}
