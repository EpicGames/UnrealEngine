// Copyright Epic Games, Inc. All Rights Reserved.

#include "USDTestsModule.h"

#include "USDMemory.h"

class FUsdTestsModule : public IUsdTestsModule
{
public:
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Usd);
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE_USD(FUsdTestsModule, USDTests);
