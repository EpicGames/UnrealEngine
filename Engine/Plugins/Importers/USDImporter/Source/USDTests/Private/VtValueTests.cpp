// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "UnrealUSDWrapper.h"
#include "UsdWrappers/VtValue.h"

#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FVtValueTests, "Editor.Import.USD.VtValueConversions", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

using FUsdMetadataMap = TMap<FString, UE::FVtValue>;  // Can't have a comma in the SPECIALIZE() argument list or the preprocessor thinks it's two arguments

bool operator==(const FVector2DHalf& A, const FVector2DHalf& B)
{
	return A.X == B.X && A.Y == B.Y;
}

bool operator==(const FUsdMetadataMap& A, const FUsdMetadataMap& B)
{
	return A.OrderIndependentCompareEqual(B);
}

bool FVtValueTests::RunTest(const FString& Parameters)
{
#define TEST_TYPE(TYPE, VALUE)                  \
	{                                           \
		const TYPE In = VALUE;                  \
		UE::FVtValue OrigValue;                 \
		OrigValue.Set<TYPE>(In);                \
		TYPE Out = OrigValue.Get<TYPE>();       \
		TestEqual(#TYPE " roundtrip", In, Out); \
	}
	TEST_TYPE(bool, true);
	TEST_TYPE(uint8, 6);
	TEST_TYPE(int32, -641);
	TEST_TYPE(uint32, 54715);
	TEST_TYPE(int64, -41456);
	TEST_TYPE(uint64, 74615);
	TEST_TYPE(FFloat16, FFloat16(2.4f));
	TEST_TYPE(float, 1.3e5);
	TEST_TYPE(double, 2.5e10);
	TEST_TYPE(FSdfTimeCode, 1.2);
	TEST_TYPE(FString, TEXT("hello"));
	TEST_TYPE(FName, TEXT("FName test"));
	TEST_TYPE(FSdfAssetPath, FSdfAssetPath(TEXT("First path"), TEXT("Second path")));

	FMatrix2D Matrix2D;
	Matrix2D.Row0 = FVector2D(2.6f, -1.4f);
	Matrix2D.Row1 = FVector2D(64.0f, -0.2f);
	TEST_TYPE(FMatrix2D, Matrix2D);

	FMatrix3D Matrix3D;
	Matrix3D.Row0 = FVector(2.5f, 45.f, -65.f);
	Matrix3D.Row1 = FVector(25.0f, -6.f, 6.f);
	Matrix3D.Row2 = FVector(0.0f, 0.0f, -1.0f);
	TEST_TYPE(FMatrix3D, Matrix3D);

	TEST_TYPE(FMatrix44d, FMatrix44d(FTransform3d(FRotator3d(25.0, 2.0, 3.14159), FVector3d(150.0, -203.0, 4500.7), FVector3d(1.0, 1.0, 1.0)).ToMatrixWithScale()));
	TEST_TYPE(FMatrix44f, FMatrix44f(FTransform3f(FRotator3f(25.0f, 2.0f, 3.14159f), FVector3f(150.0f, -203.0f, 4500.7f), FVector3f(1.0f, 1.0f, 1.0f)).ToMatrixWithScale()));
	TEST_TYPE(FQuat4d, FQuat4d(-0.7, 0.7, 0.0, 1.2));
	TEST_TYPE(FQuat4f, FQuat4f(-0.7f, 0.7f, 0.0f, 1.2f));
	TEST_TYPE(FQuat4h, FQuat4h(-0.7f, 0.7f, 0.0f, 1.2f));
	TEST_TYPE(FVector2d, FVector2d(2.5f, -2.6f));
	TEST_TYPE(FVector2f, FVector2f(150.0f, -203.0f));
	TEST_TYPE(FVector2DHalf, FVector2DHalf(1.2f, -2.4f));
	TEST_TYPE(FIntPoint, FIntPoint(2, -6));
	TEST_TYPE(FVector3d, FVector3d(150.0, -203.0, 4500.7));
	TEST_TYPE(FVector3f, FVector3f(150.0f, -203.0f, 4500.7f));
	TEST_TYPE(FVector3h, FVector3h(150.0f, -203.0f, 4500.7f));
	TEST_TYPE(FIntVector, FIntVector(1, -6, 200));
	TEST_TYPE(FVector4d, FVector4d(150.0, -203.0, 4500.7, -2.0));
	TEST_TYPE(FVector4f, FVector4f(150.0f, -203.0f, 4500.7f, -2.0f));
	TEST_TYPE(FVector4h, FVector4f(150.0f, -203.0f, 4500.7f, -2.0f));
	TEST_TYPE(FIntRect, FIntRect(50, -6, 200, 2));

	FUsdMetadataMap MetadataMap;
	UE::FVtValue First;
	First.Set<int32>(4651);
	UE::FVtValue Second;
	Second.Set<FString>(TEXT("some value"));
	UE::FVtValue Third;
	MetadataMap.Add(TEXT("Firstkey"), First);
	MetadataMap.Add(TEXT("SecondKey"), Second);
	MetadataMap.Add(TEXT("ThirdKey"), Third);
	TEST_TYPE(FUsdMetadataMap, MetadataMap);

	TEST_TYPE(TArray<bool>, TArray<bool>({false, true, true, false}));
	TEST_TYPE(TArray<uint8>, TArray<uint8>({0x7f, 65}));
	TEST_TYPE(TArray<int32>, TArray<int32>({0x7fccabcd, 655}));
	TEST_TYPE(TArray<uint32>, TArray<uint32>({0x85ccabcd}));
	TEST_TYPE(TArray<int64>, TArray<int64>({0x7fccabcd12345678, 456154}));
	TEST_TYPE(TArray<uint64>, TArray<uint64>({0x85ccabcd12345678}));
	TEST_TYPE(TArray<FFloat16>, TArray<FFloat16>({0.12f, 0.0f, -0.2f}));
	TEST_TYPE(TArray<float>, TArray<float>({0.12f, 0.0f, -0.2f}));
	TEST_TYPE(TArray<double>, TArray<double>({0.12, 0.0, -0.2}));
	TEST_TYPE(TArray<FSdfTimeCode>, TArray<FSdfTimeCode>({0.12, 0.0, -0.2}));
	TEST_TYPE(TArray<FString>, TArray<FString>({TEXT("First"), TEXT("Second"), TEXT("")}));
	TEST_TYPE(TArray<FName>, TArray<FName>({TEXT("FirstName"), TEXT("SecondName"), TEXT("")}));
	TEST_TYPE(TArray<FSdfAssetPath>, TArray<FSdfAssetPath>({
		FSdfAssetPath(TEXT("First asset path"), TEXT("First resolved path")),
		FSdfAssetPath(TEXT("Second asset path")),
		FSdfAssetPath()
	}));

	TArray<FMatrix2D> Matrix2Ds;
	Matrix2Ds.Add(Matrix2D);
	Matrix2Ds.Add(Matrix2D);
	Matrix2Ds.Add(FMatrix2D());
	TEST_TYPE(TArray<FMatrix2D>, Matrix2Ds);

	TArray<FMatrix3D> Matrix3Ds;
	Matrix3Ds.Add(Matrix3D);
	Matrix3Ds.Add(Matrix3D);
	Matrix3Ds.Add(FMatrix3D());
	TEST_TYPE(TArray<FMatrix3D>, Matrix3Ds);

	TEST_TYPE(TArray<FMatrix44d>, TArray<FMatrix44d>({
		(FTransform3d(FRotator3d(25.0, 2.0, 3.14159), FVector3d(150.0, -203.0, 4500.7), FVector3d(1.0, 1.0, 1.0)).ToMatrixWithScale()),
		(FTransform3d(FRotator3d(0.0, -2.0, 5), FVector3d(15.0, -2.0, 40.7), FVector3d(4.0, 4.0, 4.0)).ToMatrixWithScale()),
		FMatrix44d::Identity
	}));
	TEST_TYPE(TArray<FMatrix44f>, TArray<FMatrix44f>({
		(FTransform3f(FRotator3f(25.0f, 2.0f, 3.14159f), FVector3f(150.0f, -203.0f, 4500.7f), FVector3f(1.0f, 1.0f, 1.0f)).ToMatrixWithScale()),
		(FTransform3f(FRotator3f(0.0f, -2.0f, 3.1f), FVector3f(150.0f, 23.0f, 450.7f), FVector3f(2.0f, 2.0f, 2.0f)).ToMatrixWithScale()),
		FMatrix44f::Identity
	}));
	TEST_TYPE(TArray<FQuat4d>, TArray<FQuat4d>({FQuat4d(1.0, 1.0, 0, 1.0), FQuat4d(1.0f, 1.0f, 0.0f, 2.0f), FQuat4d::Identity}));
	TEST_TYPE(TArray<FQuat4f>, TArray<FQuat4f>({FQuat4f(1.0f, 1.0f, 0.0f, 1.0f), FQuat4f(-1.0f, -1.0f, 0.0f, 1.0f), FQuat4f()}));
	TEST_TYPE(TArray<FQuat4h>, TArray<FQuat4h>({FQuat4h(1.0f, 1.0f, 0.0f, 1.0f), FQuat4h(-1.0f, -1.0f, 0.0f, 1.0f), FQuat4h()}));
	TEST_TYPE(TArray<FVector2d>, TArray<FVector2d>({FVector2d(150.0, -203.0), FVector2d(16550.0, 203.0), FVector2d()}));
	TEST_TYPE(TArray<FVector2f>, TArray<FVector2f>({FVector2f(150.0f, -203.0f), FVector2f(15065.0f, -03.0f), FVector2f()}));
	TEST_TYPE(TArray<FVector2DHalf>, TArray<FVector2DHalf>({FVector2DHalf(150.0, -203.0)}));
	TEST_TYPE(TArray<FIntPoint>, TArray<FIntPoint>({FIntPoint{1, 4}, FIntPoint{0, 0}}));
	TEST_TYPE(TArray<FVector3d>, TArray<FVector3d>({FVector3d(150.0, -203.0, 4500.7), FVector3d()}));
	TEST_TYPE(TArray<FVector3f>, TArray<FVector3f>({FVector3f(150.0f, -203.0f, 4500.7f), FVector3f()}));
	TEST_TYPE(TArray<FVector3h>, TArray<FVector3h>({FVector3h(150.0f, -203.0f, 4500.7f), FVector3h()}));
	TEST_TYPE(TArray<FIntVector>, TArray<FIntVector>({FIntVector(1, 4, 3), FIntVector()}));
	TEST_TYPE(TArray<FVector4d>, TArray<FVector4d>({FVector4d(150.0, -203.0, 4500.7, 2.0), FVector4d()}));
	TEST_TYPE(TArray<FVector4f>, TArray<FVector4f>({FVector4f(150.0f, -203.0f, 4500.7f, 2.0f), FVector4f()}));
	TEST_TYPE(TArray<FVector4h>, TArray<FVector4h>({FVector4h(150.0f, -203.0f, 4500.7f, 2.0f), FVector4h()}));
	TEST_TYPE(TArray<FIntRect>, TArray<FIntRect>({FIntRect(-1,-2, 1, 2), FIntRect(-1,-2, 1, 2), FIntRect()}));
#undef TEST_TYPE

	return true;
}

#endif // #if WITH_DEV_AUTOMATION_TESTS