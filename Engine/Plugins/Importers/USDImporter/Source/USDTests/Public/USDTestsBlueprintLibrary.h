// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "USDTestsBlueprintLibrary.generated.h"

#define UE_API USDTESTS_API

UCLASS(MinimalAPI, meta = (ScriptName = "USDTestingLibrary"))
class UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release") USDTestsBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release.")
	UFUNCTION()
	static UE_API bool RecompileBlueprintStageActor(AUsdStageActor* BlueprintDerivedStageActor);

	UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release.")
	UFUNCTION()
	static UE_API void DirtyStageActorBlueprint(AUsdStageActor* BlueprintDerivedStageActor);

	UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release.")
	UFUNCTION()
	static UE_API int64 GetSubtreeVertexCount(AUsdStageActor* StageActor, const FString& PrimPath);

	UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release.")
	UFUNCTION()
	static UE_API int64 GetSubtreeMaterialSlotCount(AUsdStageActor* StageActor, const FString& PrimPath);

	UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release.")
	UFUNCTION()
	static UE_API void SetUsdStageCpp(AUsdStageActor* StageActor, const FString& NewStageRootLayer);

	UE_DEPRECATED(all, "These functions were meant for internal testing only and will be removed in a future release.")
	UFUNCTION()
	static UE_API void ClearTransactionHistory();
};

#undef UE_API
