// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool.Rules
{
	public class USDTests : ModuleRules
	{
		public USDTests(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"UnrealUSDWrapper",
					"USDClasses",
					"USDSchemas",
					"USDStage",
					"USDUtilities",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"BlueprintGraph",
						"UnrealEd",
					}
				);
			}

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}
