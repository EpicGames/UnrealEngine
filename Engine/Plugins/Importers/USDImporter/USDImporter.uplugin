{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "USD Importer",
	"Description": "Adds support for importing the USD file format into Unreal Engine",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "USDSchemas",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDStage",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDStageEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDStageImporter",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDStageEditorViewModels",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDExporter",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
        {
			"Name": "USDClassesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "GeometryCacheUSD",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		},
		{
			"Name": "USDTests",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64", "Mac", "Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "USDCore",
			"Enabled": true
		},
		{
			"Name": "AnalyticsBlueprintLibrary",
			"Enabled": true
		},
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "Interchange",
			"Enabled": true,
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		},
		{
			"Name": "HairStrands",
			"Enabled": true
		}
	]
}
